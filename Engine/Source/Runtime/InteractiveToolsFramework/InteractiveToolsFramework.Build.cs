// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InteractiveToolsFramework : ModuleRules
{
	public InteractiveToolsFramework(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "Core",
                "CoreUObject",
                "GeometryCore",
                "InputCore",
                "ApplicationCore",
				"MeshDescription"
				// ... add other public dependencies that you statically link with here ...
			}
            );			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
                "RHI",
				"RenderCore", // for gizmo components
				"Slate",
				//"SlateCore",
				// ... add private dependencies that you statically link with here ...	
			}
			);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("EditorToolEvents");
		}
	}
}
