// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/AnyButtonInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnyButtonInputBehavior)



UAnyButtonInputBehavior::UAnyButtonInputBehavior()
{
	SetUseLeftMouseButton();
}


EInputDevices UAnyButtonInputBehavior::GetSupportedDevices()
{
	return EInputDevices::Mouse;
}


bool UAnyButtonInputBehavior::IsPressed(const FInputDeviceState& input)
{
	if (input.IsFromDevice(EInputDevices::Mouse)) 
	{
		ActiveDevice = EInputDevices::Mouse;
		return GetButtonStateFunc(input).bPressed;
	} 
	else if (input.IsFromDevice(EInputDevices::TabletFingers))
	{
		ActiveDevice = EInputDevices::TabletFingers;
		//return input.TouchPressed;  // not implemented yet
		return false;
	}
	return false;
}

bool UAnyButtonInputBehavior::IsDown(const FInputDeviceState& input)
{
	if (input.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		return GetButtonStateFunc(input).bDown;
	}
	return false;
}

bool UAnyButtonInputBehavior::IsReleased(const FInputDeviceState& input)
{
	if (input.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		return GetButtonStateFunc(input).bReleased;
	}
	return false;
}

bool UAnyButtonInputBehavior::IsDoubleClicked(const FInputDeviceState& Input)
{
	if (Input.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		return GetButtonStateFunc(Input).bDoubleClicked;
	}
	return false;
}

FVector2D UAnyButtonInputBehavior::GetClickPoint(const FInputDeviceState& input)
{
	if (input.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		return input.Mouse.Position2D;
	}
	return FVector2D::ZeroVector;
}


FRay UAnyButtonInputBehavior::GetWorldRay(const FInputDeviceState& input)
{
	if (input.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		return input.Mouse.WorldRay;
	}
	return FRay(FVector::ZeroVector, FVector(0, 0, 1), true);

}


FInputDeviceRay UAnyButtonInputBehavior::GetDeviceRay(const FInputDeviceState& input)
{
	if (input.IsFromDevice(EInputDevices::Mouse))
	{
		ActiveDevice = EInputDevices::Mouse;
		return FInputDeviceRay(input.Mouse.WorldRay, input.Mouse.Position2D);
	}
	return FInputDeviceRay(FRay(FVector::ZeroVector, FVector(0, 0, 1), true));

}


EInputDevices UAnyButtonInputBehavior::GetActiveDevice() const
{
	return ActiveDevice;
}



void UAnyButtonInputBehavior::SetUseLeftMouseButton()
{
	GetMouseButtonStateFunc = [](const FInputDeviceState& input)
	{
		return input.Mouse.Left;
	};
}

void UAnyButtonInputBehavior::SetUseMiddleMouseButton()
{
	GetMouseButtonStateFunc = [](const FInputDeviceState& input)
	{
		return input.Mouse.Middle;
	};
}

void UAnyButtonInputBehavior::SetUseRightMouseButton()
{
	GetMouseButtonStateFunc = [](const FInputDeviceState& input)
	{
		return input.Mouse.Right;
	};
}

void UAnyButtonInputBehavior::SetUseCustomMouseButton(TUniqueFunction<FDeviceButtonState(const FInputDeviceState& Input)> ButtonFunc)
{
	GetMouseButtonStateFunc = MoveTemp(ButtonFunc);
}



FDeviceButtonState UAnyButtonInputBehavior::GetButtonStateFunc(const FInputDeviceState& Input)
{
	if (ActiveDevice == EInputDevices::Mouse)
	{
		return GetMouseButtonStateFunc(Input);
	}
	return FDeviceButtonState();
}
