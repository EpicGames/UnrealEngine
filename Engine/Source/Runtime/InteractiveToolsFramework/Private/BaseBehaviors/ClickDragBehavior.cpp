// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/ClickDragBehavior.h"

#include "InputBehaviorBindingComplexity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClickDragBehavior)



UClickDragInputBehavior::UClickDragInputBehavior()
{
}


void UClickDragInputBehavior::Initialize(IClickDragBehaviorTarget* TargetIn)
{
	check(TargetIn != nullptr);
	this->Target = TargetIn;
}

FInputCapturePriority UClickDragInputBehavior::GetPriority()
{
	const FInputCapturePriority BasePriority = Super::GetPriority();
	
	if (UE::ITF::Private::UseInputBindingComplexity())
	{
		return BasePriority.ApplyBindingComplexity(Modifiers.GetActiveModifierCount());
	}
	
	return BasePriority;
}

FInputCaptureRequest UClickDragInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	bInClickDrag = false;	// should never be true here, but weird things can happen w/ focus

	if (IsPressed(Input) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)) )
	{
		FInputRayHit HitResult = Target->CanBeginClickDragSequence(GetDeviceRay(Input));
		if (HitResult.bHit)
		{
			Modifiers.CacheModifierState(Input);
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UClickDragInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	Modifiers.UpdateModifiers(Input, Target);
	OnClickPressInternal(Input, Side);
	bInClickDrag = true;
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate UClickDragInputBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	if (bUpdateModifiersDuringDrag)
	{
		Modifiers.UpdateModifiers(Input, Target);
	}

	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid ray data.
	if ((GetSupportedDevices() & Input.InputDevice) == EInputDevices::None)
	{
		return FInputCaptureUpdate::Continue();
	}

	if (IsReleased(Input)) 
	{
		OnClickReleaseInternal(Input, Data);
		bInClickDrag = false;
		return FInputCaptureUpdate::End();
	}
	else
	{
		OnClickDragInternal(Input, Data);
		return FInputCaptureUpdate::Continue();
	}
}


void UClickDragInputBehavior::ForceEndCapture(const FInputCaptureData& Data)
{
	Target->OnTerminateDragSequence();
	bInClickDrag = false;
}



void UClickDragInputBehavior::OnClickPressInternal(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	Target->OnClickPress(GetDeviceRay(Input));
}

void UClickDragInputBehavior::OnClickDragInternal(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	Target->OnClickDrag(GetDeviceRay(Input));
}

void UClickDragInputBehavior::OnClickReleaseInternal(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	Target->OnClickRelease(GetDeviceRay(Input));
}

