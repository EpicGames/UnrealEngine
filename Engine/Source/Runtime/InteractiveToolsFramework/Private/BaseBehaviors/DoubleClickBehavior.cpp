// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/DoubleClickBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DoubleClickBehavior)


UDoubleClickInputBehavior::UDoubleClickInputBehavior()
{
	HitTestOnRelease = true;
}


FInputCaptureRequest UDoubleClickInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	if (IsDoubleClicked(Input) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)) )
	{
		FInputRayHit HitResult = Target->IsHitByClick(GetDeviceRay(Input));
		if (HitResult.bHit)
		{
			Modifiers.CacheModifierState(Input);
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}


