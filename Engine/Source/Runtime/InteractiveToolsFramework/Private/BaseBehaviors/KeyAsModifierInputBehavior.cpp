// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/KeyAsModifierInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KeyAsModifierInputBehavior)

UKeyAsModifierInputBehavior::UKeyAsModifierInputBehavior()
{
}

void UKeyAsModifierInputBehavior::Initialize(IModifierToggleBehaviorTarget* TargetIn, int ModifierID, const FKey& ModifierKeyIn)
{
	Initialize(TargetIn, ModifierID, [ModifierKeyIn](const FInputDeviceState& Input)
	{
		return (ModifierKeyIn == EKeys::AnyKey || Input.Keyboard.ActiveKey.Button == ModifierKeyIn) && Input.Keyboard.ActiveKey.bPressed;
	});
}

void UKeyAsModifierInputBehavior::Initialize(IModifierToggleBehaviorTarget* TargetIn, int ModifierID, TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc)
{
	this->Target = TargetIn;
	Modifiers.RegisterModifier(ModifierID, ModifierCheckFunc);
}

FInputCaptureRequest UKeyAsModifierInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	Modifiers.UpdateModifiers(Input, Target);
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UKeyAsModifierInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	return FInputCaptureUpdate::End();
}


FInputCaptureUpdate UKeyAsModifierInputBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	return FInputCaptureUpdate::End();
}

bool UKeyAsModifierInputBehavior::WantsForceEndCapture()
{
	return true;
}

void UKeyAsModifierInputBehavior::ForceEndCapture(const FInputCaptureData& Data)
{
	if (Target)
	{
		Target->OnForceEndCapture();
	}
}
