// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/KeyInputBehavior.h"
#include "Algo/Unique.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(KeyInputBehavior)

UKeyInputBehavior::UKeyInputBehavior()
{
}

void UKeyInputBehavior::Initialize(IKeyInputBehaviorTarget* TargetIn, const FKey& KeyIn)
{
	this->Target = TargetIn;
	this->TargetKeys.Add(KeyIn);
	InitializeKeyActivations();
	
}

void UKeyInputBehavior::Initialize(IKeyInputBehaviorTarget* TargetIn, const TArray<FKey>& KeysIn)
{
	Target = TargetIn;

	TargetKeys = KeysIn;
	TargetKeys.Sort();
	TargetKeys.SetNum(Algo::Unique(TargetKeys)); // Clean up array in case we have duplicate keys

	InitializeKeyActivations();
}

FInputCaptureRequest UKeyInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	if (bAlwaysUpdateModifiers)
	{
		Modifiers.UpdateModifiers(Input, Target);
	}

	InitializeKeyActivations();
	
	if (IsTargetedKey(Input) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)))
	{
		return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any);
	}
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UKeyInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	Modifiers.UpdateModifiers(Input, Target);
	UpdateActivations(Input, !bRequireAllKeys);

	if ((bRequireAllKeys && AreAllKeysPressed()))
	{
		OnKeyPressedInternal(Input);
		bAllKeysSeenPressed = true;
	}

	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate UKeyInputBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	Modifiers.UpdateModifiers(Input, Target);
	bool bRelevantKey = UpdateActivations(Input, !bRequireAllKeys);

	if (!bRelevantKey)
	{
		return FInputCaptureUpdate::Continue();
	}

	if (bRequireAllKeys && AreAllKeysPressed())
	{
		OnKeyPressedInternal(Input);
		bAllKeysSeenPressed = true;
		return FInputCaptureUpdate::Continue();
	}

	if (bRequireAllKeys && (!AreAllKeysPressed() && bAllKeysSeenPressed) )
	{
		OnKeyReleasedInternal(Input);
		bAllKeysSeenPressed = false;
		return FInputCaptureUpdate::End();
	}

	if (!bRequireAllKeys && !IsAnyKeyPressed())
	{
		return FInputCaptureUpdate::End();
	}

	return FInputCaptureUpdate::Continue();
}

void UKeyInputBehavior::ForceEndCapture(const FInputCaptureData& Data)
{
	if (Target && (bAllKeysSeenPressed || !bRequireAllKeys))
	{
		Target->OnForceEndCapture();
	}
}

bool UKeyInputBehavior::WantsForceEndCapture()
{
	return bAlwaysUpdateModifiers;
}

bool UKeyInputBehavior::IsKeyPressed(FKey InKey) const
{
	const int32 KeyIndex = TargetKeys.Find(InKey);
	if (KeyIndex > -1)
	{
		return KeyActivations[KeyIndex];
	}
	return false;
}

void UKeyInputBehavior::OnKeyPressedInternal(const FInputDeviceState& Input)
{
	if (Target)
	{
		Target->OnKeyPressed(Input.Keyboard.ActiveKey.Button);
	}
}

void UKeyInputBehavior::OnKeyReleasedInternal(const FInputDeviceState& Input)
{
	if (Target)
	{
		Target->OnKeyReleased(Input.Keyboard.ActiveKey.Button);
	}
}

void UKeyInputBehavior::InitializeKeyActivations()
{
	KeyActivations.SetNum(TargetKeys.Num());
	for (bool& Activation : KeyActivations)
	{
		Activation = false;
	}
}

bool UKeyInputBehavior::IsTargetedKey(const FInputDeviceState& Input)
{
	return (Input.InputDevice == EInputDevices::Keyboard) &&
		   (Input.Keyboard.ActiveKey.bPressed) &&
		   (TargetKeys.Contains(Input.Keyboard.ActiveKey.Button));
}

bool UKeyInputBehavior::UpdateActivations(const FInputDeviceState& Input, bool bEmitOnChange)
{
	if ((Input.InputDevice == EInputDevices::Keyboard) && (Input.Keyboard.ActiveKey.bPressed))
	{
		int32 KeyIndex = TargetKeys.Find(Input.Keyboard.ActiveKey.Button);
		if (KeyIndex > -1)
		{
			KeyActivations[KeyIndex] = true;
			if (bEmitOnChange)
			{
				OnKeyPressedInternal(Input);
			}
			return true;
		}
	}
	if ((Input.InputDevice == EInputDevices::Keyboard) && (Input.Keyboard.ActiveKey.bReleased))
	{
		int32 KeyIndex = TargetKeys.Find(Input.Keyboard.ActiveKey.Button);
		if (KeyIndex > -1)
		{
			KeyActivations[KeyIndex] = false;
			if (bEmitOnChange)
			{
				OnKeyReleasedInternal(Input);
			}
			return true;
		}
	}

	return false;
}

bool UKeyInputBehavior::IsAnyKeyPressed() const
{
	return KeyActivations.Contains(true);
}

bool UKeyInputBehavior::AreAllKeysPressed() const
{
	return !KeyActivations.Contains(false);
}

