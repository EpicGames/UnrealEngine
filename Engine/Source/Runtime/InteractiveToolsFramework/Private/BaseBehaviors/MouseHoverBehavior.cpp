// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/MouseHoverBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MouseHoverBehavior)


UMouseHoverBehavior::UMouseHoverBehavior()
{
	Target = nullptr;
}

EInputDevices UMouseHoverBehavior::GetSupportedDevices()
{
	return EInputDevices::Mouse;
}

void UMouseHoverBehavior::Initialize(IHoverBehaviorTarget* TargetIn)
{
	this->Target = TargetIn;
}

bool UMouseHoverBehavior::WantsHoverEvents()
{
	return true;
}


FInputCaptureRequest UMouseHoverBehavior::WantsHoverCapture(const FInputDeviceState& InputState)
{
	if (Target != nullptr && ((HoverModifierCheckFunc == nullptr || HoverModifierCheckFunc(InputState))) )
	{
		Modifiers.UpdateModifiers(InputState, Target);

		FInputRayHit Hit = Target->BeginHoverSequenceHitTest(FInputDeviceRay(InputState));
		if (Hit.bHit)
		{
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, Hit.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate UMouseHoverBehavior::BeginHoverCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide)
{
	check(Target != nullptr);
	Modifiers.UpdateModifiers(InputState, Target);
	Target->OnBeginHover(FInputDeviceRay(InputState));
	return FInputCaptureUpdate::Begin(this, eSide);
}

FInputCaptureUpdate UMouseHoverBehavior::UpdateHoverCapture(const FInputDeviceState& InputState)
{
	check(Target != nullptr);
	Modifiers.UpdateModifiers(InputState, Target);

	// Check device here because we may get keyboard inputs for updating the modifier, and they don't have a valid ray.
	if ((GetSupportedDevices() & InputState.InputDevice) == EInputDevices::None)
	{
		return FInputCaptureUpdate::Continue();
	}

	if (Target->OnUpdateHover(FInputDeviceRay(InputState)))
	{
		return FInputCaptureUpdate::Continue();
	}
	return FInputCaptureUpdate::End();
}

void UMouseHoverBehavior::EndHoverCapture()
{
	check(Target != nullptr);
	Target->OnEndHover();
}




