// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/MouseWheelBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MouseWheelBehavior)

UMouseWheelInputBehavior::UMouseWheelInputBehavior()
{
}

void UMouseWheelInputBehavior::Initialize(IMouseWheelBehaviorTarget* TargetIn)
{
	this->Target = TargetIn;
}

FInputCaptureRequest UMouseWheelInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	if (Input.Mouse.WheelDelta != 0 && (ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)))
	{
		FInputRayHit HitResult = Target->ShouldRespondToMouseWheel(FInputDeviceRay(Input));
		if (HitResult.bHit)
		{
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UMouseWheelInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide)
{
	check(Input.Mouse.WheelDelta != 0);

	Modifiers.UpdateModifiers(Input, Target);

	if (Input.Mouse.WheelDelta > 0)
	{
		Target->OnMouseWheelScrollUp(Input);
	}
	else
	{
		Target->OnMouseWheelScrollDown(Input);
	}

	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate UMouseWheelInputBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	return FInputCaptureUpdate::End();
}


void UMouseWheelInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	// nothing to do
}


