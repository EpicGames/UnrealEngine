// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/MouseWheelModifierInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MouseWheelModifierInputBehavior)

FInputCaptureRequest UMouseWheelModifierInputBehavior::WantsCapture(const FInputDeviceState& InputState)
{
	if (InputState.Mouse.WheelDelta != 0 && (ModifierCheckFunc == nullptr || ModifierCheckFunc(InputState)))
	{
		if (InputState.Mouse.WheelDelta > 0)
		{
			Target->OnMouseWheelScrollUp(InputState);
		}
		else
		{
			Target->OnMouseWheelScrollDown(InputState);
		}
	}

	return FInputCaptureRequest::Ignore();
}
