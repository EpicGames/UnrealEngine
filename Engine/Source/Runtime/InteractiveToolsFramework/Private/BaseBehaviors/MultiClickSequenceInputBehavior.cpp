// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/MultiClickSequenceInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiClickSequenceInputBehavior)



UMultiClickSequenceInputBehavior::UMultiClickSequenceInputBehavior()
{
}


void UMultiClickSequenceInputBehavior::Initialize(IClickSequenceBehaviorTarget* TargetIn)
{
	this->Target = TargetIn;
}

FInputCaptureRequest UMultiClickSequenceInputBehavior::WantsCapture(const FInputDeviceState& input)
{
	if (!Target)
	{
		return FInputCaptureRequest::Ignore();
	}

	if (IsPressed(input))
	{
		switch (State)
		{
		case ESequenceState::NotStarted:
		{
			if ((ModifierCheckFunc == nullptr || ModifierCheckFunc(input))
				&& Target->CanBeginClickSequence(GetDeviceRay(input)))
			{
				return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any);
			}
			break;
		}
		case ESequenceState::WaitingForNextClick:
		{
			// TODO: we could consider doing the extra checks for non-first clicks as well.
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any);
		}
		}
	}

	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UMultiClickSequenceInputBehavior::BeginCapture(const FInputDeviceState& input, EInputCaptureSide eSide)
{
	check(Target); // WantsCapture should not have allowed this if we didn't have a target

	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate UMultiClickSequenceInputBehavior::UpdateCapture(const FInputDeviceState& input, const FInputCaptureData& data)
{
	check(Target);

	Modifiers.UpdateModifiers(input, Target);

	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid mouse data.
	if (!input.IsFromDevice(GetSupportedDevices()))
	{
		return FInputCaptureUpdate::Continue();
	}

	// Allow target to abort click sequence
	if (State == ESequenceState::WaitingForNextClick && Target->RequestAbortClickSequence())
	{
		Target->OnTerminateClickSequence();
		State = ESequenceState::NotStarted;
		return FInputCaptureUpdate::End();
	}

	// Look for click completion
	if (IsReleased(input))
	{
		switch (State)
		{
		case ESequenceState::NotStarted:
		{
			// Extra check when beginning the sequence
			if (Target->CanBeginClickSequence(GetDeviceRay(input)))
			{
				Target->OnBeginClickSequence(GetDeviceRay(input));
				State = ESequenceState::WaitingForNextClick;
			}
			break;
		}
		case ESequenceState::WaitingForNextClick:
		{
			bool bContinue = Target->OnNextSequenceClick(GetDeviceRay(input));
			if (!bContinue)
			{
				// Done with the click sequence
				State = ESequenceState::NotStarted;
			}
			break;
		}
		}

		// Mouse release always ends the click capture
		return FInputCaptureUpdate::End();
	}
	return FInputCaptureUpdate::Continue();
}

void UMultiClickSequenceInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	// Only affects us if we were in the middle of a sequence
	if (Target && State == ESequenceState::WaitingForNextClick)
	{
		Target->OnTerminateClickSequence();
	}
}

bool UMultiClickSequenceInputBehavior::WantsHoverEvents()
{
	return true;
}

FInputCaptureRequest UMultiClickSequenceInputBehavior::WantsHoverCapture(const FInputDeviceState& InputState)
{
	return (Target != nullptr) && ((HoverModifierCheckFunc == nullptr || HoverModifierCheckFunc(InputState))) ? FInputCaptureRequest::Begin(this, EInputCaptureSide::Any) : FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate UMultiClickSequenceInputBehavior::BeginHoverCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide) 
{
	check(Target); // WantsHoverCapture shouldn't have allowed this without a target

	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UMultiClickSequenceInputBehavior::UpdateHoverCapture(const FInputDeviceState& InputState)
{
	check(Target);

	Modifiers.UpdateModifiers(InputState, Target);

	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid mouse data.
	if (!InputState.IsFromDevice(GetSupportedDevices()))
	{
		return FInputCaptureUpdate::Continue();
	}

	// Allow target to abort click sequence
	if (State == ESequenceState::WaitingForNextClick && Target->RequestAbortClickSequence())
	{
		Target->OnTerminateClickSequence();
		State = ESequenceState::NotStarted;
		return FInputCaptureUpdate::End();
	}

	// Dispatch the correct preview callback
	switch (State)
	{
	case ESequenceState::NotStarted:
	{
		Target->OnBeginSequencePreview(FInputDeviceRay(InputState.Mouse.WorldRay, InputState.Mouse.Position2D));
		break;
	}
	case ESequenceState::WaitingForNextClick:
	{
		Target->OnNextSequencePreview(FInputDeviceRay(InputState.Mouse.WorldRay, InputState.Mouse.Position2D));
		break;
	}
	}

	return FInputCaptureUpdate::Continue();
}


void UMultiClickSequenceInputBehavior::EndHoverCapture()
{
	// Nothing to do.
}

