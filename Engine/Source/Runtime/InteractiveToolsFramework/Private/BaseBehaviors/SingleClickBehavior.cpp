// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/SingleClickBehavior.h"

#include "InputBehaviorBindingComplexity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleClickBehavior)



USingleClickInputBehavior::USingleClickInputBehavior()
{
	HitTestOnRelease = true;
}


void USingleClickInputBehavior::Initialize(IClickBehaviorTarget* TargetIn)
{
	this->Target = TargetIn;
}


FInputCapturePriority USingleClickInputBehavior::GetPriority()
{
	const FInputCapturePriority BasePriority = Super::GetPriority();
	
	if (UE::ITF::Private::UseInputBindingComplexity())
	{
		return BasePriority.ApplyBindingComplexity(Modifiers.GetActiveModifierCount());
	}
	
	return BasePriority;
}


FInputCaptureRequest USingleClickInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	if (IsPressed(Input) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)) )
	{
		FInputRayHit HitResult = Target->IsHitByClick(GetDeviceRay(Input));
		if (HitResult.bHit)
		{
			Modifiers.CacheModifierState(Input);
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate USingleClickInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide)
{
	Modifiers.UpdateModifiers(Input, Target);
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate USingleClickInputBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	Modifiers.UpdateModifiers(Input, Target);
	if (IsReleased(Input))
	{
		if (HitTestOnRelease == false || 
			Target->IsHitByClick(GetDeviceRay(Input)).bHit )
		{
			Clicked(Input, Data);
		}

		return FInputCaptureUpdate::End();
	}

	return FInputCaptureUpdate::Continue();
}


void USingleClickInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	// nothing to do
}


void USingleClickInputBehavior::Clicked(const FInputDeviceState& input, const FInputCaptureData& data)
{
	Target->OnClicked(GetDeviceRay(input));
}


