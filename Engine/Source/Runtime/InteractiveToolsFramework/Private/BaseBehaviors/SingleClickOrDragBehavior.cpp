// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/SingleClickOrDragBehavior.h"

#include "InputBehaviorBindingComplexity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleClickOrDragBehavior)



void USingleClickOrDragInputBehavior::Initialize(IClickBehaviorTarget* ClickTargetIn, IClickDragBehaviorTarget* DragTargetIn)
{
	this->ClickTarget = ClickTargetIn;
	this->DragTarget = DragTargetIn;
}

void USingleClickOrDragInputBehavior::SetDragTarget(IClickDragBehaviorTarget* DragTargetIn)
{
	if ( ensure(!bInDrag) )
	{
		this->DragTarget = DragTargetIn;
	}
}

FInputCapturePriority USingleClickOrDragInputBehavior::GetPriority()
{
	const FInputCapturePriority BasePriority = Super::GetPriority();
	
	if (UE::ITF::Private::UseInputBindingComplexity())
	{
		return BasePriority.ApplyBindingComplexity(Modifiers.GetActiveModifierCount());
	}
	
	return BasePriority;
}

FInputCaptureRequest USingleClickOrDragInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	bImmediatelyBeginDragInBeginCapture = false;
	if (IsPressed(Input) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)) )
	{
		FInputRayHit ClickHitResult = (ClickTarget) ? ClickTarget->IsHitByClick(GetDeviceRay(Input)) : FInputRayHit();
		if (ClickHitResult.bHit)
		{
			Modifiers.CacheModifierState(Input);
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, ClickHitResult.HitDepth);
		}
		else if (bBeginDragIfClickTargetNotHit)
		{
			FInputRayHit DragHitResult = DragTarget->CanBeginClickDragSequence(GetDeviceRay(Input));
			if (DragHitResult.bHit)
			{
				Modifiers.CacheModifierState(Input);
				bImmediatelyBeginDragInBeginCapture = true;
				return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, DragHitResult.HitDepth);
			}
		}
	}
	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate USingleClickOrDragInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide)
{
	if ( ClickTarget != nullptr )
	{
		Modifiers.UpdateModifiers(Input, ClickTarget);
	}
	if ( DragTarget != nullptr )
	{
		Modifiers.UpdateModifiers(Input, DragTarget);
	}

	ensure(Input.IsFromDevice(EInputDevices::Mouse));	// todo: handle other devices
	MouseDownPosition2D = Input.Mouse.Position2D;	
	MouseDownRay = Input.Mouse.WorldRay;
	CaptureSide = eSide;
	bInDrag = false;

	// If the WantsCapture() hit-test did not hit the target, then start the drag action directly, instead of starting
	// a click and then switching to a drag (could alternately repeat the hit-test here...)
	if (bImmediatelyBeginDragInBeginCapture)
	{
		bInDrag = true;
		OnClickDragPressInternal(Input, CaptureSide);
	}

	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate USingleClickOrDragInputBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	if ( ClickTarget != nullptr )
	{
		Modifiers.UpdateModifiers(Input, ClickTarget);
	}
	if ( DragTarget != nullptr )
	{
		Modifiers.UpdateModifiers(Input, DragTarget);
	}

	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid mouse data.
	if (!Input.IsFromDevice(GetSupportedDevices()))
	{
		return FInputCaptureUpdate::Continue();
	}

	// check if mouse has moved far enough that we want to swap to drag behavior
	if (bInDrag == false)
	{
		float MouseMovement = static_cast<float>(FVector2D::Distance(Input.Mouse.Position2D, MouseDownPosition2D));
		if (MouseMovement > ClickDistanceThreshold)
		{
			FInputDeviceState StartInput = Input;
			StartInput.Mouse.Position2D = MouseDownPosition2D;		// attempt to reconstruct the input state that would have existed in WantsCapture/BeginCapture
			StartInput.Mouse.WorldRay = MouseDownRay;
			if ( DragTarget )
			{
				FInputRayHit DragHitResult = DragTarget->CanBeginClickDragSequence(GetDeviceRay(StartInput));
				if (DragHitResult.bHit)
				{
					bInDrag = true;
					OnClickDragPressInternal(StartInput, CaptureSide);
				}
			}
		}
	}

	if (IsReleased(Input))
	{
		if (bInDrag)
		{
			OnClickDragReleaseInternal(Input, Data);
			bInDrag = false;
		}
		else   // click path
		{
			if (ClickTarget && ClickTarget->IsHitByClick(GetDeviceRay(Input)).bHit )
			{
				OnClickedInternal(Input, Data);
			}
		}

		return FInputCaptureUpdate::End();
	}
	else
	{
		if (bInDrag)
		{
			OnClickDragInternal(Input, Data);
		}
		return FInputCaptureUpdate::Continue();
	}
}


void USingleClickOrDragInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	if (bInDrag)
	{
		if ( DragTarget )
		{
			DragTarget->OnTerminateDragSequence();
		}
		bInDrag = false;
	}

	// nothing to do
}


void USingleClickOrDragInputBehavior::OnClickedInternal(const FInputDeviceState& input, const FInputCaptureData& data)
{
	if ( ClickTarget )
	{
		ClickTarget->OnClicked(GetDeviceRay(input));
	}
}



void USingleClickOrDragInputBehavior::OnClickDragPressInternal(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	if ( DragTarget )
	{
		DragTarget->OnClickPress(GetDeviceRay(Input));
	}
}

void USingleClickOrDragInputBehavior::OnClickDragInternal(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	if ( DragTarget )
	{
		DragTarget->OnClickDrag(GetDeviceRay(Input));
	}
}

void USingleClickOrDragInputBehavior::OnClickDragReleaseInternal(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	if ( DragTarget )
	{
		DragTarget->OnClickRelease(GetDeviceRay(Input));
	}
}

