// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/SingleKeyCaptureBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleKeyCaptureBehavior)

USingleKeyCaptureBehavior::USingleKeyCaptureBehavior()
{
}

void USingleKeyCaptureBehavior::Initialize(IModifierToggleBehaviorTarget* TargetIn, int ModifierID, const FKey& ModifierKeyIn)
{
	this->Target = TargetIn;
	ModifierKey = ModifierKeyIn;
	FKey TempKey = ModifierKeyIn;
	Modifiers.RegisterModifier(ModifierID, [TempKey](const FInputDeviceState& Input)
		{
			return (TempKey == EKeys::AnyKey || Input.Keyboard.ActiveKey.Button == TempKey) && Input.Keyboard.ActiveKey.bPressed;
		});
}

FInputCaptureRequest USingleKeyCaptureBehavior::WantsCapture(const FInputDeviceState& Input)
{
	if ((ModifierCheckFunc == nullptr || ModifierCheckFunc(Input)))
	{
		if ((ModifierKey == EKeys::AnyKey || Input.Keyboard.ActiveKey.Button == ModifierKey) && Input.Keyboard.ActiveKey.bPressed)
		{
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any);
		}
	}
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate USingleKeyCaptureBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	PressedButton = Input.Keyboard.ActiveKey.Button;
	Modifiers.UpdateModifiers(Input, Target);
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}


FInputCaptureUpdate USingleKeyCaptureBehavior::UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	if (Input.Keyboard.ActiveKey.Button != PressedButton)
	{
		return FInputCaptureUpdate::Continue();
	}

	Modifiers.UpdateModifiers(Input, Target);

	if (Input.Keyboard.ActiveKey.bReleased)
	{
		return FInputCaptureUpdate::End();
	}

	return FInputCaptureUpdate::Continue();
}


void USingleKeyCaptureBehavior::ForceEndCapture(const FInputCaptureData& Data)
{
}


