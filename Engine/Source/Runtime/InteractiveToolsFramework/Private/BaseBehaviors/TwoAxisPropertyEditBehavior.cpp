// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "GenericPlatform/GenericPlatformApplicationMisc.h"
#include "ToolContextInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TwoAxisPropertyEditBehavior)

#define LOCTEXT_NAMESPACE "UTwoAxisPropertyEditInputBehavior"

void UTwoAxisPropertyEditInputBehavior::Initialize(ITwoAxisPropertyEditBehaviorTarget* InTarget)
{
	check(InTarget);
	Target = InTarget;
	
	ResetDragState();
}

void UTwoAxisPropertyEditInputBehavior::ResetOrigin(FVector2D InScreenPosition, bool bHorizontalAdjust, bool bResetStartOrigin)
{
	using IPropertyInterface = ITwoAxisPropertyEditBehaviorTarget::IPropertyInterface;

	if (bResetStartOrigin)
	{
		State.StartOrigin = InScreenPosition;
	}
	
	State.bAdjustingHorizontally = bHorizontalAdjust;
	State.CurrentOrigin = InScreenPosition;
	
	if (IPropertyInterface* Property = State.bAdjustingHorizontally ? Target->GetHorizontalProperty() : Target->GetVerticalProperty(); Property && Property->IsEnabled())
	{
		State.StartValue = Property->GetValue();
	}
}

void UTwoAxisPropertyEditInputBehavior::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) const
{
	using IPropertyInterface = ITwoAxisPropertyEditBehaviorTarget::IPropertyInterface;
	
	if (!IsEditing())
	{
		return;
	}

	FText BrushAdjustmentMessage;
	if (IPropertyInterface* Property = State.bAdjustingHorizontally ? Target->GetHorizontalProperty() : Target->GetVerticalProperty(); Property && Property->IsEnabled())
	{
		BrushAdjustmentMessage = FText::Format(LOCTEXT("BrushAdjustmentMessage", "{0}: {1}"), Property->GetName(), FText::AsNumber(Property->GetValue()));
	}
	else
	{
		return;
	}

	FCanvasTextItem TextItem(State.StartOrigin, BrushAdjustmentMessage, GEngine->GetMediumFont(), FLinearColor::White);
	TextItem.EnableShadow(FLinearColor::Black);
	Canvas->DrawItem(TextItem);
}

void UTwoAxisPropertyEditInputBehavior::OnDragStart(FVector2D InScreenPosition)
{
	bInDrag = true;
	constexpr bool bHorizontalAdjust = true;
	constexpr bool bResetStartOrigin = true;
	ResetOrigin(InScreenPosition, bHorizontalAdjust, bResetStartOrigin);
}

void UTwoAxisPropertyEditInputBehavior::OnDragUpdate(FVector2D InScreenPosition)
{
	using IPropertyInterface = ITwoAxisPropertyEditBehaviorTarget::IPropertyInterface;
	
	// Calculate screen space cursor delta relative to current origin
	const float HorizontalDelta = InScreenPosition.X - State.CurrentOrigin.X;
	const float VerticalDelta = InScreenPosition.Y - State.CurrentOrigin.Y;
	const float HorizDeltaMag = FMath::Abs(HorizontalDelta);
	const float VertDeltaMag = FMath::Abs(VerticalDelta);
	
	// Apply directional adjustments
	if (IPropertyInterface* Property = State.bAdjustingHorizontally ? Target->GetHorizontalProperty() : Target->GetVerticalProperty(); Property && Property->IsEnabled())
	{
		const float DPIScale = FGenericPlatformApplicationMisc::GetDPIScaleFactorAtPoint(InScreenPosition.X, InScreenPosition.Y);
		const float Delta = State.bAdjustingHorizontally ? HorizontalDelta : VerticalDelta;
		
		const float NewValue = State.StartValue + (State.bAdjustingHorizontally ? 1.f : -1.f) * Property->MutateDelta(Delta * Property->GetEditRate() * DPIScale);
		Property->SetValue(NewValue);
	}

	// Determine if we need to swap adjustment axis
	if (State.bAdjustingHorizontally && HorizDeltaMag < VertDeltaMag && Target->GetVerticalProperty()->IsEnabled())
	{
		// Switch to adjusting vertically and re-center current origin
		ResetOrigin(InScreenPosition, false);
	}
	else if (!State.bAdjustingHorizontally && VertDeltaMag < HorizDeltaMag && Target->GetHorizontalProperty()->IsEnabled())
	{
		// Switch to adjusting horizontally and re-center current origin
		ResetOrigin(InScreenPosition, true);
	}
	
	Target->PostDragUpdated();
}

void UTwoAxisPropertyEditInputBehavior::OnDragEnd()
{
	bInDrag = false;
	ResetDragState();
}

void UTwoAxisPropertyEditInputBehavior::ResetDragState()
{
	State = FDragState();
}

FInputCaptureRequest UTwoAxisPropertyEditInputBehavior::WantsCapture(const FInputDeviceState& InputState)
{
	switch (InputState.InputDevice)
	{
	case EInputDevices::Mouse: return WantsMouseCapture(InputState);
	case EInputDevices::Keyboard: return WantsKeyboardCapture(InputState);
	default: return FInputCaptureRequest::Ignore();
	}
}

FInputCaptureUpdate UTwoAxisPropertyEditInputBehavior::BeginCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide)
{
	switch (InputState.InputDevice)
	{
	case EInputDevices::Mouse: return BeginMouseCapture(InputState, eSide);
	case EInputDevices::Keyboard: return BeginKeyboardCapture(InputState, eSide);
	default: return FInputCaptureUpdate::Ignore();
	}
}

FInputCaptureUpdate UTwoAxisPropertyEditInputBehavior::UpdateCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData)
{
	switch (InputState.InputDevice)
	{
	case EInputDevices::Mouse: return UpdateMouseCapture(InputState, CaptureData);
	case EInputDevices::Keyboard: return UpdateKeyboardCapture(InputState, CaptureData);
	default: return FInputCaptureUpdate::Ignore();
	}
}

void UTwoAxisPropertyEditInputBehavior::ForceEndCapture(const FInputCaptureData& CaptureData)
{
	OnDragEnd();
}

FInputCaptureRequest UTwoAxisPropertyEditInputBehavior::WantsMouseCapture(const FInputDeviceState& InputState)
{
	if (IsMousePressed(InputState))
	{
		return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, 0.0);
	}
	else
	{
		return FInputCaptureRequest::Ignore();
	}
}

FInputCaptureUpdate UTwoAxisPropertyEditInputBehavior::BeginMouseCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide)
{
	OnDragStart(InputState.Mouse.Position2D);
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UTwoAxisPropertyEditInputBehavior::UpdateMouseCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData)
{
	if (IsMousePressed(InputState))
	{
		OnDragUpdate(InputState.Mouse.Position2D);
		return FInputCaptureUpdate::Continue();
	}
	else
	{
		OnDragEnd();
		return FInputCaptureUpdate::End();
	}
}

bool UTwoAxisPropertyEditInputBehavior::IsMousePressed(const FInputDeviceState& InputState) const
{
	return InputState.Mouse.Left.bDown && bKeyPressed;
}

FInputCaptureRequest UTwoAxisPropertyEditInputBehavior::WantsKeyboardCapture(const FInputDeviceState& InputState)
{
	if (IsKeyboardPressed(InputState))
	{
		return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, 0.0);
	}
	else
	{
		return FInputCaptureRequest::Ignore();
	}
}

FInputCaptureUpdate UTwoAxisPropertyEditInputBehavior::BeginKeyboardCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide)
{
	bKeyPressed = true;
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UTwoAxisPropertyEditInputBehavior::UpdateKeyboardCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData)
{
	if (IsKeyboardPressed(InputState))
    {
    	return FInputCaptureUpdate::Continue();
    }
    else
    {
    	bKeyPressed = false;
    	return FInputCaptureUpdate::End();
    }
}

bool UTwoAxisPropertyEditInputBehavior::IsKeyboardPressed(const FInputDeviceState& InputState) const
{
	
	if (!Target->GetCaptureKey().IsModifierKey())
	{
		// Explicitly guarding against unused modifiers here so that modified same-key hotkeys still get the input.
		// In this case,  If B is the capture key, Shift+B will not be captured and passes through to the next input handler
		// This is assuming this behavior only support a single key for capture key.
		const bool bModifierKeyDown = 
			FInputDeviceState::IsShiftKeyDown(InputState) ||
			FInputDeviceState::IsCtrlKeyDown(InputState) ||
			FInputDeviceState::IsCmdKeyDown(InputState) ||
			FInputDeviceState::IsAltKeyDown(InputState);
		if (bModifierKeyDown)
		{
			return false;
		}
	}
	
	return InputState.Keyboard.ActiveKey.Button == Target->GetCaptureKey() && InputState.Keyboard.ActiveKey.bDown;;
}

#undef LOCTEXT_NAMESPACE