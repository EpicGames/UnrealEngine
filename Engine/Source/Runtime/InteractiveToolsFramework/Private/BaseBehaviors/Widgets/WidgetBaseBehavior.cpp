// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseBehaviors/Widgets/WidgetBaseBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBaseBehavior)

bool IWidgetBaseBehavior::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return false;
}

bool IWidgetBaseBehavior::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return false;
}

void IWidgetBaseBehavior::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
}

int32 IWidgetBaseBehavior::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	return LayerId;
}

