// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/AxisAngleGizmo.h"
#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // SetCommonSubGizmoProperties
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams

#include UE_INLINE_GENERATED_CPP_BY_NAME(AxisAngleGizmo)



UInteractiveGizmo* UAxisAngleGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UAxisAngleGizmo* NewGizmo = NewObject<UAxisAngleGizmo>(SceneState.GizmoManager);
	return NewGizmo;
}




void UAxisAngleGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	// Add default mouse input behavior
	MouseBehavior = NewObject<UClickDragInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(MouseBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(HoverBehavior);

	AxisSource = NewObject<UGizmoConstantAxisSource>(this);
	AngleSource = NewObject<UGizmoLocalFloatParameterSource>(this);
	HitTarget = NewObject<UGizmoComponentHitTarget>(this);
	StateTarget = NewObject<UGizmoNilStateTarget>(this);

	bInInteraction = false;
}

bool UAxisAngleGizmo::InitializeAsRotateGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params, 
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy
		|| Params.Axis == EAxis::None)
	{
		return false;
	}

	UGizmoScaledAndUnscaledTransformSources* TransformSource;
	if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();

	// Parameter source maps axis-parameter-change to translation of TransformSource's transform
	UGizmoAxisRotationParameterSource* CastAngleSource = UGizmoAxisRotationParameterSource::Construct(
		AxisSource.GetInterface(), TransformSource, Owner);
	AngleSource = CastAngleSource;

	return true;
}

void UAxisAngleGizmo::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
}


FInputRayHit UAxisAngleGizmo::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget && AxisSource && AngleSource)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
		if (GizmoHit.bHit)
		{
			LastHitPosition = PressPos.WorldRay.PointAt(GizmoHit.HitDepth);
		}
	}
	return GizmoHit;
}

void UAxisAngleGizmo::OnClickPress(const FInputDeviceRay& PressPos)
{
	check(bInInteraction == false);

	RotationAxis = AxisSource->GetDirection();
	RotationOrigin = GizmoMath::ProjectPointOntoLine(LastHitPosition, AxisSource->GetOrigin(), RotationAxis);

	if (AxisSource->HasTangentVectors())
	{
		AxisSource->GetTangentVectors(RotationPlaneX, RotationPlaneY);
	}
	else
	{
		GizmoMath::MakeNormalPlaneBasis(RotationAxis, RotationPlaneX, RotationPlaneY);
	}

	bool bIntersects; FVector IntersectionPoint;
	GizmoMath::RayPlaneIntersectionPoint(
		RotationOrigin, RotationAxis,
		PressPos.WorldRay.Origin, PressPos.WorldRay.Direction,
		bIntersects, IntersectionPoint);
	check(bIntersects);  // need to handle this case...

	InteractionStartPoint = IntersectionPoint;
	InteractionCurPoint = InteractionStartPoint;

	InteractionStartAngle = GizmoMath::ComputeAngleInPlane(InteractionCurPoint,
		RotationOrigin, RotationAxis, RotationPlaneX, RotationPlaneY);
	InteractionCurAngle = InteractionStartAngle;

	// Figure out the angle of the closest axis so that we know what to snap if using alignment.
	float StartAngleAbs = FMath::Abs(InteractionStartAngle);
	ClosestAxisStartAngle = StartAngleAbs <= PI / 4 ? 0
		: StartAngleAbs >= PI * 3 / 4 ? PI
		: InteractionStartAngle > 0 ? PI / 2 : - PI / 2;

	InitialTargetAngle = AngleSource->GetParameter();
	AngleSource->BeginModify();

	bInInteraction = true;

	if (StateTarget)
	{
		StateTarget->BeginUpdate();
	}
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}



void UAxisAngleGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	check(bInInteraction);

	FVector HitPoint;
	float DeltaAngle;

	// See if we should use custom destination funtion
	FCustomDestinationParams Params;
	Params.WorldRay = &DragPos.WorldRay;
	if (ShouldUseCustomDestinationFunc() && CustomDestinationFunc(Params, HitPoint))
	{
		InteractionCurPoint = HitPoint;
		InteractionCurAngle = GizmoMath::ComputeAngleInPlane(InteractionCurPoint,
			RotationOrigin, RotationAxis, RotationPlaneX, RotationPlaneY);
		
		// Align nearest axis to destination point.
		DeltaAngle = InteractionCurAngle - ClosestAxisStartAngle;
	}
	else
	{
		bool bIntersects; FVector IntersectionPoint;
		GizmoMath::RayPlaneIntersectionPoint(
			RotationOrigin, RotationAxis,
			DragPos.WorldRay.Origin, DragPos.WorldRay.Direction,
			bIntersects, IntersectionPoint);

		if (bIntersects == false)
		{
			return;
		}

		InteractionCurPoint = IntersectionPoint;

		InteractionCurAngle = GizmoMath::ComputeAngleInPlane(InteractionCurPoint,
			RotationOrigin, RotationAxis, RotationPlaneX, RotationPlaneY);

		DeltaAngle = InteractionCurAngle - InteractionStartAngle;
	}

	float NewAngle = InitialTargetAngle + DeltaAngle;
	AngleSource->SetParameter(NewAngle);
}


void UAxisAngleGizmo::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	check(bInInteraction);

	AngleSource->EndModify();
	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;

	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}


void UAxisAngleGizmo::OnTerminateDragSequence()
{
	check(bInInteraction);

	AngleSource->EndModify();
	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;

	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}




FInputRayHit UAxisAngleGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
	}
	return GizmoHit;
}

void UAxisAngleGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	HitTarget->UpdateHoverState(true);
}

bool UAxisAngleGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// not necessary...
	HitTarget->UpdateHoverState(true);
	return true;
}

void UAxisAngleGizmo::OnEndHover()
{
	HitTarget->UpdateHoverState(false);
}



