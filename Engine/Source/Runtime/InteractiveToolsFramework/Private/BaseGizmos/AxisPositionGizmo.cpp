// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/AxisPositionGizmo.h"

#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // SetCommonSubGizmoProperties
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AxisPositionGizmo)



UInteractiveGizmo* UAxisPositionGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UAxisPositionGizmo* NewGizmo = NewObject<UAxisPositionGizmo>(SceneState.GizmoManager);
	return NewGizmo;
}




void UAxisPositionGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	// Add default mouse input behavior
	MouseBehavior = NewObject<UClickDragInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(MouseBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(HoverBehavior);

	AxisSource = NewObject<UGizmoConstantAxisSource>(this);
	ParameterSource = NewObject<UGizmoLocalFloatParameterSource>(this);
	HitTarget = NewObject<UGizmoComponentHitTarget>(this);
	StateTarget = NewObject<UGizmoNilStateTarget>(this);

	bInInteraction = false;
}

bool UAxisPositionGizmo::InitializeAsTranslateGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy
		|| Params.Axis == EAxis::None)
	{
		return false;
	}

	UGizmoScaledAndUnscaledTransformSources* TransformSource;
	if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();

	// Parameter source maps axis-parameter-change to translation of TransformSource's transform
	ParameterSource = UGizmoAxisTranslationParameterSource::Construct(
		AxisSource.GetInterface(), TransformSource, Owner);

	return true;
}

bool UAxisPositionGizmo::InitializeAsScaleGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	bool bDisallowNegativeScaling, UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy
		|| Params.Axis == EAxis::None)
	{
		return false;
	}

	int AxisIndex = Params.GetClampedAxisIndex();

	UGizmoScaledAndUnscaledTransformSources* TransformSource;
	if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();
	bEnableSignedAxis = true;

	// Although the normal axis source gets used for detecting interactions, the parameter application has
	// to happen along unrotated axes because the scaling gets applied before rotation. In other words if we
	// tried to apply scaling measured along a rotated vector, we would end up incorrectly scaling along
	// multiple axes.
	UGizmoComponentAxisSource* UnitCardinalAxisSource = nullptr;
	// See if we already have it in our shared state
	if (SharedState && SharedState->UnitCardinalAxisSources[AxisIndex])
	{
		UnitCardinalAxisSource = SharedState->UnitCardinalAxisSources[AxisIndex];
	}
	else
	{
		// Create new and add to shared state.
		USceneComponent* RootComponent = Params.Component->GetOwner()->GetRootComponent();
		UGizmoComponentAxisSource* CastAxisSource = UGizmoComponentAxisSource::Construct(RootComponent, AxisIndex,
			// bUseLocalAxes, has to be false to always use cardinal axes
			false,
			Owner);
		UnitCardinalAxisSource = CastAxisSource;
		if (SharedState)
		{
			SharedState->UnitCardinalAxisSources[AxisIndex] = UnitCardinalAxisSource;
		}
	}

	// Parameter source maps axis-parameter-change to scale of TransformSource's transform
	UGizmoAxisScaleParameterSource* CastParameterSource = UGizmoAxisScaleParameterSource::Construct(
		UnitCardinalAxisSource, TransformSource, Owner);
	ParameterSource = CastParameterSource;
	CastParameterSource->bClampToZero = bDisallowNegativeScaling;

	return true;
}

FInputRayHit UAxisPositionGizmo::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget && AxisSource && ParameterSource)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
		if (GizmoHit.bHit)
		{
			LastHitPosition = PressPos.WorldRay.PointAt(GizmoHit.HitDepth);
		}
	}
	return GizmoHit;
}

void UAxisPositionGizmo::OnClickPress(const FInputDeviceRay& PressPos)
{
	InteractionOrigin = LastHitPosition;
	InteractionAxis = AxisSource->GetDirection();

	// Find interaction start point and parameter.
	FVector NearestPt; float RayNearestParam;
	GizmoMath::NearestPointOnLineToRay(InteractionOrigin, InteractionAxis,
		PressPos.WorldRay.Origin, PressPos.WorldRay.Direction,
		InteractionStartPoint, InteractionStartParameter,
		NearestPt, RayNearestParam);

	FVector AxisOrigin = AxisSource->GetOrigin();

	double DirectionSign = FVector::DotProduct(InteractionStartPoint - AxisOrigin, InteractionAxis);
	ParameterSign = (bEnableSignedAxis && DirectionSign < 0) ? -1.0f : 1.0f;

	// Figure out how the parameter would need to be adjusted to bring the axis origin to the
	// interaction start point. This is used when aligning the axis origin to a custom destination.
	float AxisOriginParamValue;
	GizmoMath::NearestPointOnLine(InteractionOrigin, InteractionAxis, AxisOrigin,
		NearestPt, AxisOriginParamValue);
	InteractionStartAxisOriginParameterOffset = InteractionStartParameter - AxisOriginParamValue;

	InteractionCurPoint = InteractionStartPoint;
	InteractionStartParameter *= ParameterSign;
	InteractionCurParameter = InteractionStartParameter;

	InitialTargetParameter = ParameterSource->GetParameter();
	ParameterSource->BeginModify();

	bInInteraction = true;

	if (StateTarget)
	{
		StateTarget->BeginUpdate();
	}
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}

void UAxisPositionGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	FVector HitPoint;

	// See if we should use the custom destination function.
	FCustomDestinationParams Params;
	Params.WorldRay = &DragPos.WorldRay;
	if (ShouldUseCustomDestinationFunc() && CustomDestinationFunc(Params, HitPoint))
	{
		GizmoMath::NearestPointOnLine(InteractionOrigin, InteractionAxis, HitPoint,
			InteractionCurPoint, InteractionCurParameter);
		InteractionCurParameter += bCustomDestinationAlignsAxisOrigin ? InteractionStartAxisOriginParameterOffset : 0;
	}
	else
	{
		float RayNearestParam; float AxisNearestParam;
		FVector RayNearestPt; 
		GizmoMath::NearestPointOnLineToRay(InteractionOrigin, InteractionAxis,
			DragPos.WorldRay.Origin, DragPos.WorldRay.Direction,
			InteractionCurPoint, AxisNearestParam,
			RayNearestPt, RayNearestParam);

		InteractionCurParameter = ParameterSign * AxisNearestParam;
	}

	float DeltaParam = InteractionCurParameter - InteractionStartParameter;
	float NewParamValue = InitialTargetParameter + DeltaParam;

	ParameterSource->SetParameter(NewParamValue);
}

void UAxisPositionGizmo::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	check(bInInteraction);

	ParameterSource->EndModify();
	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;

	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}


void UAxisPositionGizmo::OnTerminateDragSequence()
{
	check(bInInteraction);

	ParameterSource->EndModify();
	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}



FInputRayHit UAxisPositionGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
	}
	return GizmoHit;
}

void UAxisPositionGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	HitTarget->UpdateHoverState(true);
}

bool UAxisPositionGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// not necessary...
	HitTarget->UpdateHoverState(true);
	return true;
}

void UAxisPositionGizmo::OnEndHover()
{
	HitTarget->UpdateHoverState(false);
}
