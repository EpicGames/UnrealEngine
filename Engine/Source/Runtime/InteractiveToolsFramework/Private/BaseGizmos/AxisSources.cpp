// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/AxisSources.h"
#include "Components/SceneComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AxisSources)

FVector UGizmoComponentAxisSource::GetOrigin() const
{
	const FTransform& WorldTransform = Component->GetComponentToWorld();
	return WorldTransform.GetLocation();
}

FVector UGizmoComponentAxisSource::GetDirection() const
{
	const FTransform& WorldTransform = Component->GetComponentToWorld();
	FVector Axis(0, 0, 0);
	Axis[FMath::Clamp(AxisIndex, 0, 2)] = 1.0;
	if (bLocalAxes)
	{
		return WorldTransform.GetRotation().RotateVector(Axis);
	} 
	else
	{
		return Axis;
	}
}

void UGizmoComponentAxisSource::GetTangentVectors(FVector& TangentXOut, FVector& TangentYOut) const
{
	// Note that in giving the tangent vectors, we need to give them in an order that
	// preserves the handedness of our coordinate system so that we can use the function
	// for generating planes in which we can measure rotation angles about the input axis.
	// So, TangentXOut, TangentYOut, and Input(Z) should give a left-handed frame (since 
	// unreal is left handed).

	const FTransform& WorldTransform = Component->GetComponentToWorld();
	TangentXOut = FVector(0, 1, 0);
	TangentYOut = FVector(0, 0, 1);
	int Index = FMath::Clamp(AxisIndex, 0, 2);
	if (Index == 1)
	{
		TangentXOut = FVector(0, 0, 1);
		TangentYOut = FVector(1, 0, 0);
	}
	else if (Index == 2)
	{
		TangentXOut = FVector(1, 0, 0);
		TangentYOut = FVector(0, 1, 0);
	}
	if (bLocalAxes)
	{
		TangentXOut = WorldTransform.GetRotation().RotateVector(TangentXOut);
		TangentYOut = WorldTransform.GetRotation().RotateVector(TangentYOut);
	}
}
