// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/BrushStampIndicator.h"
#include "InteractiveGizmoManager.h"
#include "Components/PrimitiveComponent.h"
#include "ToolDataVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushStampIndicator)


UInteractiveGizmo* UBrushStampIndicatorBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UBrushStampIndicator* NewGizmo = NewObject<UBrushStampIndicator>(SceneState.GizmoManager);
	return NewGizmo;
}



void UBrushStampIndicator::Setup()
{
}

void UBrushStampIndicator::Shutdown()
{
}

void UBrushStampIndicator::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (bVisible == false)
	{
		return;
	}

	if (bDrawIndicatorLines)
	{
		FToolDataVisualizer Draw;
		Draw.BeginFrame(RenderAPI);

		if (bDrawSecondaryLines)
		{
			const bool bCirclesCoincident = bDrawRadiusCircle && FMath::IsNearlyEqual(BrushFalloff, 1.0f);
			if (!bCirclesCoincident)
			{
				Draw.DrawCircle(BrushPosition, BrushNormal, BrushRadius*BrushFalloff, SampleStepCount, SecondaryLineColor, SecondaryLineThickness, bDepthTested);
			}
			
			const float NormalScale = bScaleNormalByStrength ? BrushStrength : 1.0f;
			Draw.DrawLine(BrushPosition, BrushPosition + BrushRadius * BrushNormal * NormalScale, SecondaryLineColor, SecondaryLineThickness, bDepthTested);
		}
		
		if (bDrawRadiusCircle)
		{
			Draw.DrawCircle(BrushPosition, BrushNormal, BrushRadius, SampleStepCount, LineColor, LineThickness, bDepthTested);
		}
		
		Draw.EndFrame();
	}
}

void UBrushStampIndicator::Tick(float DeltaTime)
{
}


void UBrushStampIndicator::Update(float Radius, const FVector& Position, const FVector& Normal, float Falloff, float Strength)
{
	BrushRadius = Radius;
	BrushPosition = Position;
	BrushNormal = Normal;
	BrushFalloff = Falloff;
	BrushStrength = Strength;

	if (AttachedComponent != nullptr)
	{
		FTransform Transform = AttachedComponent->GetComponentTransform();

		if (ScaleInitializedComponent != AttachedComponent)
		{
			InitialComponentScale = Transform.GetScale3D();
			InitialComponentScale *= 1.0f / InitialComponentScale.Z;
			ScaleInitializedComponent = AttachedComponent;
		}

		Transform.SetTranslation(BrushPosition);

		FQuat CurRotation = Transform.GetRotation();
		FQuat ApplyRotation = FQuat::FindBetween(CurRotation.GetAxisZ(), BrushNormal);
		Transform.SetRotation(ApplyRotation * CurRotation);

		Transform.SetScale3D(Radius * InitialComponentScale);

		AttachedComponent->SetWorldTransform(Transform);
	}
}



void UBrushStampIndicator::Update(float Radius, const FTransform& WorldTransform, float Falloff)
{
	BrushRadius = Radius;
	BrushPosition = WorldTransform.GetLocation();
	BrushNormal = WorldTransform.GetRotation().GetAxisZ();
	BrushFalloff = Falloff;

	if (AttachedComponent != nullptr)
	{
		FTransform Transform = WorldTransform;

		if (ScaleInitializedComponent != AttachedComponent)
		{
			InitialComponentScale = AttachedComponent->GetComponentTransform().GetScale3D();
			InitialComponentScale *= 1.0f / InitialComponentScale.Z;
			ScaleInitializedComponent = AttachedComponent;
		}

		Transform.SetScale3D(Radius * InitialComponentScale);

		AttachedComponent->SetWorldTransform(Transform);
	}
}

