// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "InteractiveGizmoManager.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/FreePositionSubGizmo.h"
#include "BaseGizmos/FreeRotationSubGizmo.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/AxisAngleGizmo.h"
#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/GizmoUtil.h"

#include "BaseGizmos/GizmoArrowComponent.h"
#include "BaseGizmos/GizmoRectangleComponent.h"
#include "BaseGizmos/GizmoCircleComponent.h"
#include "BaseGizmos/GizmoBoxComponent.h"
#include "BaseGizmos/GizmoLineHandleComponent.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // ToAxis
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams, FTransformSubGizmoSharedState
#include "BaseGizmos/ViewAdjustedStaticMeshGizmoComponent.h"
#include "BaseGizmos/ViewBasedTransformAdjusters.h" // FSubGizmoTransformAdjuster

#include "Quaternion.h"
#include "MathUtil.h"
#include "MatrixTypes.h"
#include "VectorUtil.h"

// need this to implement hover
#include "BaseGizmos/GizmoBaseComponent.h"

#include "Components/SphereComponent.h"
#include "Components/PrimitiveComponent.h"
#include "ContextObjectStore.h"
#include "Engine/World.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef

#include UE_INLINE_GENERATED_CPP_BY_NAME(CombinedTransformGizmo)


#define LOCTEXT_NAMESPACE "UCombinedTransformGizmo"

namespace CombinedTransformGizmoLocals
{
	const int32 DrawModeValue_Meshes = 1;
	
	// CVar that determines how we draw the gizmo
	int32 GizmoDrawMode = DrawModeValue_Meshes;
	static FAutoConsoleVariableRef CVarGizmoDrawMode(
		TEXT("modeling.Gizmo.DrawMode"),
		GizmoDrawMode,
		TEXT("When 0, modeling gizmos are drawn using the old PDI system. When 1, modeling gizmos use new adjusted-size components. "
			"Gizmos have to be recreated (by restarting mode/tools) for the change to take effect."));

	// Helper functions that determine whether parts of the gizmo should be visible when using DrawModeValue_Meshes. For example
	//  we don't want the size of a rotation component to be hiding the axis behind it in ortho view.
	auto ShouldAxisBeVisible = [](const UE::GizmoRenderingUtil::ISceneViewInterface& View,
		const FTransform& ComponentToWorld) 
		{
			static const double ARROW_RENDERVISIBILITY_DOT_THRESHOLD = FMath::Cos(FMath::DegreesToRadians(3));

			FVector ViewDirection = View.IsPerspectiveProjection() ?
				ComponentToWorld.GetLocation() - View.GetViewLocation() : View.GetViewDirection();
			ViewDirection.Normalize();
			FVector ArrowDirection = ComponentToWorld.TransformVector(FVector::XAxisVector);
			ArrowDirection.Normalize();

			return FMath::Abs(FVector::DotProduct(ArrowDirection, ViewDirection)) <= ARROW_RENDERVISIBILITY_DOT_THRESHOLD;
		};
	auto ShouldPlaneBeVisible = [](const UE::GizmoRenderingUtil::ISceneViewInterface& View,
		const FTransform& ComponentToWorld) 
		{
			static const double RECTANGLE_RENDERVISIBILITY_DOT_THRESHOLD = FMath::Cos(FMath::DegreesToRadians(87));;

			FVector ViewDirection = View.IsPerspectiveProjection() ?
				ComponentToWorld.GetLocation() - View.GetViewLocation() : View.GetViewDirection();
			ViewDirection.Normalize();
			FVector PlaneNormal = ComponentToWorld.TransformVector(FVector::XAxisVector);
			PlaneNormal.Normalize();

			return
				FMath::Abs(FVector::DotProduct(PlaneNormal, ViewDirection)) >= RECTANGLE_RENDERVISIBILITY_DOT_THRESHOLD;
		};
	
	FLinearColor FreeRotateColor(0.5f, 0.5f, 0.5f, 0.15f); // A faint translucent gray
	// Slightly gray so that the selection highlight pops a bit more
	FLinearColor FreeTranslateColor(0.7f, 0.7f, 0.7f, 1.0f);
	FLinearColor UniformScaleColor = FreeTranslateColor;
	FVector CornerScalePositionCombined(0, 120, 120);
	FVector CornerScalePositionSeparate(0, 75, 75);
	double CornerScaleHandleScale = 0.5;

	// Helper functions that get the appropriate values to use from an EAxis value, so
	//  that we can write helpers that just take that as an argument.
	FLinearColor AxisToLegacyColor(EAxis::Type Axis)
	{
		switch (Axis)
		{
		case EAxis::X:
			return FLinearColor::Red;
		case EAxis::Y:
			return FLinearColor::Green;
		case EAxis::Z:
			return FLinearColor::Blue;
		default:
			ensure(false);
		}
		return FLinearColor::Black;
	}
	FVector AxisToVector(EAxis::Type Axis)
	{
		switch (Axis)
		{
		case EAxis::X:
			return FVector::XAxisVector;
		case EAxis::Y:
			return FVector::YAxisVector;
		case EAxis::Z:
			return FVector::ZAxisVector;
		default:
			ensure(false);
		}
		return FVector::XAxisVector;
	}
	void AxisToLegacyPairOfVectors(EAxis::Type Axis, FVector& Vector1, FVector& Vector2)
	{
		switch (Axis)
		{
		case EAxis::X:
			Vector1 = FVector::YAxisVector;
			Vector2 = FVector::ZAxisVector;
			return;
		case EAxis::Y:
			Vector1 = FVector::XAxisVector;
			Vector2 = FVector::ZAxisVector;
			return;
		case EAxis::Z:
			Vector1 = FVector::XAxisVector;
			Vector2 = FVector::YAxisVector;
			return;
		default:
			ensure(false);
		}
	}

	// Looks at a gizmo actor and figures out what sub element flags must have been active when creating it.
	ETransformGizmoSubElements GetSubElementFlagsFromActor(ACombinedTransformGizmoActor* GizmoActor)
	{
		ETransformGizmoSubElements Elements = ETransformGizmoSubElements::None;
		if (!GizmoActor)
		{
			return Elements;
		}

		if (GizmoActor->TranslateX) { Elements |= ETransformGizmoSubElements::TranslateAxisX; }
		if (GizmoActor->TranslateY) { Elements |= ETransformGizmoSubElements::TranslateAxisY; }
		if (GizmoActor->TranslateZ) { Elements |= ETransformGizmoSubElements::TranslateAxisZ; }
		if (GizmoActor->TranslateXY) { Elements |= ETransformGizmoSubElements::TranslatePlaneXY; }
		if (GizmoActor->TranslateYZ) { Elements |= ETransformGizmoSubElements::TranslatePlaneYZ; }
		if (GizmoActor->TranslateXZ) { Elements |= ETransformGizmoSubElements::TranslatePlaneXZ; }
		if (GizmoActor->FreeTranslateHandle) { Elements |= ETransformGizmoSubElements::FreeTranslate; }

		if (GizmoActor->RotateX) { Elements |= ETransformGizmoSubElements::RotateAxisX; }
		if (GizmoActor->RotateY) { Elements |= ETransformGizmoSubElements::RotateAxisY; }
		if (GizmoActor->RotateZ) { Elements |= ETransformGizmoSubElements::RotateAxisZ; }
		if (GizmoActor->FreeRotateHandle) { Elements |= ETransformGizmoSubElements::FreeRotate; }

		if (GizmoActor->AxisScaleX) { Elements |= ETransformGizmoSubElements::ScaleAxisX; }
		if (GizmoActor->AxisScaleY) { Elements |= ETransformGizmoSubElements::ScaleAxisY; }
		if (GizmoActor->AxisScaleZ) { Elements |= ETransformGizmoSubElements::ScaleAxisZ; }
		if (GizmoActor->PlaneScaleXY) { Elements |= ETransformGizmoSubElements::ScalePlaneXY; }
		if (GizmoActor->PlaneScaleYZ) { Elements |= ETransformGizmoSubElements::ScalePlaneYZ; }
		if (GizmoActor->PlaneScaleXZ) { Elements |= ETransformGizmoSubElements::ScalePlaneXZ; }

		if (GizmoActor->UniformScale) { Elements |= ETransformGizmoSubElements::ScaleUniform; }

		return Elements;
	}
}

ACombinedTransformGizmoActor::ACombinedTransformGizmoActor()
{
	// root component is a hidden sphere
	USphereComponent* SphereComponent = CreateDefaultSubobject<USphereComponent>(TEXT("GizmoCenter"));
	RootComponent = SphereComponent;
	SphereComponent->InitSphereRadius(1.0f);
	SphereComponent->SetVisibility(false);
	SphereComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}




bool ACombinedTransformGizmoActor::ReplaceSubGizmoComponent(ETransformGizmoSubElements Element,
	UPrimitiveComponent* NewComponent, const FTransform& SubGizmoToGizmo, UPrimitiveComponent** ReplacedComponentOut)
{
	// We allow a null NewComponent (which equates to element removal), but if we do have a component,
	//  it should have this actor in its outer chain. It might be possible to loosen that restriction,
	//  but it's likely that something is wrong in this case.
	if (NewComponent && !ensure(NewComponent->GetOwner() == this))
	{
		return false;
	}

	auto ReplaceComponent = [this, &SubGizmoToGizmo, NewComponent, ReplacedComponentOut](TObjectPtr<UPrimitiveComponent>& ComponentToReplace)
	{
		if (ComponentToReplace)
		{
			ComponentToReplace->DestroyComponent();
		}
		if (ReplacedComponentOut)
		{
			*ReplacedComponentOut = ComponentToReplace;
		}

		ComponentToReplace = NewComponent;

		if (NewComponent)
		{
			AddInstanceComponent(NewComponent);
			NewComponent->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
			NewComponent->SetRelativeTransform(SubGizmoToGizmo);
			if (!NewComponent->IsRegistered())
			{
				NewComponent->RegisterComponent();
			}
		}
	};

	switch (Element)
	{
	case ETransformGizmoSubElements::TranslateAxisX:
		ReplaceComponent(TranslateX);
		break;
	case ETransformGizmoSubElements::TranslateAxisY:
		ReplaceComponent(TranslateY);
		break;
	case ETransformGizmoSubElements::TranslateAxisZ:
		ReplaceComponent(TranslateZ);
		break;
	case ETransformGizmoSubElements::TranslatePlaneXY:
		ReplaceComponent(TranslateXY);
		break;
	case ETransformGizmoSubElements::TranslatePlaneXZ:
		ReplaceComponent(TranslateXZ);
		break;
	case ETransformGizmoSubElements::TranslatePlaneYZ:
		ReplaceComponent(TranslateYZ);
		break;
	case ETransformGizmoSubElements::RotateAxisX:
		ReplaceComponent(RotateX);
		break;
	case ETransformGizmoSubElements::RotateAxisY:
		ReplaceComponent(RotateY);
		break;
	case ETransformGizmoSubElements::RotateAxisZ:
		ReplaceComponent(RotateZ);
		break;
	case ETransformGizmoSubElements::ScaleAxisX:
		ReplaceComponent(AxisScaleX);
		if (FullAxisScaleX)
		{
			FullAxisScaleX->DestroyComponent();
			FullAxisScaleX = nullptr;
		}
		break;
	case ETransformGizmoSubElements::ScaleAxisY:
		ReplaceComponent(AxisScaleY);
		if (FullAxisScaleY)
		{
			FullAxisScaleY->DestroyComponent();
			FullAxisScaleY = nullptr;
		}
		break;
	case ETransformGizmoSubElements::ScaleAxisZ:
		ReplaceComponent(AxisScaleZ);
		if (FullAxisScaleZ)
		{
			FullAxisScaleZ->DestroyComponent();
			FullAxisScaleZ = nullptr;
		}
		break;
	case ETransformGizmoSubElements::ScalePlaneXY:
		ReplaceComponent(PlaneScaleXY);
		break;
	case ETransformGizmoSubElements::ScalePlaneXZ:
		ReplaceComponent(PlaneScaleXZ);
		break;
	case ETransformGizmoSubElements::ScalePlaneYZ:
		ReplaceComponent(PlaneScaleYZ);
		break;
	case ETransformGizmoSubElements::ScaleUniform:
		ReplaceComponent(UniformScale);
		break;
	// We use the RotateAllAxes identifier for replacing the rotation sphere
	case ETransformGizmoSubElements::RotateAllAxes:
		ReplaceComponent(RotationSphere);
		break;
	case ETransformGizmoSubElements::FreeRotate:
		ReplaceComponent(FreeRotateHandle);
		break;
	case ETransformGizmoSubElements::FreeTranslate:
		ReplaceComponent(FreeTranslateHandle);
		break;
	default:
		UE_LOGF(LogGeometry, Warning, "UCombinedTransformGizmo::SetSubGizmoComponent currently only supports a "
			"single sub gizmo element at a time.");
		return false;
	}
	return true;
}

ACombinedTransformGizmoActor* ACombinedTransformGizmoActor::ConstructDefault3AxisGizmo(UWorld* World, UGizmoViewContext* GizmoViewContext)
{
	return ConstructCustom3AxisGizmo(World, GizmoViewContext,
		ETransformGizmoSubElements::TranslateAllAxes |
		ETransformGizmoSubElements::TranslateAllPlanes |
		ETransformGizmoSubElements::RotateAllAxes |
		ETransformGizmoSubElements::ScaleAllAxes |
		ETransformGizmoSubElements::ScaleAllPlanes |
		ETransformGizmoSubElements::ScaleUniform
	);
}


ACombinedTransformGizmoActor* ACombinedTransformGizmoActor::ConstructCustom3AxisGizmo(
	UWorld* World, UGizmoViewContext* GizmoViewContext,
	ETransformGizmoSubElements Elements)
{
	using namespace CombinedTransformGizmoLocals;
	using FSubGizmoTransformAdjuster = UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster;

	FActorSpawnParameters SpawnInfo;
	ACombinedTransformGizmoActor* NewActor = World->SpawnActor<ACombinedTransformGizmoActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);

	float GizmoLineThickness = 3.0f;

	enum class EMirror 
	{
		Always,
		WhenCombined,
		Never
	};

	// Helper for adding a mesh-based sub gizmo component (when using DrawModeValue_Meshes)
	auto AddMeshGizmoComponent = [GizmoViewContext, NewActor](const TCHAR* MeshPath, const FLinearColor& Color, 
		const FTransform& RelativeTransform, EMirror Mirror, bool bAddHoverMaterial = true) -> UViewAdjustedStaticMeshGizmoComponent*
	{
		UStaticMesh* Mesh = LoadObject<UStaticMesh>(nullptr, MeshPath);
		if (!ensure(Mesh))
		{
			return nullptr;
		}

		UViewAdjustedStaticMeshGizmoComponent* Component = UE::GizmoRenderingUtil::CreateDefaultMaterialGizmoMeshComponent(
			Mesh, GizmoViewContext, NewActor, Color, bAddHoverMaterial);
		if (!ensure(Component))
		{
			return nullptr;
		}
		NewActor->AddInstanceComponent(Component);
		Component->AttachToComponent(NewActor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
		Component->SetRelativeTransform(RelativeTransform);
		Component->RegisterComponent();
		TSharedPtr<FSubGizmoTransformAdjuster> Adjuster = FSubGizmoTransformAdjuster::AddTransformAdjuster(
				Component, NewActor->GetRootComponent(), Mirror == EMirror::Always);
		if (Mirror == EMirror::WhenCombined)
		{
			NewActor->AdjustersThatMirrorOnlyInCombinedMode.Add(Adjuster);
		}

		return Component;
	};

	auto MakeAxisArrowFunc = [World, NewActor, GizmoViewContext, GizmoLineThickness, &AddMeshGizmoComponent](EAxis::Type ElementAxis) -> UPrimitiveComponent*
	{
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			if (UViewAdjustedStaticMeshGizmoComponent* Component = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoArrowHandle"),
				UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis),
				UE::GizmoUtil::GetRotatedBasisTransform(
					// Transform for the X axis, relative to gizmo root
					FTransform(FQuat::Identity, FVector::ZeroVector, FVector::OneVector), ElementAxis),
				EMirror::WhenCombined))
			{
				Component->SetRenderVisibilityFunction(ShouldAxisBeVisible);
				return Component;
			}
		}

		UGizmoArrowComponent* Component = AddDefaultArrowComponent(World, NewActor, GizmoViewContext, 
			AxisToLegacyColor(ElementAxis), AxisToVector(ElementAxis), 60.0f);
		Component->Gap = 20.0f;
		Component->Thickness = GizmoLineThickness;
		Component->NotifyExternalPropertyUpdates();
		return Component;
	};
	if ((Elements & ETransformGizmoSubElements::TranslateAxisX) != ETransformGizmoSubElements::None)
	{
		NewActor->TranslateX = MakeAxisArrowFunc(EAxis::X);
	}
	if ((Elements & ETransformGizmoSubElements::TranslateAxisY) != ETransformGizmoSubElements::None)
	{
		NewActor->TranslateY = MakeAxisArrowFunc(EAxis::Y);
	}
	if ((Elements & ETransformGizmoSubElements::TranslateAxisZ) != ETransformGizmoSubElements::None)
	{
		NewActor->TranslateZ = MakeAxisArrowFunc(EAxis::Z);
	}


	auto MakePlaneRectFunc = [World, NewActor, GizmoViewContext, GizmoLineThickness, &AddMeshGizmoComponent](EAxis::Type ElementAxis) -> UPrimitiveComponent*
	{
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			if (UViewAdjustedStaticMeshGizmoComponent* Component = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoPlaneHandle"),
				UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis),
				UE::GizmoUtil::GetRotatedBasisTransform(
					// Transform for the X axis, relative to gizmo root
					FTransform(FQuat::Identity, FVector(0, 40, 40), FVector::One()), ElementAxis),
				EMirror::WhenCombined))
			{
				Component->SetRenderVisibilityFunction(ShouldPlaneBeVisible);
				return Component;
			}
		}

		// If we got to here, then we're creating the PDI drawn rectangle component
		FVector AxisX, AxisY;
		AxisToLegacyPairOfVectors(ElementAxis, AxisX, AxisY); 
		UGizmoRectangleComponent* Component = AddDefaultRectangleComponent(World, NewActor, GizmoViewContext, 
			AxisToLegacyColor(ElementAxis), AxisX, AxisY);
		Component->LengthX = Component->LengthY = 30.0f;
		Component->SegmentFlags = 0x2 | 0x4;
		Component->Thickness = GizmoLineThickness;
		Component->NotifyExternalPropertyUpdates();
		return Component;
	};
	if ((Elements & ETransformGizmoSubElements::TranslatePlaneYZ) != ETransformGizmoSubElements::None)
	{
		NewActor->TranslateYZ = MakePlaneRectFunc(EAxis::X);
	}
	if ((Elements & ETransformGizmoSubElements::TranslatePlaneXZ) != ETransformGizmoSubElements::None)
	{
		NewActor->TranslateXZ = MakePlaneRectFunc(EAxis::Y);
	}
	if ((Elements & ETransformGizmoSubElements::TranslatePlaneXY) != ETransformGizmoSubElements::None)
	{
		NewActor->TranslateXY = MakePlaneRectFunc(EAxis::Z);
	}
	if ((Elements & ETransformGizmoSubElements::FreeTranslate) != ETransformGizmoSubElements::None)
	{
		NewActor->FreeTranslateHandle = nullptr;
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			NewActor->FreeTranslateHandle = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoSphereHandle"),
				FreeTranslateColor,
				FTransform::Identity,
				EMirror::Never);
		}

		if (!NewActor->FreeTranslateHandle)
		{
			float BoxSize = 20.0f;
			// We use a box as the backup because it already has hittesting for the inside, unlike our circles
			NewActor->FreeTranslateHandle = AddDefaultBoxComponent(World, NewActor, GizmoViewContext, FLinearColor::Gray,
				FVector(BoxSize / 2, BoxSize / 2, BoxSize / 2), FVector(BoxSize, BoxSize, BoxSize));
		}
	}

	auto MakeAxisRotateCircleFunc = [World, NewActor, GizmoViewContext, GizmoLineThickness, &AddMeshGizmoComponent](EAxis::Type ElementAxis) -> UPrimitiveComponent*
	{
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			FLinearColor Color = UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis);
			Color.A = 0.75f; // Partially transparent, like editor gizmo

			if (UViewAdjustedStaticMeshGizmoComponent* Component = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoQuarterCircleHandle"),
				Color,
				UE::GizmoUtil::GetRotatedBasisTransform(
					FTransform(FQuat::Identity, FVector::ZeroVector, FVector::OneVector), ElementAxis),
				EMirror::Always))
			{
				Component->SetRenderVisibilityFunction(ShouldPlaneBeVisible);

				UStaticMesh* SubstituteMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoFullCircleHandle"));
				if (SubstituteMesh)
				{
					
					UViewAdjustedStaticMeshGizmoComponent* SubstituteComponent = UE::GizmoRenderingUtil::CreateDefaultMaterialGizmoMeshComponent(
						SubstituteMesh, GizmoViewContext, Component, Color,
						// No need for hover material
						false);
					if (SubstituteComponent)
					{
						Component->SetSubstituteInteractionComponent(SubstituteComponent);

						UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::AddTransformAdjuster(
							SubstituteComponent,
							NewActor->GetRootComponent(),
							/*bMirror*/ false);
					}
				}

				return Component;
			}
		}

		UGizmoCircleComponent* Component = AddDefaultCircleComponent(World, NewActor, GizmoViewContext, 
			AxisToLegacyColor(ElementAxis), AxisToVector(ElementAxis), 120.0f);
		Component->Thickness = GizmoLineThickness;
		Component->NotifyExternalPropertyUpdates();
		return Component;
	};

	bool bAnyRotate = false;
	if ((Elements & ETransformGizmoSubElements::RotateAxisX) != ETransformGizmoSubElements::None)
	{
		NewActor->RotateX = MakeAxisRotateCircleFunc(EAxis::X);
		bAnyRotate = true;
	}
	if ((Elements & ETransformGizmoSubElements::RotateAxisY) != ETransformGizmoSubElements::None)
	{
		NewActor->RotateY = MakeAxisRotateCircleFunc(EAxis::Y);
		bAnyRotate = true;
	}
	if ((Elements & ETransformGizmoSubElements::RotateAxisZ) != ETransformGizmoSubElements::None)
	{
		NewActor->RotateZ = MakeAxisRotateCircleFunc(EAxis::Z);
		bAnyRotate = true;
	}


	// add a non-interactive view-aligned circle element, so the axes look like a sphere.
	if (bAnyRotate && GizmoDrawMode != DrawModeValue_Meshes)
	{
		UGizmoCircleComponent* SphereEdge = NewObject<UGizmoCircleComponent>(NewActor);
		NewActor->AddInstanceComponent(SphereEdge);
		SphereEdge->AttachToComponent(NewActor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
		SphereEdge->SetGizmoViewContext(GizmoViewContext);
		SphereEdge->Color = FLinearColor::Gray;
		SphereEdge->Thickness = 1.0f;
		SphereEdge->Radius = 120.0f;
		SphereEdge->bViewAligned = true;
		SphereEdge->RegisterComponent();
		NewActor->RotationSphere = SphereEdge;
	}

	if ((Elements & ETransformGizmoSubElements::FreeRotate) != ETransformGizmoSubElements::None)
	{
		NewActor->FreeRotateHandle = nullptr;
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			NewActor->FreeRotateHandle = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoSphereHandle"),
				FreeRotateColor,
				FTransform(FQuat::Identity, FVector::ZeroVector, FVector(9)),
				EMirror::Never, /*bAddHoverMaterial*/ false);
		}

		if (!NewActor->FreeRotateHandle)
		{
			float BoxSize = 20.0f;
			// We use a box as the backup because it already has hittesting for the inside, unlike our circles
			NewActor->FreeRotateHandle = AddDefaultBoxComponent(World, NewActor, GizmoViewContext, FLinearColor::Gray,
				FVector(BoxSize/2, BoxSize/2, BoxSize/2), FVector(BoxSize, BoxSize, BoxSize));
		}
	}


	if ((Elements & ETransformGizmoSubElements::ScaleUniform) != ETransformGizmoSubElements::None)
	{
		NewActor->UniformScale = nullptr;
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			NewActor->UniformScale = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoBoxHandle"),
				UniformScaleColor,
				FTransform::Identity,
				EMirror::Never);
		}

		if (!NewActor->UniformScale)
		{
			float BoxSize = 20.0f;
			UGizmoBoxComponent* ScaleComponent = AddDefaultBoxComponent(World, NewActor, GizmoViewContext, FLinearColor::Black,
				FVector(BoxSize/2, BoxSize/2, BoxSize/2), FVector(BoxSize, BoxSize, BoxSize));
			NewActor->UniformScale = ScaleComponent;
		}
	}



	auto MakeAxisScaleFunc = [World, NewActor, GizmoViewContext, GizmoLineThickness, &AddMeshGizmoComponent](EAxis::Type ElementAxis, const FVector& PerpendicularAxis, bool bLockSinglePlane,
		TObjectPtr<UPrimitiveComponent>& FullHandleOut) -> UPrimitiveComponent*
	{
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			if (UViewAdjustedStaticMeshGizmoComponent* Component = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoBoxHandle"),
				UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis),
				UE::GizmoUtil::GetRotatedBasisTransform(
					FTransform(FQuat::Identity, FVector(130, 0, 0), FVector(0.8)),
					ElementAxis),
				EMirror::WhenCombined))
			{
				Component->SetRenderVisibilityFunction(ShouldAxisBeVisible);

				// Also try to add a full handle to use when we're not using a combined gizmo.
				FullHandleOut = AddMeshGizmoComponent(
					TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoBoxArrowHandle"),
					UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis),
					UE::GizmoUtil::GetRotatedBasisTransform(
						FTransform(FQuat::Identity, FVector::ZeroVector, FVector::One()),
						ElementAxis),
					EMirror::WhenCombined);
				if (FullHandleOut)
				{
					FullHandleOut->SetVisibility(false);
				}

				return Component;
			}
		}

		UGizmoRectangleComponent* ScaleComponent = AddDefaultRectangleComponent(World, NewActor, GizmoViewContext, 
			AxisToLegacyColor(ElementAxis), AxisToVector(ElementAxis), PerpendicularAxis);
		ScaleComponent->OffsetX = 140.0f; ScaleComponent->OffsetY = -10.0f;
		ScaleComponent->LengthX = 7.0f; ScaleComponent->LengthY = 20.0f;
		ScaleComponent->Thickness = GizmoLineThickness;
		ScaleComponent->bOrientYAccordingToCamera = !bLockSinglePlane;
		ScaleComponent->NotifyExternalPropertyUpdates();
		ScaleComponent->SegmentFlags = 0x1 | 0x2 | 0x4; // | 0x8;
		return ScaleComponent;
	};

	// This is designed so we can properly handle the visual orientations of the scale handles under the condition of a
	// planar gizmo (such as in the UV Editor).
	// In this case we want to lock the handle on to the other axis of the plane, rather than use the component's camera orientation option. This requires
	// both tracking how many axes are being requested and also *which* axes are requested, in order to configure the correct planar basis vectors.
	// In the case of a single axis, we have to pick a cross axis arbitrarily, but we also keep the auto orientation mode on the component active, so the initial
	// choice isn't as critical. If we want to some day have a single axis handle that is locked, we may need to revisit this again.
	auto ConfigureAdditionalAxis = [&Elements](ETransformGizmoSubElements AxisToTest, int32& TotalAxisCount, FVector& NewPerpendicularAxis) {
		if ((Elements & ETransformGizmoSubElements::ScaleAxisX & AxisToTest) != ETransformGizmoSubElements::None)
		{
			TotalAxisCount++;
			NewPerpendicularAxis = FVector(1, 0, 0);
			return;
		}
		if ((Elements & ETransformGizmoSubElements::ScaleAxisY & AxisToTest) != ETransformGizmoSubElements::None)
		{
			TotalAxisCount++;
			NewPerpendicularAxis = FVector(0, 1, 0);
			return;
		}
		if ((Elements & ETransformGizmoSubElements::ScaleAxisZ & AxisToTest) != ETransformGizmoSubElements::None)
		{
			TotalAxisCount++;
			NewPerpendicularAxis = FVector(0, 0, 1);
			return;
		}
	};

	if ((Elements & ETransformGizmoSubElements::ScaleAxisX) != ETransformGizmoSubElements::None)
	{
		int32 TotalAxisCount = 1;
		FVector PerpendicularAxis(0,1,0);
		ConfigureAdditionalAxis(ETransformGizmoSubElements::ScaleAxisY, TotalAxisCount, PerpendicularAxis);
		ConfigureAdditionalAxis(ETransformGizmoSubElements::ScaleAxisZ, TotalAxisCount, PerpendicularAxis);
		NewActor->AxisScaleX = MakeAxisScaleFunc(EAxis::X, PerpendicularAxis, TotalAxisCount == 2, 
			NewActor->FullAxisScaleX);
	}

	if ((Elements & ETransformGizmoSubElements::ScaleAxisY) != ETransformGizmoSubElements::None)
	{
		int32 TotalAxisCount = 1;
		FVector PerpendicularAxis(1, 0, 0);
		ConfigureAdditionalAxis(ETransformGizmoSubElements::ScaleAxisX, TotalAxisCount, PerpendicularAxis);
		ConfigureAdditionalAxis(ETransformGizmoSubElements::ScaleAxisZ, TotalAxisCount, PerpendicularAxis);
		NewActor->AxisScaleY = MakeAxisScaleFunc(EAxis::Y, PerpendicularAxis, TotalAxisCount == 2, 
			NewActor->FullAxisScaleY);
	}

	if ((Elements & ETransformGizmoSubElements::ScaleAxisZ) != ETransformGizmoSubElements::None)
	{
		int32 TotalAxisCount = 1;
		FVector PerpendicularAxis(1, 0, 0);
		ConfigureAdditionalAxis(ETransformGizmoSubElements::ScaleAxisY, TotalAxisCount, PerpendicularAxis);
		ConfigureAdditionalAxis(ETransformGizmoSubElements::ScaleAxisX, TotalAxisCount, PerpendicularAxis);
		NewActor->AxisScaleZ = MakeAxisScaleFunc(EAxis::Z, PerpendicularAxis, TotalAxisCount == 2, 
			NewActor->FullAxisScaleZ);
	}


	auto MakePlaneScaleFunc = [World, NewActor, GizmoViewContext, GizmoLineThickness, &AddMeshGizmoComponent](EAxis::Type ElementAxis) -> UPrimitiveComponent*
	{
		if (GizmoDrawMode == DrawModeValue_Meshes)
		{
			if (UViewAdjustedStaticMeshGizmoComponent* Component = AddMeshGizmoComponent(
				TEXT("/Engine/InteractiveToolsFramework/Meshes/GizmoCornerHandle"),
				UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis),
				UE::GizmoUtil::GetRotatedBasisTransform(
					// Transform for the X axis, relative to gizmo root
					FTransform(FQuat::Identity, CornerScalePositionCombined, FVector(CornerScaleHandleScale)), ElementAxis),
				// We actually adjust the transform of the plane scale handles and swap the adjuster inside
				//  ApplyGizmoActiveMode, so we don't need this one to be updated.
				EMirror::Always))
			{
				Component->SetRenderVisibilityFunction(ShouldPlaneBeVisible);
				return Component;
			}
		}

		// If we got to here, then we're creating the PDI drawn rectangle component
		FVector Axis0, Axis1;
		AxisToLegacyPairOfVectors(ElementAxis, Axis0, Axis1);
		UGizmoRectangleComponent* ScaleComponent = AddDefaultRectangleComponent(World, NewActor, GizmoViewContext,
			AxisToLegacyColor(ElementAxis), Axis0, Axis1);
		ScaleComponent->OffsetX = ScaleComponent->OffsetY = 120.0f;
		ScaleComponent->LengthX = ScaleComponent->LengthY = 20.0f;
		ScaleComponent->Thickness = GizmoLineThickness;
		ScaleComponent->NotifyExternalPropertyUpdates();
		ScaleComponent->SegmentFlags = 0x2 | 0x4;
		return ScaleComponent;
	};
	if ((Elements & ETransformGizmoSubElements::ScalePlaneYZ) != ETransformGizmoSubElements::None)
	{
		NewActor->PlaneScaleYZ = MakePlaneScaleFunc(EAxis::X);
	}
	if ((Elements & ETransformGizmoSubElements::ScalePlaneXZ) != ETransformGizmoSubElements::None)
	{
		NewActor->PlaneScaleXZ = MakePlaneScaleFunc(EAxis::Y);
	}
	if ((Elements & ETransformGizmoSubElements::ScalePlaneXY) != ETransformGizmoSubElements::None)
	{
		NewActor->PlaneScaleXY = MakePlaneScaleFunc(EAxis::Z);
	}


	return NewActor;
}




ACombinedTransformGizmoActor* FCombinedTransformGizmoActorFactory::CreateNewGizmoActor(UWorld* World) const
{
	return ACombinedTransformGizmoActor::ConstructCustom3AxisGizmo(World, GizmoViewContext, EnableElements);
}



UInteractiveGizmo* UCombinedTransformGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UCombinedTransformGizmo* NewGizmo = NewObject<UCombinedTransformGizmo>(SceneState.GizmoManager);
	NewGizmo->SetWorld(SceneState.World);

	UGizmoViewContext* GizmoViewContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	check(GizmoViewContext && GizmoViewContext->IsValidLowLevel());

	// use default gizmo actor if client has not given us a new builder
	NewGizmo->SetGizmoActorBuilder(GizmoActorBuilder ? GizmoActorBuilder : MakeShared<FCombinedTransformGizmoActorFactory>(GizmoViewContext));

	NewGizmo->SetSubGizmoBuilderIdentifiers(AxisPositionBuilderIdentifier, PlanePositionBuilderIdentifier, AxisAngleBuilderIdentifier);

	// override default hover function if proposed
	if (UpdateHoverFunction)
	{
		NewGizmo->SetUpdateHoverFunction(UpdateHoverFunction);
	}

	if (UpdateCoordSystemFunction)
	{
		NewGizmo->SetUpdateCoordSystemFunction(UpdateCoordSystemFunction);
	}

	return NewGizmo;
}



void UCombinedTransformGizmo::SetWorld(UWorld* WorldIn)
{
	this->World = WorldIn;
}

void UCombinedTransformGizmo::SetGizmoActorBuilder(TSharedPtr<FCombinedTransformGizmoActorFactory> Builder)
{
	GizmoActorBuilder = Builder;
}

void UCombinedTransformGizmo::SetSubGizmoBuilderIdentifiers(FString AxisPositionBuilderIdentifierIn, FString PlanePositionBuilderIdentifierIn, FString AxisAngleBuilderIdentifierIn)
{
	AxisPositionBuilderIdentifier = AxisPositionBuilderIdentifierIn;
	PlanePositionBuilderIdentifier = PlanePositionBuilderIdentifierIn;
	AxisAngleBuilderIdentifier = AxisAngleBuilderIdentifierIn;
}

void UCombinedTransformGizmo::SetUpdateHoverFunction(TFunction<void(UPrimitiveComponent*, bool)> HoverFunction)
{
	UpdateHoverFunction = HoverFunction;
}

void UCombinedTransformGizmo::SetUpdateCoordSystemFunction(TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> CoordSysFunction)
{
	UpdateCoordSystemFunction = CoordSysFunction;
}

bool UCombinedTransformGizmo::SetSubGizmoComponent(ETransformGizmoSubElements Element, 
	UPrimitiveComponent* NewComponent, const FTransform& SubGizmoToGizmo)
{
	using namespace UE::Geometry;
	using namespace CombinedTransformGizmoLocals;

	if (!IsValid(GizmoActor))
	{
		return false;
	}

	EAxis::Type Axis = UE::GizmoUtil::ToAxis(Element);

	UPrimitiveComponent* ReplacedComponent = nullptr;
	if (!GizmoActor->ReplaceSubGizmoComponent(Element, NewComponent, SubGizmoToGizmo, &ReplacedComponent))
	{
		return false;
	}

	if (!ActiveTarget)
	{
		// If the target is not set yet, then we're done for now. The rest of the setup
		// should end up being done correctly once SetActiveTarget is called.
		return true;
	}

	// If we got here, we'll need to do some more work to initialize or reinitialize our gizmo

	// Look for the existing gizmo through our gizmo info arrays
	if (ReplacedComponent)
	{
		ActiveComponents.RemoveSwap(ReplacedComponent);

		TArray<FSubGizmoInfo>* ArrayToSearch = nullptr;
		if ((Element & (ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes))
			!= ETransformGizmoSubElements::None)
		{
			ArrayToSearch = &TranslationSubGizmos;
		}
		else if ((Element & ETransformGizmoSubElements::RotateAllAxes) != ETransformGizmoSubElements::None)
		{
			ArrayToSearch = &RotationSubGizmos;
		}
		else if ((Element & (ETransformGizmoSubElements::ScaleAllAxes | ETransformGizmoSubElements::ScaleAllPlanes))
			!= ETransformGizmoSubElements::None)
		{
			ArrayToSearch = &NonUniformScaleSubGizmos;
		}
		else if ((Element & ETransformGizmoSubElements::ScaleUniform) != ETransformGizmoSubElements::None)
		{
			ArrayToSearch = &UniformScaleSubGizmos;
		}

		int32 GizmoInfoIndex = -1;
		FSubGizmoInfo* ExistingGizmoInfo = nullptr;
		if (ensure(ArrayToSearch))
		{
			GizmoInfoIndex = ArrayToSearch->IndexOfByPredicate(
				[ReplacedComponent](const FSubGizmoInfo& GizmoInfo) { return GizmoInfo.Component == ReplacedComponent; });
			if (GizmoInfoIndex >= 0)
			{
				ExistingGizmoInfo = &(*ArrayToSearch)[GizmoInfoIndex];
			}
		}

		if (ensure(ExistingGizmoInfo))
		{
			// We could call InitializeAs... on an existing gizmo to swap the component, but then we also need to set
			// our constraint functions, etc. It seems cleaner code-wise to just destroy this gizmo and create a new one
			// to make sure everything is updated. We just have to make sure we do the removal thoroughly.
			if (UInteractiveGizmo* ExistingGizmo = ExistingGizmoInfo->Gizmo.Get())
			{
				GetGizmoManager()->DestroyGizmo(ExistingGizmo);
				ActiveGizmos.Remove(ExistingGizmo);
			}
			ArrayToSearch->RemoveAtSwap(GizmoInfoIndex);
		}
	}

	if (!NewComponent)
	{
		// If we're replacing with a nullptr, then we just wanted to remove that component.
		// No need to add a gizmo back.
		return true;
	}

	UE::GizmoUtil::FTransformSubGizmoCommonParams Params;
	Params.TransformProxy = ActiveTarget;
	Params.Axis = Axis;
	Params.Component = NewComponent;
	Params.TransactionProvider = TransactionProviderAtLastSetActiveTarget;
	Params.bManipulatesRootComponent = true;

	// The shared data struct should have been created during SetActiveTarget
	if (!ensure(SubGizmoSharedState.IsValid()))
	{
		SubGizmoSharedState = MakeUnique<FTransformSubGizmoSharedState>();
	}

	switch (Element)
	{
	case ETransformGizmoSubElements::TranslateAxisX:
	case ETransformGizmoSubElements::TranslateAxisY:
	case ETransformGizmoSubElements::TranslateAxisZ:
	{
		AddAxisTranslationGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::TranslatePlaneXY:
	case ETransformGizmoSubElements::TranslatePlaneXZ:
	case ETransformGizmoSubElements::TranslatePlaneYZ:
	{
		AddPlaneTranslationGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::RotateAxisX:
	case ETransformGizmoSubElements::RotateAxisY:
	case ETransformGizmoSubElements::RotateAxisZ:
	{
		AddAxisRotationGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::ScaleAxisX:
	case ETransformGizmoSubElements::ScaleAxisY:
	case ETransformGizmoSubElements::ScaleAxisZ:
	{
		AddAxisScaleGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::ScalePlaneXY:
	case ETransformGizmoSubElements::ScalePlaneXZ:
	case ETransformGizmoSubElements::ScalePlaneYZ:
	{
		AddPlaneScaleGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::ScaleUniform:
	{
		AddUniformScaleGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::FreeTranslate:
	{
		AddFreeTranslationGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::FreeRotate:
	{
		AddFreeRotationGizmo(Params, *SubGizmoSharedState);
		break;
	}
	case ETransformGizmoSubElements::RotateAllAxes:
	{
		// no gizmo for the drawn sphere
		if (ensure(GizmoActor->RotationSphere == NewComponent))
		{
			ActiveComponents.Add(GizmoActor->RotationSphere);
			RotationSubGizmos.Add(FSubGizmoInfo{ GizmoActor->RotationSphere, nullptr });
		}
		break;
	}
	default:
		return ensure(false);
	}

	return true;
}

void UCombinedTransformGizmo::SetWorldAlignmentFunctions(
	TUniqueFunction<bool()>&& ShouldAlignTranslationIn,
	TUniqueFunction<bool(const FRay&, FVector&)>&& TranslationAlignmentRayCasterIn)
{
	// Save these so that later changes of gizmo target keep the settings.
	ShouldAlignDestination = MoveTemp(ShouldAlignTranslationIn);
	DestinationAlignmentRayCaster = MoveTemp(TranslationAlignmentRayCasterIn);

	if (!ensure(IsValid(GizmoActor)))
	{
		return;
	}

	// We allow this function to be called after Setup(), so modify any existing translation/rotation sub gizmos.
	// Unfortunately we keep all the sub gizmos in one list, and the scaling gizmos are differentiated from the
	// translation ones mainly in the components they use. So this ends up being a slightly messy set of checks,
	// but it didn't seem worth keeping a segregated list for something that will only happen once.
	for (UInteractiveGizmo* SubGizmo : this->ActiveGizmos)
	{
		if (UAxisPositionGizmo* CastGizmo = Cast<UAxisPositionGizmo>(SubGizmo))
		{
			if (UGizmoComponentHitTarget* CastHitTarget = Cast<UGizmoComponentHitTarget>(CastGizmo->HitTarget.GetObject()))
			{
				if (CastHitTarget->Component == GizmoActor->TranslateX
					|| CastHitTarget->Component == GizmoActor->TranslateY
					|| CastHitTarget->Component == GizmoActor->TranslateZ)
				{
					CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
					CastGizmo->CustomDestinationFunc = 
						[this](const UAxisPositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) { 
						return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint); 
					};
				}
			}
		}
		if (UPlanePositionGizmo* CastGizmo = Cast<UPlanePositionGizmo>(SubGizmo))
		{
			if (UGizmoComponentHitTarget* CastHitTarget = Cast<UGizmoComponentHitTarget>(CastGizmo->HitTarget.GetObject()))
			{
				if (CastHitTarget->Component == GizmoActor->TranslateXY
					|| CastHitTarget->Component == GizmoActor->TranslateXZ
					|| CastHitTarget->Component == GizmoActor->TranslateYZ
					|| CastHitTarget->Component == GizmoActor->FreeTranslateHandle)
				{
					CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
					CastGizmo->CustomDestinationFunc =
						[this](const UPlanePositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
						return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
					};
				}
			}
		}
		if (UAxisAngleGizmo* CastGizmo = Cast<UAxisAngleGizmo>(SubGizmo))
		{
			CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
			CastGizmo->CustomDestinationFunc =
				[this](const UAxisAngleGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
				return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
			};
		}
	}
}

void UCombinedTransformGizmo::SetCustomTranslationDeltaFunctions(
	TFunction<bool(double AxisDelta, double& SnappedDelta)> XAxis, 
	TFunction<bool(double AxisDelta, double& SnappedDelta)> YAxis, 
	TFunction<bool(double AxisDelta, double& SnappedDelta)> ZAxis)
{
	CustomTranslationDeltaConstraintFunctions[0] = XAxis;
	CustomTranslationDeltaConstraintFunctions[1] = YAxis;
	CustomTranslationDeltaConstraintFunctions[2] = ZAxis;
}

void UCombinedTransformGizmo::SetCustomRotationDeltaFunctions(
	TFunction<bool(double AxisDelta, double& SnappedDelta)> XAxis,
	TFunction<bool(double AxisDelta, double& SnappedDelta)> YAxis, 
	TFunction<bool(double AxisDelta, double& SnappedDelta)> ZAxis)
{
	CustomRotationDeltaConstraintFunctions[0] = XAxis;
	CustomRotationDeltaConstraintFunctions[1] = YAxis;
	CustomRotationDeltaConstraintFunctions[2] = ZAxis;
}

void UCombinedTransformGizmo::SetCustomScaleDeltaFunctions(
	TFunction<bool(double AxisDelta, double& SnappedDelta)> XAxis, 
	TFunction<bool(double AxisDelta, double& SnappedDelta)> YAxis, 
	TFunction<bool(double AxisDelta, double& SnappedDelta)> ZAxis)
{
	CustomScaleDeltaConstraintFunctions[0] = XAxis;
	CustomScaleDeltaConstraintFunctions[1] = YAxis;
	CustomScaleDeltaConstraintFunctions[2] = ZAxis;
}

void UCombinedTransformGizmo::SetDisallowNegativeScaling(bool bDisallow)
{
	if (bDisallowNegativeScaling != bDisallow)
	{
		bDisallowNegativeScaling = bDisallow;
		for (UInteractiveGizmo* SubGizmo : this->ActiveGizmos)
		{
			if (UAxisPositionGizmo* CastGizmo = Cast<UAxisPositionGizmo>(SubGizmo))
			{
				if (UGizmoAxisScaleParameterSource* ParamSource = Cast<UGizmoAxisScaleParameterSource>(CastGizmo->ParameterSource.GetObject()))
				{
					ParamSource->bClampToZero = bDisallow;
				}
			}
			if (UPlanePositionGizmo* CastGizmo = Cast<UPlanePositionGizmo>(SubGizmo))
			{
				if (UGizmoPlaneScaleParameterSource* ParamSource = Cast<UGizmoPlaneScaleParameterSource>(CastGizmo->ParameterSource.GetObject()))
				{
					ParamSource->bClampToZero = bDisallow;
				}
			}
		}
	}
}

void UCombinedTransformGizmo::SetIsNonUniformScaleAllowedFunction(TUniqueFunction<bool()>&& IsNonUniformScaleAllowedIn)
{
	IsNonUniformScaleAllowedFunc = MoveTemp(IsNonUniformScaleAllowedIn);
}


void UCombinedTransformGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	if (!UpdateHoverFunction)
	{
		UpdateHoverFunction = [](UPrimitiveComponent* Component, bool bHovering)
		{
			if (IGizmoBaseComponentInterface* CastComponent = Cast<IGizmoBaseComponentInterface>(Component))
			{
				CastComponent->UpdateHoverState(bHovering);
			}
		};
	}

	if (!UpdateCoordSystemFunction)
	{
		UpdateCoordSystemFunction = [](UPrimitiveComponent* Component, EToolContextCoordinateSystem CoordSystem)
		{
			if (IGizmoBaseComponentInterface* CastComponent = Cast<IGizmoBaseComponentInterface>(Component))
			{
				CastComponent->UpdateWorldLocalState(CoordSystem == EToolContextCoordinateSystem::World);
			}
		};
	}

	GizmoActor = GizmoActorBuilder->CreateNewGizmoActor(World);

	PreviousActiveGizmoMode = ActiveGizmoMode;
}

void UCombinedTransformGizmo::Shutdown()
{
	ClearActiveTarget();

	if (IsValid(GizmoActor))
	{
		GizmoActor->Destroy();
		GizmoActor = nullptr;
	}
}



void UCombinedTransformGizmo::UpdateCameraAxisSource()
{
	if (CameraAxisSource != nullptr && IsValid(GizmoActor))
	{
		UE::GizmoUtil::UpdateCameraAxisSource(*CameraAxisSource, GetGizmoManager(), 
			GizmoActor->GetTransform().GetLocation());
	}
}


void UCombinedTransformGizmo::Tick(float DeltaTime)
{
	if (bUseContextCoordinateSystem)
	{
		CurrentCoordinateSystem = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	}
	check(CurrentCoordinateSystem == EToolContextCoordinateSystem::World || CurrentCoordinateSystem == EToolContextCoordinateSystem::Local)

	FToolContextSnappingConfiguration SnappingConfig = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
	RelativeTranslationSnapping.UpdateContextValue(SnappingConfig.bEnableAbsoluteWorldSnapping == false);

	bool bUseLocalAxes = (CurrentCoordinateSystem == EToolContextCoordinateSystem::Local);
	if (AxisXSource != nullptr && AxisYSource != nullptr && AxisZSource != nullptr)
	{
		AxisXSource->bLocalAxes = bUseLocalAxes;
		AxisYSource->bLocalAxes = bUseLocalAxes;
		AxisZSource->bLocalAxes = bUseLocalAxes;
	}
	if (UpdateCoordSystemFunction)
	{
		for (UPrimitiveComponent* Component : ActiveComponents)
		{
			UpdateCoordSystemFunction(Component, CurrentCoordinateSystem);
		}
	}

	if (bUseContextGizmoMode)
	{
		ActiveGizmoMode = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentTransformGizmoMode();
	}

	// apply dynamic visibility filtering to sub-gizmos
	if (PreviousActiveGizmoMode != ActiveGizmoMode)
	{
		ApplyGizmoActiveMode();
	}

	UpdateCameraAxisSource();
}

void UCombinedTransformGizmo::ApplyGizmoActiveMode()
{
	using namespace CombinedTransformGizmoLocals;

	ON_SCOPE_EXIT{ PreviousActiveGizmoMode = ActiveGizmoMode; };

	auto SetSubGizmoTypeVisibility = [this](TArray<FSubGizmoInfo>& GizmoInfos, bool bVisible)
	{
		for (FSubGizmoInfo& GizmoInfo : GizmoInfos)
		{
			if (GizmoInfo.Component.IsValid())
			{
				GizmoInfo.Component->SetVisibility(bVisible);
			}
		}
	};
	bool bShouldShowTranslation =
		(ActiveGizmoMode == EToolContextTransformGizmoMode::Combined || ActiveGizmoMode == EToolContextTransformGizmoMode::Translation);
	bool bShouldShowRotation =
		(ActiveGizmoMode == EToolContextTransformGizmoMode::Combined || ActiveGizmoMode == EToolContextTransformGizmoMode::Rotation);
	bool bShouldShowUniformScale =
		(ActiveGizmoMode == EToolContextTransformGizmoMode::Combined || ActiveGizmoMode == EToolContextTransformGizmoMode::Scale);
	bool bShouldShowNonUniformScale = 
		(ActiveGizmoMode == EToolContextTransformGizmoMode::Combined || ActiveGizmoMode == EToolContextTransformGizmoMode::Scale)
		&& IsNonUniformScaleAllowedFunc();

	SetSubGizmoTypeVisibility(TranslationSubGizmos, bShouldShowTranslation);
	SetSubGizmoTypeVisibility(RotationSubGizmos, bShouldShowRotation);
	SetSubGizmoTypeVisibility(UniformScaleSubGizmos, bShouldShowUniformScale);

	// The rest of the modifications dereference GizmoActor, so go ahead and do a safety check now.
	if (!ensure(IsValid(GizmoActor)))
	{
		return;
	}

	if (ActiveGizmoMode == EToolContextTransformGizmoMode::Combined || ActiveGizmoMode == EToolContextTransformGizmoMode::Scale)
	{
		// The scale handles look different in different modes, so swap them if necessary
		auto SwapAxisScaleComponent = [this](UPrimitiveComponent* HandleInCombined, UPrimitiveComponent* HandleInSeparate)
		{
			UPrimitiveComponent* HandleToUse = ActiveGizmoMode == EToolContextTransformGizmoMode::Combined ? HandleInCombined : HandleInSeparate;
			UPrimitiveComponent* HandleToReplace = ActiveGizmoMode == EToolContextTransformGizmoMode::Combined ? HandleInSeparate : HandleInCombined;

			// Don't swap if we don't have an alternative to use
			if (!HandleToUse || !HandleToReplace) return;
				
			FSubGizmoInfo* GizmoInfo = NonUniformScaleSubGizmos.FindByPredicate([HandleToUse, HandleToReplace](const FSubGizmoInfo& Info)
			{ 
				return Info.Component == HandleToUse || Info.Component == HandleToReplace;
			});
			// Don't swap if we don't have this gizmo or if it's already using the correct one
			if (!GizmoInfo || GizmoInfo->Component == HandleToUse) return;

			UAxisPositionGizmo* SubGizmo = Cast<UAxisPositionGizmo>(GizmoInfo->Gizmo);
			if (!ensure(SubGizmo)) return;
			UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(SubGizmo->HitTarget.GetObject());
			if (!ensure(HitTarget)) return;

			HandleToReplace->SetVisibility(false);
			HitTarget->Component = HandleToUse;
			GizmoInfo->Component = HandleToUse;
		};

		SwapAxisScaleComponent(GizmoActor->AxisScaleX, GizmoActor->FullAxisScaleX);
		SwapAxisScaleComponent(GizmoActor->AxisScaleY, GizmoActor->FullAxisScaleY);
		SwapAxisScaleComponent(GizmoActor->AxisScaleZ, GizmoActor->FullAxisScaleZ);

		// The plane scale handles look better if they are closer to the gizmo when not combined.
		auto AdjustPlaneScaleComponent = [this](UViewAdjustedStaticMeshGizmoComponent* Component, EAxis::Type ElementAxis)
		{
			if (!Component) return;
			Component->SetRelativeTransform(UE::GizmoUtil::GetRotatedBasisTransform(
				// Transform for the X axis, relative to gizmo root
				FTransform(FQuat::Identity, ActiveGizmoMode == EToolContextTransformGizmoMode::Combined ? CornerScalePositionCombined : CornerScalePositionSeparate,
					FVector(CornerScaleHandleScale)), ElementAxis));

			// Just replace the adjuster
			// TODO: Maybe keep track of whether this needs doing instead of doing it each time we switch to scale mode
			TSharedPtr<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster> Adjuster =
				UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::AddTransformAdjuster(
					Component, GizmoActor->GetRootComponent(), ActiveGizmoMode == EToolContextTransformGizmoMode::Combined);
		};
		AdjustPlaneScaleComponent(Cast<UViewAdjustedStaticMeshGizmoComponent>(GizmoActor->PlaneScaleXY), EAxis::Z);
		AdjustPlaneScaleComponent(Cast<UViewAdjustedStaticMeshGizmoComponent>(GizmoActor->PlaneScaleXZ), EAxis::Y);
		AdjustPlaneScaleComponent(Cast<UViewAdjustedStaticMeshGizmoComponent>(GizmoActor->PlaneScaleYZ), EAxis::X);
	}
	// This is done after the above, since the above affects NonUniformScaleSubGizmos
	SetSubGizmoTypeVisibility(NonUniformScaleSubGizmos, bShouldShowUniformScale);

	auto SetVisibilityIfExists = [](UPrimitiveComponent* Component, bool bVisible)
	{
		if (Component)
		{
			Component->SetVisibility(bVisible);
		}
	};
	if (GizmoActor->FreeRotateHandle)
	{
		GizmoActor->FreeRotateHandle->SetVisibility(ActiveGizmoMode == EToolContextTransformGizmoMode::Rotation);
	}
	if (GizmoActor->FreeTranslateHandle)
	{
		GizmoActor->FreeTranslateHandle->SetVisibility(ActiveGizmoMode == EToolContextTransformGizmoMode::Translation
			|| (ActiveGizmoMode == EToolContextTransformGizmoMode::Combined && UniformScaleSubGizmos.Num() == 0));
	}

	if (GizmoDrawMode == DrawModeValue_Meshes)
	{
		// Many components mirror in combined mode but not separate mode, mostly because it is weird for them
		//  to not mirror when the rotation components do.
		for (int32 i = GizmoActor->AdjustersThatMirrorOnlyInCombinedMode.Num() - 1; i >= 0; --i)
		{
			TSharedPtr<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster> Adjuster =
				GizmoActor->AdjustersThatMirrorOnlyInCombinedMode[i].Pin();
			if (ensure(Adjuster))
			{
				Adjuster->SetMirrorBasedOnOctant(ActiveGizmoMode == EToolContextTransformGizmoMode::Combined);
			}
		}
	}
}



void UCombinedTransformGizmo::SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider)
{
	if (ActiveTarget != nullptr)
	{
		ClearActiveTarget();
	}

	if (!ensure(IsValid(GizmoActor)))
	{
		return;
	}

	ActiveTarget = Target;
	TransactionProviderAtLastSetActiveTarget = TransactionProvider;

	// move gizmo to target location
	USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();

	FTransform TargetTransform = Target->GetTransform();
	FTransform GizmoTransform = TargetTransform;
	GizmoTransform.SetScale3D(FVector(1, 1, 1));
	GizmoComponent->SetWorldTransform(GizmoTransform);

	UE::GizmoUtil::FTransformSubGizmoCommonParams Params;
	Params.TransformProxy = ActiveTarget;
	Params.TransactionProvider = TransactionProvider;
	Params.bManipulatesRootComponent = true;

	SubGizmoSharedState = MakeUnique<UE::GizmoUtil::FTransformSubGizmoSharedState>();

	EAxis::Type Axes[3] = { EAxis::X, EAxis::Y, EAxis::Z };
	UPrimitiveComponent* TranslateAxisComponents[3]{ GizmoActor->TranslateX, GizmoActor->TranslateY, GizmoActor->TranslateZ };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (TranslateAxisComponents[AxisIndex])
		{
			Params.Component = TranslateAxisComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			AddAxisTranslationGizmo(Params, *SubGizmoSharedState);
		}
	}
	UPrimitiveComponent* TranslatePlaneComponents[3]{ GizmoActor->TranslateYZ, GizmoActor->TranslateXZ, GizmoActor->TranslateXY };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (TranslatePlaneComponents[AxisIndex])
		{
			Params.Component = TranslatePlaneComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			AddPlaneTranslationGizmo(Params, *SubGizmoSharedState);
		}
	}
	if (GizmoActor->FreeTranslateHandle != nullptr)
	{
		Params.Component = GizmoActor->FreeTranslateHandle;
		Params.Axis = EAxis::None;
		AddFreeTranslationGizmo(Params, *SubGizmoSharedState);
	}
	UPrimitiveComponent* RotationAxisComponents[3]{ GizmoActor->RotateX, GizmoActor->RotateY, GizmoActor->RotateZ };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (RotationAxisComponents[AxisIndex])
		{
			Params.Component = RotationAxisComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			AddAxisRotationGizmo(Params, *SubGizmoSharedState);
		}
	}
	if (GizmoActor->RotationSphere != nullptr)
	{
		ActiveComponents.Add(GizmoActor->RotationSphere);
		RotationSubGizmos.Add(FSubGizmoInfo{ GizmoActor->RotationSphere, nullptr });
	}
	if (GizmoActor->FreeRotateHandle != nullptr)
	{
		Params.Component = GizmoActor->FreeRotateHandle;
		Params.Axis = EAxis::None;
		AddFreeRotationGizmo(Params, *SubGizmoSharedState);
	}
	if (GizmoActor->UniformScale != nullptr)
	{
		Params.Component = GizmoActor->UniformScale;
		Params.Axis = EAxis::None;
		AddUniformScaleGizmo(Params, *SubGizmoSharedState);
	}
	UPrimitiveComponent* ScaleAxisComponents[3]{ GizmoActor->AxisScaleX, GizmoActor->AxisScaleY, GizmoActor->AxisScaleZ };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (ScaleAxisComponents[AxisIndex])
		{
			Params.Component = ScaleAxisComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			AddAxisScaleGizmo(Params, *SubGizmoSharedState);
		}
	}
	UPrimitiveComponent* ScalePlaneComponents[3]{ GizmoActor->PlaneScaleYZ, GizmoActor->PlaneScaleXZ, GizmoActor->PlaneScaleXY };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (ScalePlaneComponents[AxisIndex])
		{
			Params.Component = ScalePlaneComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			AddPlaneScaleGizmo(Params, *SubGizmoSharedState);
		}
	}

	// Unpack the shared state into our properties. It might be nicer to just hold on to the shared state
	//  object (in case it is needed later), but we do this for compatibility with existing child classes.
	StateTarget = SubGizmoSharedState->StateTarget;
	AxisXSource = SubGizmoSharedState->CardinalAxisSources[0];
	AxisYSource = SubGizmoSharedState->CardinalAxisSources[1];
	AxisZSource = SubGizmoSharedState->CardinalAxisSources[2];
	CameraAxisSource = SubGizmoSharedState->CameraAxisSource;
	UnitAxisXSource = SubGizmoSharedState->UnitCardinalAxisSources[0];
	UnitAxisYSource = SubGizmoSharedState->UnitCardinalAxisSources[1];
	UnitAxisZSource = SubGizmoSharedState->UnitCardinalAxisSources[2];

	ApplyGizmoActiveMode();

	OnSetActiveTarget.Broadcast(this, ActiveTarget);
}

FTransform UCombinedTransformGizmo::GetGizmoTransform() const
{
	if (!ensure(IsValid(GizmoActor)))
	{
		return FTransform::Identity;
	}
	USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();
	return GizmoComponent->GetComponentTransform();
}

void UCombinedTransformGizmo::ReinitializeGizmoTransform(const FTransform& NewTransform, bool bKeepGizmoUnscaled)
{
	if (!ensure(IsValid(GizmoActor)))
	{
		return;
	}

	// To update the gizmo location without triggering any callbacks, we temporarily
	// store a copy of the callback list, detach them, reposition, and then reattach
	// the callbacks.
	USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();
	auto temp = GizmoComponent->TransformUpdated;
	GizmoComponent->TransformUpdated.Clear();
	FTransform GizmoTransform = NewTransform;
	if (bKeepGizmoUnscaled)
	{
		GizmoTransform.SetScale3D(FVector(1, 1, 1));
	}
	GizmoComponent->SetWorldTransform(GizmoTransform);
	GizmoComponent->TransformUpdated = temp;

	// The underlying proxy has an existing way to reinitialize its transform without callbacks.
	bool bSavedSetPivotMode = ActiveTarget->bSetPivotMode;
	ActiveTarget->bSetPivotMode = true;
	ActiveTarget->SetTransform(NewTransform);
	ActiveTarget->bSetPivotMode = bSavedSetPivotMode;
}

void UCombinedTransformGizmo::SetNewGizmoTransform(const FTransform& NewTransform, bool bKeepGizmoUnscaled)
{
	if (!ensure(ActiveTarget))
	{
		return;
	}
	
	BeginTransformEditSequence();
	UpdateTransformDuringEditSequence(NewTransform, bKeepGizmoUnscaled);
	EndTransformEditSequence();
}

void UCombinedTransformGizmo::BeginTransformEditSequence()
{
	if (ensure(StateTarget))
	{
		StateTarget->BeginUpdate();
	}
}

void UCombinedTransformGizmo::EndTransformEditSequence()
{
	if (ensure(StateTarget))
	{
		StateTarget->EndUpdate();
	}
}

void UCombinedTransformGizmo::UpdateTransformDuringEditSequence(const FTransform& NewTransform, bool bKeepGizmoUnscaled)
{
	if (!ensure(ActiveTarget && IsValid(GizmoActor)))
	{
		return;
	}

	USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();
	FTransform GizmoTransform = NewTransform;
	if (bKeepGizmoUnscaled)
	{
		GizmoTransform.SetScale3D(FVector(1, 1, 1));
	}
	GizmoComponent->SetWorldTransform(GizmoTransform);
	ActiveTarget->SetTransform(NewTransform);
}

void UCombinedTransformGizmo::SetNewChildScale(const FVector& NewChildScale)
{
	FTransform NewTransform = ActiveTarget->GetTransform();
	NewTransform.SetScale3D(NewChildScale);

	bool bSavedSetPivotMode = ActiveTarget->bSetPivotMode;
	ActiveTarget->bSetPivotMode = true;
	ActiveTarget->SetTransform(NewTransform);
	ActiveTarget->bSetPivotMode = bSavedSetPivotMode;
}


void UCombinedTransformGizmo::SetVisibility(bool bVisible)
{
	if (!ensure(IsValid(GizmoActor)))
	{
		return;
	}

	bool bPreviousVisibility = !GizmoActor->IsHidden();

	GizmoActor->SetActorHiddenInGame(bVisible == false);
#if WITH_EDITOR
	GizmoActor->SetIsTemporarilyHiddenInEditor(bVisible == false);
#endif

	if (bPreviousVisibility != bVisible)
	{
		OnVisibilityChanged.Broadcast(this, bVisible);
	}
}

void UCombinedTransformGizmo::SetDisplaySpaceTransform(TOptional<FTransform> TransformIn)
{
	if (DisplaySpaceTransform.IsSet() != TransformIn.IsSet()
		|| (TransformIn.IsSet() && !TransformIn.GetValue().Equals(DisplaySpaceTransform.GetValue())))
	{
		DisplaySpaceTransform = TransformIn;
		OnDisplaySpaceTransformChanged.Broadcast(this, TransformIn);
	}
}

ETransformGizmoSubElements UCombinedTransformGizmo::GetGizmoElements()
{
	using namespace CombinedTransformGizmoLocals;

	return GetSubElementFlagsFromActor(GizmoActor.Get());
}

UInteractiveGizmo* UCombinedTransformGizmo::AddAxisTranslationGizmo(
	FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState)
{
	UAxisPositionGizmo* Gizmo = Cast<UAxisPositionGizmo>(GetGizmoManager()->CreateGizmo(AxisPositionBuilderIdentifier));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	ensure(Gizmo->InitializeAsTranslateGizmo(Params, &SharedState));

	if (UGizmoAxisTranslationParameterSource* ParamSource = Cast<UGizmoAxisTranslationParameterSource>(Gizmo->ParameterSource.GetObject()))
	{
		int AxisIndex = Params.GetClampedAxisIndex();
		ParamSource->PositionConstraintFunction = [this](const FVector& Pos, FVector& Snapped) { return PositionSnapFunction(Pos, Snapped); };
		ParamSource->AxisDeltaConstraintFunction = [this, AxisIndex](double AxisDelta, double& SnappedAxisDelta) { return PositionAxisDeltaSnapFunction(AxisDelta, SnappedAxisDelta, AxisIndex); };
	}
	else
	{
		ensure(false);
	}

	TranslationSubGizmos.Add(FSubGizmoInfo{ Params.Component, Gizmo });
	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddPlaneTranslationGizmo(
	FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState)
{
	UPlanePositionGizmo* Gizmo = Cast<UPlanePositionGizmo>(GetGizmoManager()->CreateGizmo(PlanePositionBuilderIdentifier));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	ensure(Gizmo->InitializeAsTranslateGizmo(Params, &SharedState));
	
	if (UGizmoPlaneTranslationParameterSource* ParamSource = Cast<UGizmoPlaneTranslationParameterSource>(Gizmo->ParameterSource.GetObject()))
	{
		int AxisIndex = Params.GetClampedAxisIndex();
		int XAxes[3] = {1, 2, 0};
		int YAxes[3] = {2, 0, 1};
		ParamSource->PositionConstraintFunction = [this](const FVector& Pos, FVector& Snapped) { return PositionSnapFunction(Pos, Snapped); };
		ParamSource->AxisXDeltaConstraintFunction = [this, XAxisIndex = XAxes[AxisIndex]](double AxisDelta, double& SnappedAxisDelta) { return PositionAxisDeltaSnapFunction(AxisDelta, SnappedAxisDelta, XAxisIndex); };
		ParamSource->AxisYDeltaConstraintFunction = [this, YAxisIndex = YAxes[AxisIndex]](double AxisDelta, double& SnappedAxisDelta) { return PositionAxisDeltaSnapFunction(AxisDelta, SnappedAxisDelta, YAxisIndex); };
	}
	else
	{
		ensure(false);
	}

	TranslationSubGizmos.Add(FSubGizmoInfo{ Params.Component, Gizmo });
	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddAxisRotationGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UAxisAngleGizmo* Gizmo = Cast<UAxisAngleGizmo>(GetGizmoManager()->CreateGizmo(AxisAngleBuilderIdentifier));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	ensure(Gizmo->InitializeAsRotateGizmo(Params, &SharedState));

	if (UGizmoAxisRotationParameterSource* AngleSource = Cast<UGizmoAxisRotationParameterSource>(Gizmo->AngleSource.GetObject()))
	{
		int AxisIndex = Params.GetClampedAxisIndex();
		AngleSource->AngleDeltaConstraintFunction = [this, AxisIndex](double AngleDelta, double& SnappedDelta) { return RotationAxisAngleSnapFunction(AngleDelta, SnappedDelta, AxisIndex); };
	}
	else
	{
		ensure(false);
	}

	RotationSubGizmos.Add(FSubGizmoInfo{ Params.Component, Gizmo });
	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddAxisScaleGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UAxisPositionGizmo* Gizmo = Cast<UAxisPositionGizmo>(GetGizmoManager()->CreateGizmo(AxisPositionBuilderIdentifier));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	ensure(Gizmo->InitializeAsScaleGizmo(Params, bDisallowNegativeScaling, &SharedState));

	if (UGizmoAxisScaleParameterSource* ParameterSource = Cast<UGizmoAxisScaleParameterSource>(Gizmo->ParameterSource.GetObject()))
	{
		int AxisIndex = Params.GetClampedAxisIndex();
		ParameterSource->ScaleAxisDeltaConstraintFunction = [this, AxisIndex](const double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta, AxisIndex); };
	}
	else
	{
		ensure(false);
	}
	
	NonUniformScaleSubGizmos.Add(FSubGizmoInfo{ Params.Component, Gizmo });
	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddPlaneScaleGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UPlanePositionGizmo* Gizmo = Cast<UPlanePositionGizmo>(GetGizmoManager()->CreateGizmo(PlanePositionBuilderIdentifier));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	ensure(Gizmo->InitializeAsScaleGizmo(Params, bDisallowNegativeScaling, &SharedState));

	if (UGizmoPlaneScaleParameterSource* ParameterSource = Cast<UGizmoPlaneScaleParameterSource>(Gizmo->ParameterSource.GetObject()))
	{
		int AxisIndex = Params.GetClampedAxisIndex();
		int XAxes[3] = { 1, 2, 0 };
		int YAxes[3] = { 2, 0, 1 };
		ParameterSource->ScaleAxisXDeltaConstraintFunction = [this, XAxisIndex = XAxes[AxisIndex]](double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta, XAxisIndex); };
		ParameterSource->ScaleAxisYDeltaConstraintFunction = [this, YAxisIndex = YAxes[AxisIndex]](double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta, YAxisIndex); };
	}
	else
	{
		ensure(false);
	}

	NonUniformScaleSubGizmos.Add(FSubGizmoInfo{ Params.Component, Gizmo });
	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddUniformScaleGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UPlanePositionGizmo* Gizmo = Cast<UPlanePositionGizmo>(GetGizmoManager()->CreateGizmo(PlanePositionBuilderIdentifier));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	ensure(Gizmo->InitializeAsUniformScaleGizmo(Params, bDisallowNegativeScaling, &SharedState));

	if (UGizmoUniformScaleParameterSource* ParameterSource = Cast<UGizmoUniformScaleParameterSource>(Gizmo->ParameterSource.GetObject()))
	{
		ParameterSource->ScaleAxisDeltaConstraintFunction = [this](const double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta); };
	}
	else
	{
		ensure(false);
	}
	
	UniformScaleSubGizmos.Add(FSubGizmoInfo{ Params.Component, Gizmo });
	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddFreeTranslationGizmo(
	FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState)
{
	UFreePositionSubGizmo* Gizmo = UE::GizmoUtil::CreateGizmoViaSimpleBuilder<UFreePositionSubGizmo>(
		GetGizmoManager(), FString(), this);
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	ensure(Gizmo->InitializeAsScreenPlaneTranslateGizmo(Params, &SharedState));
	if (UGizmoPlaneTranslationParameterSource* ParameterSource = Cast<UGizmoPlaneTranslationParameterSource>(Gizmo->ParameterSource.GetObject()))
	{
		ParameterSource->PositionConstraintFunction = [this](const FVector& Pos, FVector& Snapped) { return PositionSnapFunction(Pos, Snapped); };
		ParameterSource->bProjectConstrainedPosition = false;
		ParameterSource->AxisXDeltaConstraintFunction = [this, XAxisIndex = 0](double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return PositionAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta, XAxisIndex); };
		ParameterSource->AxisYDeltaConstraintFunction = [this, YAxisIndex = 1](double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return PositionAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta, YAxisIndex); };
	}
	else
	{
		ensure(false);
	}


	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}

UInteractiveGizmo* UCombinedTransformGizmo::AddFreeRotationGizmo(
	FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState)
{
	UFreeRotationSubGizmo* Gizmo = UE::GizmoUtil::CreateGizmoViaSimpleBuilder<UFreeRotationSubGizmo>(
		GetGizmoManager(), FString(), this);
	if (!ensure(Gizmo))
	{
		return nullptr;
	}
	UGizmoViewContext* GizmoViewContext = UE::GizmoUtil::GetGizmoViewContext(GetGizmoManager());
	ensure(Gizmo->InitializeAsRotationGizmo(Params, GizmoViewContext, &SharedState));

	ActiveComponents.Add(Params.Component);
	ActiveGizmos.Add(Gizmo);

	return Gizmo;
}


// These are deprecated initialization functions that do sub gizmo initialization by hand instead of using the
//  "InitializeAs..." functions that were added to subgizmos to make them simpler to instantiate outside of
//  this class.
UInteractiveGizmo* UCombinedTransformGizmo::AddAxisTranslationGizmo(
	UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
	IGizmoAxisSource* AxisSource,
	IGizmoTransformSource* TransformSource,
	IGizmoStateTarget* StateTargetIn,
	int AxisIndex)
{
	// create axis-position gizmo, axis-position parameter will drive translation
	UAxisPositionGizmo* TranslateGizmo = Cast<UAxisPositionGizmo>(GetGizmoManager()->CreateGizmo(AxisPositionBuilderIdentifier));
	check(TranslateGizmo);

	// axis source provides the translation axis
	TranslateGizmo->AxisSource = Cast<UObject>(AxisSource);

	// parameter source maps axis-parameter-change to translation of TransformSource's transform
	UGizmoAxisTranslationParameterSource* ParamSource = UGizmoAxisTranslationParameterSource::Construct(AxisSource, TransformSource, this);
	ParamSource->PositionConstraintFunction = [this](const FVector& Pos, FVector& Snapped) { return PositionSnapFunction(Pos, Snapped); };
	ParamSource->AxisDeltaConstraintFunction = [this, AxisIndex](double AxisDelta, double& SnappedAxisDelta) { return PositionAxisDeltaSnapFunction(AxisDelta, SnappedAxisDelta, AxisIndex); };
	TranslateGizmo->ParameterSource = ParamSource;

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(AxisComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [AxisComponent, this](bool bHovering) { this->UpdateHoverFunction(AxisComponent, bHovering); };
	}
	TranslateGizmo->HitTarget = HitTarget;

	TranslateGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	TranslateGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
	TranslateGizmo->CustomDestinationFunc =
		[this](const UAxisPositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	ActiveGizmos.Add(TranslateGizmo);
	return TranslateGizmo;
}
UInteractiveGizmo* UCombinedTransformGizmo::AddPlaneTranslationGizmo(
	UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
	IGizmoAxisSource* AxisSource,
	IGizmoTransformSource* TransformSource,
	IGizmoStateTarget* StateTargetIn,
	int XAxisIndex, int YAxisIndex)
{
	// create axis-position gizmo, axis-position parameter will drive translation
	UPlanePositionGizmo* TranslateGizmo = Cast<UPlanePositionGizmo>(GetGizmoManager()->CreateGizmo(PlanePositionBuilderIdentifier));
	check(TranslateGizmo);

	// axis source provides the translation axis
	TranslateGizmo->AxisSource = Cast<UObject>(AxisSource);

	// parameter source maps axis-parameter-change to translation of TransformSource's transform
	UGizmoPlaneTranslationParameterSource* ParamSource = UGizmoPlaneTranslationParameterSource::Construct(AxisSource, TransformSource, this);
	ParamSource->PositionConstraintFunction = [this](const FVector& Pos, FVector& Snapped) { return PositionSnapFunction(Pos, Snapped); };
	ParamSource->AxisXDeltaConstraintFunction = [this, XAxisIndex](double AxisDelta, double& SnappedAxisDelta) { return PositionAxisDeltaSnapFunction(AxisDelta, SnappedAxisDelta, XAxisIndex); };
	ParamSource->AxisYDeltaConstraintFunction = [this, YAxisIndex](double AxisDelta, double& SnappedAxisDelta) { return PositionAxisDeltaSnapFunction(AxisDelta, SnappedAxisDelta, YAxisIndex); };
	TranslateGizmo->ParameterSource = ParamSource;

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(AxisComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [AxisComponent, this](bool bHovering) { this->UpdateHoverFunction(AxisComponent, bHovering); };
	}
	TranslateGizmo->HitTarget = HitTarget;

	TranslateGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	TranslateGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
	TranslateGizmo->CustomDestinationFunc =
		[this](const UPlanePositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	ActiveGizmos.Add(TranslateGizmo);
	return TranslateGizmo;
}
UInteractiveGizmo* UCombinedTransformGizmo::AddAxisRotationGizmo(
	UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
	IGizmoAxisSource* AxisSource,
	IGizmoTransformSource* TransformSource,
	IGizmoStateTarget* StateTargetIn)
{
	// create axis-angle gizmo, angle will drive axis-rotation
	UAxisAngleGizmo* RotateGizmo = Cast<UAxisAngleGizmo>(GetGizmoManager()->CreateGizmo(AxisAngleBuilderIdentifier));
	check(RotateGizmo);

	// axis source provides the rotation axis
	RotateGizmo->AxisSource = Cast<UObject>(AxisSource);

	// parameter source maps angle-parameter-change to rotation of TransformSource's transform
	UGizmoAxisRotationParameterSource* AngleSource = UGizmoAxisRotationParameterSource::Construct(AxisSource, TransformSource, this);
	// axis rotation is currently only relative so it should only ever snap angle-deltas
	//AngleSource->RotationConstraintFunction = [this](const FQuat& DeltaRotation){ return RotationSnapFunction(DeltaRotation); };
	AngleSource->AngleDeltaConstraintFunction = [this](double AngleDelta, double& SnappedDelta){ return RotationAxisAngleSnapFunction(AngleDelta, SnappedDelta, 0); };
	RotateGizmo->AngleSource = AngleSource;

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(AxisComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [AxisComponent, this](bool bHovering) { this->UpdateHoverFunction(AxisComponent, bHovering); };
	}
	RotateGizmo->HitTarget = HitTarget;

	RotateGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	RotateGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
	RotateGizmo->CustomDestinationFunc =
		[this](const UAxisAngleGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	ActiveGizmos.Add(RotateGizmo);

	return RotateGizmo;
}
UInteractiveGizmo* UCombinedTransformGizmo::AddAxisScaleGizmo(
	UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
	IGizmoAxisSource* GizmoAxisSource, IGizmoAxisSource* ParameterAxisSource,
	IGizmoTransformSource* TransformSource,
	IGizmoStateTarget* StateTargetIn)
{
	// create axis-position gizmo, axis-position parameter will drive scale
	UAxisPositionGizmo* ScaleGizmo = Cast<UAxisPositionGizmo>(GetGizmoManager()->CreateGizmo(AxisPositionBuilderIdentifier));
	ScaleGizmo->bEnableSignedAxis = true;
	check(ScaleGizmo);

	// axis source provides the translation axis
	ScaleGizmo->AxisSource = Cast<UObject>(GizmoAxisSource);

	// parameter source maps axis-parameter-change to translation of TransformSource's transform
	UGizmoAxisScaleParameterSource* ParamSource = UGizmoAxisScaleParameterSource::Construct(ParameterAxisSource, TransformSource, this);
	ParamSource->ScaleAxisDeltaConstraintFunction = [this](const double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta); };
	ParamSource->bClampToZero = bDisallowNegativeScaling;
	ScaleGizmo->ParameterSource = ParamSource;

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(AxisComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [AxisComponent, this](bool bHovering) { this->UpdateHoverFunction(AxisComponent, bHovering); };
	}
	ScaleGizmo->HitTarget = HitTarget;

	ScaleGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	ActiveGizmos.Add(ScaleGizmo);
	return ScaleGizmo;
}
UInteractiveGizmo* UCombinedTransformGizmo::AddPlaneScaleGizmo(
	UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
	IGizmoAxisSource* GizmoAxisSource, IGizmoAxisSource* ParameterAxisSource,
	IGizmoTransformSource* TransformSource,
	IGizmoStateTarget* StateTargetIn)
{
	// create axis-position gizmo, axis-position parameter will drive scale
	UPlanePositionGizmo* ScaleGizmo = Cast<UPlanePositionGizmo>(GetGizmoManager()->CreateGizmo(PlanePositionBuilderIdentifier));
	ScaleGizmo->bEnableSignedAxis = true;
	check(ScaleGizmo);

	// axis source provides the translation axis
	ScaleGizmo->AxisSource = Cast<UObject>(GizmoAxisSource);

	// parameter source maps axis-parameter-change to translation of TransformSource's transform
	UGizmoPlaneScaleParameterSource* ParamSource = UGizmoPlaneScaleParameterSource::Construct(ParameterAxisSource, TransformSource, this);
	ParamSource->ScaleAxisXDeltaConstraintFunction = [this](double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta); };
	ParamSource->ScaleAxisYDeltaConstraintFunction = [this](double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta); };
	ParamSource->bClampToZero = bDisallowNegativeScaling;
	ParamSource->bUseEqualScaling = true;
	ScaleGizmo->ParameterSource = ParamSource;

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(AxisComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [AxisComponent, this](bool bHovering) { this->UpdateHoverFunction(AxisComponent, bHovering); };
	}
	ScaleGizmo->HitTarget = HitTarget;

	ScaleGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	ActiveGizmos.Add(ScaleGizmo);
	return ScaleGizmo;
}
UInteractiveGizmo* UCombinedTransformGizmo::AddUniformScaleGizmo(
	UPrimitiveComponent* ScaleComponent, USceneComponent* RootComponent,
	IGizmoAxisSource* GizmoAxisSource, IGizmoAxisSource* ParameterAxisSource,
	IGizmoTransformSource* TransformSource,
	IGizmoStateTarget* StateTargetIn)
{
	// create plane-position gizmo, plane-position parameter will drive scale
	UPlanePositionGizmo* ScaleGizmo = Cast<UPlanePositionGizmo>(GetGizmoManager()->CreateGizmo(PlanePositionBuilderIdentifier));
	check(ScaleGizmo);

	// axis source provides the translation plane
	ScaleGizmo->AxisSource = Cast<UObject>(GizmoAxisSource);

	// parameter source maps axis-parameter-change to translation of TransformSource's transform
	UGizmoUniformScaleParameterSource* ParamSource = UGizmoUniformScaleParameterSource::Construct(ParameterAxisSource, TransformSource, this);
	ParamSource->ScaleAxisDeltaConstraintFunction = [this](const double ScaleAxisDelta, double& SnappedScaleAxisDelta) { return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedScaleAxisDelta); };
	ScaleGizmo->ParameterSource = ParamSource;

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(ScaleComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [ScaleComponent, this](bool bHovering) { this->UpdateHoverFunction(ScaleComponent, bHovering); };
	}
	ScaleGizmo->HitTarget = HitTarget;

	ScaleGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	ActiveGizmos.Add(ScaleGizmo);
	return ScaleGizmo;
}



void UCombinedTransformGizmo::ClearActiveTarget()
{
	OnAboutToClearActiveTarget.Broadcast(this, ActiveTarget);

	for (UInteractiveGizmo* Gizmo : ActiveGizmos)
	{
		GetGizmoManager()->DestroyGizmo(Gizmo);
	}
	ActiveGizmos.SetNum(0);
	ActiveComponents.SetNum(0);
	TranslationSubGizmos.SetNum(0);
	RotationSubGizmos.SetNum(0);
	UniformScaleSubGizmos.SetNum(0);
	NonUniformScaleSubGizmos.SetNum(0);

	CameraAxisSource = nullptr;
	AxisXSource = nullptr;
	AxisYSource = nullptr;
	AxisZSource = nullptr;
	UnitAxisXSource = nullptr;
	UnitAxisYSource = nullptr;
	UnitAxisZSource = nullptr;
	StateTarget = nullptr;

	ActiveTarget = nullptr;
	TransactionProviderAtLastSetActiveTarget = nullptr;
}




bool UCombinedTransformGizmo::PositionSnapFunction(const FVector& WorldPosition, FVector& SnappedPositionOut) const
{
	SnappedPositionOut = WorldPosition;

	// only snap world positions if we want world position snapping...
	if (bSnapToWorldGrid == false || RelativeTranslationSnapping.IsEnabled() == true)
	{
		return false;
	}

	// we can only snap positions in world coordinate system
	EToolContextCoordinateSystem CoordSystem = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	if (CoordSystem != EToolContextCoordinateSystem::World)
	{
		return false;
	}

	// need a snapping manager
	if ( USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GetGizmoManager()) )
	{
		FSceneSnapQueryRequest Request;
		Request.RequestType = ESceneSnapQueryType::Position;
		Request.TargetTypes = ESceneSnapQueryTargetType::Grid;
		if ( bGridSizeIsExplicit )
		{
			Request.GridSize = ExplicitGridSize;
		}
		TArray<FSceneSnapQueryResult> Results;
		Results.Reserve(1);

		Request.Position = WorldPosition;
		if (SnapManager->ExecuteSceneSnapQuery(Request, Results))
		{
			SnappedPositionOut = Results[0].Position;
			return true;
		};
	}

	return false;
}


bool UCombinedTransformGizmo::PositionAxisDeltaSnapFunction(double AxisDelta, double& SnappedDeltaOut, int AxisIndex) const
{
	if (CustomTranslationDeltaConstraintFunctions[AxisIndex])
	{
		return CustomTranslationDeltaConstraintFunctions[AxisIndex](AxisDelta, SnappedDeltaOut);
	}

	if (!bSnapToWorldGrid) return false;

	EToolContextCoordinateSystem CoordSystem = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	bool bUseRelativeSnapping = RelativeTranslationSnapping.IsEnabled() || (CoordSystem != EToolContextCoordinateSystem::World);
	if (!bUseRelativeSnapping)
	{
		return false;
	}

	if ( USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GetGizmoManager()) )
	{
		FSceneSnapQueryRequest Request;
		Request.RequestType = ESceneSnapQueryType::Position;
		Request.TargetTypes = ESceneSnapQueryTargetType::Grid;
		if ( bGridSizeIsExplicit )
		{
			Request.GridSize = ExplicitGridSize;
		}
		TArray<FSceneSnapQueryResult> Results;
		Results.Reserve(1);
		
		// this is a bit of a hack, since the snap query only snaps world points, and the grid may not be
		// uniform. A point on the specified X/Y/Z at the delta-distance is snapped, this is ideally
		// equivalent to actually computing a snap of the axis-delta
		Request.Position = FVector::Zero();
		Request.Position[AxisIndex] = AxisDelta;
		if (SnapManager->ExecuteSceneSnapQuery(Request, Results))
		{
			SnappedDeltaOut = Results[0].Position[AxisIndex];
			return true;
		};
	}
	return false;
}




FQuat UCombinedTransformGizmo::RotationSnapFunction(const FQuat& DeltaRotation) const
{
	// note: this is currently unused. Although we can snap to the "rotation grid", since the
	// gizmo only supports axis rotations, it doesn't make sense. Leaving in for now in case
	// a "tumble" handle is added, in which case it makes sense to snap to the world rotation grid...

	FQuat SnappedDeltaRotation = DeltaRotation;

	// only snap world positions if we want world position snapping...
	if (bSnapToWorldRotGrid == false )
	{
		return SnappedDeltaRotation;
	}

	// can only snap absolute rotations in World coordinates
	EToolContextCoordinateSystem CoordSystem = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	if (CoordSystem != EToolContextCoordinateSystem::World)
	{
		return SnappedDeltaRotation;
	}

	// need a snapping manager
	if ( USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GetGizmoManager()) )
	{
		FSceneSnapQueryRequest Request;
		Request.RequestType   = ESceneSnapQueryType::Rotation;
		Request.TargetTypes   = ESceneSnapQueryTargetType::Grid;
		Request.DeltaRotation = DeltaRotation;
		if ( bRotationGridSizeIsExplicit )
		{
			Request.RotGridSize = ExplicitRotationGridSize;
		}
		TArray<FSceneSnapQueryResult> Results;
		if (SnapManager->ExecuteSceneSnapQuery(Request, Results))
		{
			SnappedDeltaRotation = Results[0].DeltaRotation;
		};
	}

	return SnappedDeltaRotation;
}





bool UCombinedTransformGizmo::RotationAxisAngleSnapFunction(double AxisAngleDelta, double& SnappedAxisAngleDeltaOut, int AxisIndex) const
{
	if (CustomRotationDeltaConstraintFunctions[AxisIndex])
	{
		return CustomRotationDeltaConstraintFunctions[AxisIndex](AxisAngleDelta, SnappedAxisAngleDeltaOut);
	}

	if (!bSnapToWorldRotGrid) return false;

	FToolContextSnappingConfiguration SnappingConfig = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
	if ( SnappingConfig.IsRotationGridSnappingActive() )
	{
		double SnapDelta = SnappingConfig.RotationGridAngles.Yaw;		// could use AxisIndex here?
		if ( bRotationGridSizeIsExplicit )
		{
			SnapDelta = ExplicitRotationGridSize.Yaw;
		}
		AxisAngleDelta *= FMathd::RadToDeg;
		SnappedAxisAngleDeltaOut = UE::Geometry::SnapToIncrement(AxisAngleDelta, SnapDelta);
		SnappedAxisAngleDeltaOut *= FMathd::DegToRad;
		return true;
	}

	return false;
}

bool UCombinedTransformGizmo::ScaleAxisDeltaSnapFunction(const double ScaleAxisDelta, double& SnappedAxisScaleDeltaOut, int AxisIndex) const
{
	if (CustomScaleDeltaConstraintFunctions[AxisIndex])
	{
		return CustomScaleDeltaConstraintFunctions[AxisIndex](ScaleAxisDelta, SnappedAxisScaleDeltaOut);
	}

	return ScaleAxisDeltaSnapFunction(ScaleAxisDelta, SnappedAxisScaleDeltaOut);
}

bool UCombinedTransformGizmo::ScaleAxisDeltaSnapFunction(const double ScaleAxisDelta, double & SnappedAxisScaleDeltaOut) const
{
	if (!bSnapToScaleGrid) return false;

	const FToolContextSnappingConfiguration SnappingConfig = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
	if ( SnappingConfig.IsScaleGridSnappingActive() )
	{
		const double SnapDelta = SnappingConfig.ScaleGridSize;
		SnappedAxisScaleDeltaOut = UE::Geometry::SnapToIncrement(ScaleAxisDelta, SnapDelta);
		return true;
	}
	return false;
}


#undef LOCTEXT_NAMESPACE

