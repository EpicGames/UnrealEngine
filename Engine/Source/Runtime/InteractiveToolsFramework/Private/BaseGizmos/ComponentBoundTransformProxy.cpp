// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/ComponentBoundTransformProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentBoundTransformProxy)

void UComponentBoundTransformProxy::BindToComponent(USceneComponent* Component, bool bUseOwnScaleIn)
{
	if (Component)
	{
		BoundComponent = Component;
		TGuardValue<bool> SetPivot(bSetPivotMode, true);
		SetTransform(Component->GetComponentToWorld());
	}
	bStoreScaleSeparately = bUseOwnScaleIn;
}

FTransform UComponentBoundTransformProxy::GetTransform() const
{
	if (BoundComponent.IsValid())
	{
		FTransform TransformToUse = BoundComponent->GetComponentToWorld();
		if (bStoreScaleSeparately)
		{
			TransformToUse.SetScale3D(SharedTransform.GetScale3D());
		}
		return TransformToUse;
	}
	return Super::GetTransform();
}

INTERACTIVETOOLSFRAMEWORK_API void UComponentBoundTransformProxy::SetTransform(const FTransform& Transform)
{
	if (BoundComponent.IsValid())
	{
		FTransform TransformToUse = Transform;
		if (bStoreScaleSeparately)
		{
			TransformToUse.SetScale3D(BoundComponent->GetComponentScale());
		}
		BoundComponent->SetWorldTransform(TransformToUse);
	}
	Super::SetTransform(Transform);
}

INTERACTIVETOOLSFRAMEWORK_API void UComponentBoundTransformProxy::UpdateSharedTransform()
{
	if (!BoundComponent.IsValid())
	{
		Super::UpdateSharedTransform();
		return;
	}

	// Our shared transform is always the bound component transform.
	SharedTransform = BoundComponent->GetComponentTransform();
	for (FRelativeObject& Obj : Objects)
	{
		// The below seems like it should be writeable as Obj.RelativeTransform = Obj.StartTransform.GetRelativeTransform(SharedTransform),
		//  but we write it like this for consistency with TransformProxy.cpp.
		// TODO: it seems that GetRelativeTransform goes down a different code path in the presence of negative scale (possibly
		//  unintentionally?). Perhaps we should use that function in both places?
		Obj.RelativeTransform = Obj.StartTransform;
		Obj.RelativeTransform.SetToRelativeTransform(SharedTransform);
	}
	InitialSharedTransform = SharedTransform;
}
