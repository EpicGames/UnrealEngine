// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/FreePositionSubGizmo.h"

#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // SetCommonSubGizmoProperties, UpdateCameraAxisSource
#include "BaseGizmos/GizmoInterfaces.h" // IGizmoTransformSource
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FreePositionSubGizmo)

bool UFreePositionSubGizmo::InitializeAsScreenPlaneTranslateGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy)
	{
		return false;
	}

	// The below setup is a merging of InitializeAsUniformScaleGizmo (to set up the camera
	//  axis source) and InitializeAsTranslateGizmo to set up the translation parameter
	//  source. But we also store the transform source so that we can use it directly when
	//  using a custom destination function.

	UGizmoScaledAndUnscaledTransformSources* CastTransformSource;

	// Make sure axis index is invalid so that the SetCommonSubGizmoProperties call below doesn't create
	// an axis source for us.
	if (!ensureMsgf(Params.Axis == EAxis::None, TEXT("UFreePositionSubGizmo uses a camera axis source, so axis parameter should be None.")))
	{
		UE::GizmoUtil::FTransformSubGizmoCommonParams ParamsCopy = Params;
		ParamsCopy.Axis = EAxis::None;
		if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, ParamsCopy, SharedState, CastTransformSource))
		{
			return false;
		}
	}
	else if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, CastTransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();

	UGizmoConstantFrameAxisSource* CameraAxisSource = nullptr;

	// See if we already have the axis source in our shared state
	if (SharedState && SharedState->CameraAxisSource)
	{
		CameraAxisSource = SharedState->CameraAxisSource;
	}
	else
	{
		// Create new and add to shared state.
		CameraAxisSource = NewObject<UGizmoConstantFrameAxisSource>(Owner);
		TWeakObjectPtr<AActor> OwnerActor = Params.Component->GetOwner();
		CustomTickFunction = [this, OwnerActor](float DeltaTime)
		{
			if (UGizmoConstantFrameAxisSource* FrameAxisSource = Cast<UGizmoConstantFrameAxisSource>(AxisSource.GetObject()))
			{
				UE::GizmoUtil::UpdateCameraAxisSource(
					*FrameAxisSource,
					GetGizmoManager(),
					OwnerActor.IsValid() ? OwnerActor->GetTransform().GetLocation() : FVector3d::ZeroVector);
			}
		};

		if (SharedState)
		{
			SharedState->CameraAxisSource = CameraAxisSource;
		}
	}
	AxisSource = CameraAxisSource;

	// Parameter source maps axis-parameter-change to translation of TransformSource's transform
	ParameterSource = UGizmoPlaneTranslationParameterSource::Construct(
		AxisSource.GetInterface(), CastTransformSource, Owner);

	TransformSource = CastTransformSource;

	return true;
}

void UFreePositionSubGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!bInInteraction)
	{
		return;
	}

	// If we have a custom destination function, we use it to set our transform
	FCustomDestinationParams Params;
	Params.WorldRay = &DragPos.WorldRay;
	FVector HitPoint;
	if (ShouldUseCustomDestinationFunc() && CustomDestinationFunc(Params, HitPoint))
	{
		FTransform CurrentTransform = TransformSource->GetTransform();
		CurrentTransform.SetLocation(HitPoint);
		TransformSource->SetTransform(CurrentTransform);
		return;
	}
	// Otherwise, we do regular translation in the plane (we do end up doing a custom
	//  destination check a second time in there, but it's not worth factoring out).
	Super::OnClickDrag(DragPos);
}
