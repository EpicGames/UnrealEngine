// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/PlanePositionGizmo.h"

#include "FreePositionSubGizmo.generated.h"

class IGizmoTransformSource;

/**
 * UFreePositionSubGizmo is very similar to UPlanePositionGizmo with a camera axis source, 
 *  but when using a custom destination function, it can use the destination to directly
 *  set a transform source.
 */
UCLASS(MinimalAPI)
class UFreePositionSubGizmo : public UPlanePositionGizmo
{
	GENERATED_BODY()

public:
	bool InitializeAsScreenPlaneTranslateGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);

	// IClickDragBehaviorTarget
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;

public:
	// The below properties can be manipulated for more fine-grained control, but typically it is sufficient
	// to use one of the initialization methods above.

	/** AxisSource provides the 3D plane on which the interaction happens */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;
};

