// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/FreeRotationSubGizmo.h"

#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/AxisSources.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // SetCommonSubGizmoProperties
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/HitTargets.h"
#include "BaseGizmos/StateTargets.h"
#include "BaseGizmos/TransformSources.h"
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams
#include "ToolDataVisualizer.h"
#include "Util/ColorConstants.h" // UE::Geometry::LinearColors

#include UE_INLINE_GENERATED_CPP_BY_NAME(FreeRotationSubGizmo)

namespace FreeRotationSubGizmoLocals
{
	FLinearColor CircleColor = UE::Geometry::LinearColors::Gray3f();
	int32 CircleNumSections = 32;
	float CircleThickness = 2.0f;
}

void UFreeRotationSubGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	// Add default mouse input behavior
	MouseBehavior = NewObject<UClickDragInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(MouseBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(HoverBehavior);

	AxisSource = NewObject<UGizmoConstantAxisSource>(this);
	HitTarget = NewObject<UGizmoComponentHitTarget>(this);
	StateTarget = NewObject<UGizmoNilStateTarget>(this);

	bInInteraction = false;
}



bool UFreeRotationSubGizmo::InitializeAsRotationGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	UGizmoViewContext* GizmoViewContextIn,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy)
	{
		return false;
	}

	UGizmoScaledAndUnscaledTransformSources* TransformSourcePtr;

	// Make sure axis index is invalid so that the SetCommonSubGizmoProperties call below doesn't create
	// an axis source for us.
	if (!ensureMsgf(Params.Axis == EAxis::None, TEXT("UFreeRotationSubGizmo uses a camera axis source, so axis parameter should be None.")))
	{
		UE::GizmoUtil::FTransformSubGizmoCommonParams ParamsCopy = Params;
		ParamsCopy.Axis = EAxis::None;
		if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, ParamsCopy, SharedState, TransformSourcePtr))
		{
			return false;
		}
	}
	else if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSourcePtr))
	{
		return false;
	}
	TransformSource = TransformSourcePtr;

	// Set up the camera axis source
	UGizmoConstantFrameAxisSource* CameraAxisSource = nullptr;

	// See if we already have it in our shared state
	if (SharedState && SharedState->CameraAxisSource)
	{
		CameraAxisSource = SharedState->CameraAxisSource;
	}
	else
	{
		// Create new and add to shared state.
		UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();
		CameraAxisSource = NewObject<UGizmoConstantFrameAxisSource>(Owner);
		TWeakObjectPtr<AActor> OwnerActor = Params.Component->GetOwner();
		CustomTickFunction = [this, OwnerActor](float DeltaTime)
		{
			if (UGizmoConstantFrameAxisSource* FrameAxisSource = Cast<UGizmoConstantFrameAxisSource>(AxisSource.GetObject()))
			{
				UE::GizmoUtil::UpdateCameraAxisSource(
					*FrameAxisSource,
					GetGizmoManager(),
					OwnerActor.IsValid() ? OwnerActor->GetTransform().GetLocation() : FVector3d::ZeroVector);
			}
		};

		if (SharedState)
		{
			SharedState->CameraAxisSource = CameraAxisSource;
		}
	}
	AxisSource = CameraAxisSource;

	GizmoViewContext = GizmoViewContextIn;

	return true;
}

// Called from OnClickPress if mode is IncrementalSphereBound
bool UFreeRotationSubGizmo::ClickPress_IncrementalSphereBound(const FInputDeviceRay& PressPos)
{
	if (UnscaledSphereRadius == 0)
	{
		return false;
	}

	FVector SphereCenter = AxisSource->GetOrigin();
	double LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, SphereCenter);
	InteractionSphereRadius = UnscaledSphereRadius * LengthScale;

	bool bIntersects;
	GizmoMath::RaySphereIntersection(AxisSource->GetOrigin(),
		InteractionSphereRadius, PressPos.WorldRay.Origin, PressPos.WorldRay.Direction,
		bIntersects, LastSphereIntersectionPoint);

	return bIntersects;
}

// Called from OnClickDrag if mode is IncrementalSphereBound
bool UFreeRotationSubGizmo::ClickDrag_IncrementalSphereBound(const FInputDeviceRay& DragPos)
{
	if (InteractionSphereRadius == 0)
	{
		return false;
	}
	FVector SphereCenter = AxisSource->GetOrigin();

	bool bIntersects = false;
	FVector CurrentSphereIntersectionPoint;
	GizmoMath::RaySphereIntersection(SphereCenter,
		InteractionSphereRadius, DragPos.WorldRay.Origin, DragPos.WorldRay.Direction,
		bIntersects, CurrentSphereIntersectionPoint);

	if (!bIntersects)
	{
		// If we didn't hit the sphere itself, raycast the plane
		FVector PlaneHitPoint;
		GizmoMath::RayPlaneIntersectionPoint(
			AxisSource->GetOrigin(), AxisSource->GetDirection(),
			DragPos.WorldRay.Origin, DragPos.WorldRay.Direction,
			bIntersects, PlaneHitPoint);

		if (!bIntersects)
		{
			return false;
		}

		// Find the point on the sphere that is closest to the hit plane point
		FVector TowardSphereVector = SphereCenter - PlaneHitPoint;
		TowardSphereVector.Normalize();
		GizmoMath::RaySphereIntersection(SphereCenter,
			InteractionSphereRadius, PlaneHitPoint, TowardSphereVector,
			bIntersects, CurrentSphereIntersectionPoint);
	}

	if (!ensure(bIntersects))
	{
		return false;
	}

	// Find the angle that we've rotated the sphere from LastSphereIntersectionPoint to current
	FVector ToLastSpherePoint = (LastSphereIntersectionPoint - SphereCenter) / InteractionSphereRadius;
	FVector ToCurrentSpherePoint = (CurrentSphereIntersectionPoint - SphereCenter) / InteractionSphereRadius;
	FVector RotationAxis = ToLastSpherePoint.Cross(ToCurrentSpherePoint);
	if (!RotationAxis.Normalize())
	{
		return false;
	}
	double RotationAngle = UE::Geometry::AngleR(ToLastSpherePoint, ToCurrentSpherePoint);
	
	// Apply the new transform to the transform source
	FTransform CurrentTransform = TransformSource->GetTransform();
	FQuat RotationToApply(RotationAxis, RotationAngle);
	CurrentTransform.SetRotation(RotationToApply * CurrentTransform.GetRotation());
	TransformSource->SetTransform(CurrentTransform);
	
	LastSphereIntersectionPoint = CurrentSphereIntersectionPoint;

	return true;
}

void UFreeRotationSubGizmo::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace FreeRotationSubGizmoLocals;

	if (bShowSphereBoundsDuringInteraction && bInInteraction)
	{
		FVector3d SphereCenter = AxisSource->GetOrigin();
		double LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(RenderAPI->GetSceneView(), SphereCenter);
		double SphereRadius = UnscaledSphereRadius * LengthScale;

		FToolDataVisualizer Drawer;
		Drawer.BeginFrame(RenderAPI, RenderAPI->GetCameraState());
		Drawer.DrawViewFacingCircle(SphereCenter, SphereRadius, CircleNumSections,
			CircleColor, CircleThickness,/*bDepthTested*/ false);
		Drawer.EndFrame();
	}
}

void UFreeRotationSubGizmo::Tick(float DeltaTime)
{
	if (CustomTickFunction)
	{
		CustomTickFunction(DeltaTime);
	}
}


FInputRayHit UFreeRotationSubGizmo::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget && AxisSource && TransformSource)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
	}
	return GizmoHit;
}

void UFreeRotationSubGizmo::OnClickPress(const FInputDeviceRay& PressPos)
{
	bInInteraction = ClickPress_IncrementalSphereBound(PressPos);
	if (!bInInteraction)
	{
		return;
	}

	if (StateTarget)
	{
		StateTarget->BeginUpdate();
	}
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}

void UFreeRotationSubGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!bInInteraction)
	{
		return;
	}

	ClickDrag_IncrementalSphereBound(DragPos);
}

void UFreeRotationSubGizmo::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnTerminateDragSequence();
}


void UFreeRotationSubGizmo::OnTerminateDragSequence()
{
	if (!bInInteraction)
	{
		return;
	}

	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}

FInputRayHit UFreeRotationSubGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
	}
	return GizmoHit;
}

void UFreeRotationSubGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	HitTarget->UpdateHoverState(true);
}

bool UFreeRotationSubGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// not necessary...
	HitTarget->UpdateHoverState(true);
	return true;
}

void UFreeRotationSubGizmo::OnEndHover()
{
	HitTarget->UpdateHoverState(false);
}
