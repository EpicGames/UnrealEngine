// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InteractiveGizmo.h"
#include "Math/MathFwd.h"

#include "FreeRotationSubGizmo.generated.h"

class IGizmoAxisSource;
class IGizmoClickTarget;
class IGizmoStateTarget;
class IGizmoTransformSource;
class UClickDragInputBehavior;
class UGizmoViewContext;
namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}

/**
 * A free rotation sub gizmo implements an arcball-like rotation.
 */
UCLASS(MinimalAPI)
class UFreeRotationSubGizmo : public UInteractiveGizmo, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * Initializes the properties for the gizmo
	 */
	bool InitializeAsRotationGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& InitializationParams,
		UGizmoViewContext* ViewContext,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);
	
	// There are many potential approaches to doing a free rotate handle. Currently we
	//  implement IncrementalSphereBound from the below, which feels pretty intuitive,
	//  but we could add the other ones if desired.
	//enum class EMode
	//{
	//	// Intersect rays with a sphere and rotate the latest result such that the previous 
	//	//  intersection goes to the latest intersection. Outside of sphere, assume you are
	//	//  on the tangent side of the sphere.
	//	// This is similar to how the normal editor gizmo works if you enable arcball rotation.
	//	IncrementalSphereBound,
	//	
	//	// Like IncrementalSphereBound, but rotate the original (not latest) transform such
	//	//  that the first (i.e. original) intersection goes to latest intersection.
	//	// This is similar to how the ball works in the new gizmos (i.e. if Enable New Gizmos) is true.
	//	SourceToDestSphereBound,

	//	// Similar to SourceToDestSphereBound in that we're determining our rotation axis based on
	//	//  the first intersection going to latest intersection, but the amount to rotate is determined
	//	//  by the distance in the camera plane, so that we can keep rotating the object 360 degrees or
	//	//  more with one drag.
	//	SourceToDestUnbounded
	//};
	//void SetMode(EMode::ModeIn);
	
	/**
	 * Determines the size of the invisible sphere we raycast to perform the rotation.
	 */ 
	void SetUnscaledSphereRadius(double Radius) { UnscaledSphereRadius = Radius; }

	/**
	 * When true (default) a circle is drawn to show the outside bounds of the interaction sphere while
	 *  interacting with the gizmo.
	 */
	void SetShowSphereBoundsDuringInteraction(bool bOn) { bShowSphereBoundsDuringInteraction = bOn; }

	// UInteractiveGizmo overrides
	virtual void Setup() override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void Tick(float DeltaTime) override;

	// IClickDragBehaviorTarget implementation
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;

	// IHoverBehaviorTarget implementation
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override;

public:
	// The below properties can be manipulated for more fine-grained control, but typically it is sufficient
	// to use the initialization method above.

	/** AxisSource provides the origin of the interaction sphere and the plane to raycast when hitting outside the sphere */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;
	
	/** The HitTarget provides a hit-test against some 3D element (presumably a visual widget) that controls when interaction can start */
	UPROPERTY()
	TScriptInterface<IGizmoClickTarget> HitTarget;

	/** StateTarget is notified when interaction starts and ends, so that things like undo/redo can be handled externally */
	UPROPERTY()
	TScriptInterface<IGizmoStateTarget> StateTarget;

	/** Target that is rotated by the sub gizmo. */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;

	/** View info used during raycasts */
	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext = nullptr;

	/** The mouse click behavior of the gizmo is accessible so that it can be modified to use different mouse keys. */
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> MouseBehavior;
	
private:
	bool bInInteraction = false;
	bool bShowSphereBoundsDuringInteraction = true;

	FVector LastSphereIntersectionPoint;
	double InteractionSphereRadius = 0;
	bool ClickPress_IncrementalSphereBound(const FInputDeviceRay& PressPos);
	bool ClickDrag_IncrementalSphereBound(const FInputDeviceRay& DragPos);
	
	double UnscaledSphereRadius = 100;

	// Helper that can hold some extra upkeep to do during Tick (used to
	//  update a camera axis source if needed).
	TUniqueFunction<void(float DeltaTime)> CustomTickFunction;
};

