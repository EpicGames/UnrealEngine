// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoActor.h"

#include "BaseGizmos/GizmoArrowComponent.h"
#include "BaseGizmos/GizmoRectangleComponent.h"
#include "BaseGizmos/GizmoCircleComponent.h"
#include "BaseGizmos/GizmoBoxComponent.h"
#include "BaseGizmos/GizmoLineHandleComponent.h"

#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoActor)


#define LOCTEXT_NAMESPACE "AGizmoActor"


AGizmoActor::AGizmoActor()
{
	// generally gizmo actor creation/destruction should not be transacted
	ClearFlags(RF_Transactional);

#if WITH_EDITORONLY_DATA
	// hide this actor in the scene outliner
	bListedInSceneOutliner = false;
#endif
}


UGizmoArrowComponent* AGizmoActor::AddDefaultArrowComponent(
	UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
	const FLinearColor& Color, const FVector& LocalDirection, const float Length)
{
	ensure(GizmoViewContext);

	UGizmoArrowComponent* NewArrow = NewObject<UGizmoArrowComponent>(Actor);
	Actor->AddInstanceComponent(NewArrow);
	NewArrow->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	NewArrow->SetGizmoViewContext(GizmoViewContext);
	NewArrow->Direction = LocalDirection;
	NewArrow->Color = Color;
	NewArrow->Length = Length;
	NewArrow->RegisterComponent();
	return NewArrow;
}



UGizmoRectangleComponent* AGizmoActor::AddDefaultRectangleComponent(
	UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
	const FLinearColor& Color, const FVector& PlaneAxis1, const FVector& PlaneAxisx2)
{
	ensure(GizmoViewContext);

	UGizmoRectangleComponent* NewRectangle = NewObject<UGizmoRectangleComponent>(Actor);
	Actor->AddInstanceComponent(NewRectangle);
	NewRectangle->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	NewRectangle->SetGizmoViewContext(GizmoViewContext);
	NewRectangle->DirectionX = PlaneAxis1;
	NewRectangle->DirectionY = PlaneAxisx2;
	NewRectangle->Color = Color;
	NewRectangle->LengthX = NewRectangle->LengthY = 30.0f;
	NewRectangle->SegmentFlags = 0x2 | 0x4;
	NewRectangle->RegisterComponent();
	return NewRectangle;
}


UGizmoCircleComponent* AGizmoActor::AddDefaultCircleComponent(
	UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
	const FLinearColor& Color, const FVector& PlaneNormal, float Radius)
{
	ensure(GizmoViewContext);

	UGizmoCircleComponent* NewCircle = NewObject<UGizmoCircleComponent>(Actor);
	Actor->AddInstanceComponent(NewCircle);
	NewCircle->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	NewCircle->SetGizmoViewContext(GizmoViewContext);
	NewCircle->Normal = PlaneNormal;
	NewCircle->Color = Color;
	NewCircle->Radius = Radius;
	NewCircle->RegisterComponent();
	return NewCircle;
}



UGizmoBoxComponent* AGizmoActor::AddDefaultBoxComponent(
	UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
	const FLinearColor& Color, const FVector& Origin,
	const FVector& Dimensions
)
{
	ensure(GizmoViewContext);

	UGizmoBoxComponent* NewBox = NewObject<UGizmoBoxComponent>(Actor);
	Actor->AddInstanceComponent(NewBox);
	NewBox->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	NewBox->SetGizmoViewContext(GizmoViewContext);
	NewBox->Origin = Origin;
	NewBox->Color = Color;
	NewBox->Dimensions = Dimensions;
	NewBox->RegisterComponent();
	return NewBox;
}


UGizmoLineHandleComponent* AGizmoActor::AddDefaultLineHandleComponent(
	UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
	const FLinearColor& Color, const FVector& HandleNormal, const FVector& LocalDirection,
    const float Length, const bool bImageScale)
{
	ensure(GizmoViewContext);

	UGizmoLineHandleComponent* LineHandle = NewObject<UGizmoLineHandleComponent>(Actor);
	Actor->AddInstanceComponent(LineHandle);
	LineHandle->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	LineHandle->SetGizmoViewContext(GizmoViewContext);
	LineHandle->Normal = HandleNormal;
	LineHandle->Direction = LocalDirection;
	LineHandle->Length = Length;
	LineHandle->bImageScale = bImageScale;
	LineHandle->Color = Color;
	LineHandle->RegisterComponent();
	return LineHandle;
}

#undef LOCTEXT_NAMESPACE
