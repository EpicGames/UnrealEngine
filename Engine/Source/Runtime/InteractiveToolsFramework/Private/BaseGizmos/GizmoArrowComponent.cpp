// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoArrowComponent.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoArrowComponent)

namespace GizmoArrowComponentLocals
{
	const float ARROW_RENDERVISIBILITY_DOT_THRESHOLD = 0.965f; //~15 degrees

	template <typename SceneViewOrGizmoViewContext>
	bool GetWorldEndpoints(bool bIsViewDependent, const SceneViewOrGizmoViewContext* View,
		const FVector& WorldOrigin, const FVector& Direction, float Gap, 
		float Length, bool bUseWorldAxes,
		TFunctionRef<FVector(const FVector&)> VectorTransform, 
		FVector& StartPointOut, FVector& EndPointOut, float& PixelToWorldScaleOut)
	{
		FVector ArrowDirection = bUseWorldAxes ? Direction : VectorTransform(Direction);
		float LengthScale = 1;

		if (bIsViewDependent)
		{
			// direction to origin of gizmo
			FVector ViewDirection = View->IsPerspectiveProjection() ?
				WorldOrigin - View->ViewLocation : View->GetViewDirection();
			ViewDirection.Normalize();

			bool bFlipped = (FVector::DotProduct(ViewDirection, ArrowDirection) > 0);
			ArrowDirection = (bFlipped) ? -ArrowDirection : ArrowDirection;

			//bRenderVisibilityOut = FMath::Abs(FVector::DotProduct(ArrowDirection, ViewDirection)) < 0.985f;   // ~10 degrees
			bool bRenderVisibility =
				FMath::Abs(FVector::DotProduct(ArrowDirection, ViewDirection)) < ARROW_RENDERVISIBILITY_DOT_THRESHOLD;   // ~15 degrees

			if (!bRenderVisibility)
			{
				return false;
			}

			LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, WorldOrigin);
		}

		PixelToWorldScaleOut = LengthScale;

		double StartDist = LengthScale * Gap;
		double EndDist = LengthScale * (Gap + Length);

		StartPointOut = WorldOrigin + StartDist * ArrowDirection;
		EndPointOut = WorldOrigin + EndDist * ArrowDirection;

		return true;
	}
}

class FGizmoArrowComponentSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FGizmoArrowComponentSceneProxy(const UGizmoArrowComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent),
		Color(InComponent->Color),
		Direction(InComponent->Direction),
		Gap(InComponent->Gap),
		Length(InComponent->Length),
		Thickness(InComponent->Thickness),
		HoverThicknessMultiplier(InComponent->HoverSizeMultiplier)
	{
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		using namespace GizmoArrowComponentLocals;

		const FMatrix& LocalToWorldMatrix = GetLocalToWorld();
		FVector Origin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

				bool bWorldAxis = (bExternalWorldLocalState) ? (*bExternalWorldLocalState) : false;
				FVector StartPoint, EndPoint;
				float PixelToWorldScale = 0;

				bool bIsViewDependent = (bExternalIsViewDependent) ? (*bExternalIsViewDependent) : false;
				bool bRenderVisibility = GetWorldEndpoints(bIsViewDependent, View, Origin, 
					Direction, Gap, Length, bWorldAxis,
					[&LocalToWorldMatrix](const FVector& VectorIn) {
						return FVector{ LocalToWorldMatrix.TransformVector(VectorIn) };
					}, StartPoint, EndPoint, PixelToWorldScale);

				if (!bRenderVisibility)
				{
					continue;
				}

				float UseThickness = (bExternalHoverState != nullptr && *bExternalHoverState == true) ?
					(HoverThicknessMultiplier * Thickness) : (Thickness);
				if (View->IsPerspectiveProjection())
				{
					UseThickness *= (View->FOV / 90.0f);		// compensate for FOV scaling in Gizmos...
				}

				PDI->DrawLine(StartPoint, EndPoint, Color, SDPG_Foreground, UseThickness, 0.0f, true);
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return false;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return IntCastChecked<uint32>(sizeof *this + GetAllocatedSize()); }
	SIZE_T GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	void SetExternalHoverState(bool* HoverState)
	{
		bExternalHoverState = HoverState;
	}

	void SetExternalWorldLocalState(bool* bWorldLocalState)
	{
		bExternalWorldLocalState = bWorldLocalState;
	}

	void SetExternalIsViewDependent(bool* bExternalIsViewDependentIn)
	{
		bExternalIsViewDependent = bExternalIsViewDependentIn;
	}

private:
	FLinearColor Color;
	FVector Direction;
	float Gap;
	float Length;
	float Thickness;
	float HoverThicknessMultiplier;

	// set on Component for use in ::GetDynamicMeshElements()
	bool* bExternalHoverState = nullptr;
	bool* bExternalWorldLocalState = nullptr;
	bool* bExternalIsViewDependent = nullptr;
};


FPrimitiveSceneProxy* UGizmoArrowComponent::CreateSceneProxy()
{
	FGizmoArrowComponentSceneProxy* NewProxy = new FGizmoArrowComponentSceneProxy(this);
	NewProxy->SetExternalHoverState(&bHovering);
	NewProxy->SetExternalWorldLocalState(&bWorld);
	NewProxy->SetExternalIsViewDependent(&bIsViewDependent);
	return NewProxy;
}

FBoxSphereBounds UGizmoArrowComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FSphere(FVector::ZeroVector, Gap+Length).TransformBy(LocalToWorld));
}

bool UGizmoArrowComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	using namespace GizmoArrowComponentLocals;

	const FTransform& Transform = this->GetComponentToWorld();
	FVector UseOrigin = Transform.TransformPosition(FVector::ZeroVector);

	// Copy what is done in the proxy object, but get data from the gizmo view context.
	FVector StartPoint = FVector::ZeroVector, EndPoint = FVector::ZeroVector; // initialized to appease CIS
	float PixelToWorldScale = 0;
	bool bRenderVisibility = GetWorldEndpoints(bIsViewDependent, ToRawPtr(GizmoViewContext), UseOrigin, Direction, Gap, Length, bWorld,
		[&Transform](const FVector& VectorIn) { return Transform.TransformVector(VectorIn); },
		StartPoint, EndPoint, PixelToWorldScale);

	if (!bRenderVisibility)
	{
		return false;
	}

	FVector NearestArrow, NearestLine;
	FMath::SegmentDistToSegmentSafe(StartPoint, EndPoint, Start, End, NearestArrow, NearestLine);
	double Distance = FVector::Distance(NearestArrow, NearestLine);
	if (Distance > PixelHitDistanceThreshold * PixelToWorldScale)
	{
		return false;
	}

	OutHit.Component = this;
	OutHit.Distance = static_cast<float>(FVector::Distance(Start, NearestLine));
	OutHit.ImpactPoint = NearestLine;
	return true;
}
