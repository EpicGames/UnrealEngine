// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoBoxComponent.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoBoxComponent)

namespace GizmoBoxComponentLocals
{

	template <typename SceneViewOrGizmoViewContext>
	void GetBoxDirections(bool bIsViewDependent, const SceneViewOrGizmoViewContext* View, const FVector& WorldOrigin,
		const FVector& DirectionX, const FVector& DirectionY, const FVector& DirectionZ, 
		bool bEnableFlipping, bool bWorldAxis,
		TFunctionRef<FVector(const FVector&)> VectorTransform,
		FVector& UseDirectionXOut, FVector& UseDirectionYOut, FVector& UseDirectionZOut,
		bool& bFlippedXOut, bool& bFlippedYOut, bool& bFlippedZOut
		)
	{

		UseDirectionXOut = (bWorldAxis) ? DirectionX : VectorTransform(DirectionX);
		UseDirectionYOut = (bWorldAxis) ? DirectionY : VectorTransform(DirectionY);
		UseDirectionZOut = (bWorldAxis) ? DirectionZ : VectorTransform(DirectionZ);
		bFlippedXOut = false;
		bFlippedYOut = false;
		bFlippedZOut = false;

		if (bIsViewDependent)
		{
			// direction to origin of gizmo
			FVector ViewDirection =
				View->IsPerspectiveProjection() ? WorldOrigin - View->ViewLocation : View->GetViewDirection();
			ViewDirection.Normalize();

			bFlippedXOut = (FVector::DotProduct(ViewDirection, UseDirectionXOut) > 0);
			UseDirectionXOut = (bEnableFlipping && bFlippedXOut) ? -UseDirectionXOut : UseDirectionXOut;

			bFlippedYOut = (FVector::DotProduct(ViewDirection, UseDirectionYOut) > 0);
			UseDirectionYOut = (bEnableFlipping && bFlippedYOut) ? -UseDirectionYOut : UseDirectionYOut;

			bFlippedZOut = (FVector::DotProduct(ViewDirection, UseDirectionZOut) > 0);
			UseDirectionZOut = (bEnableFlipping && bFlippedZOut) ? -UseDirectionZOut : UseDirectionZOut;
		}
	}

}


class FGizmoBoxComponentSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FGizmoBoxComponentSceneProxy(const UGizmoBoxComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent),
		Color(InComponent->Color),
		LocalCenter(InComponent->Origin),
		DirectionX(InComponent->Rotation*FVector(1,0,0)),
		DirectionY(InComponent->Rotation*FVector(0,1,0)),
		DirectionZ(InComponent->Rotation*FVector(0,0,1)),
		Dimensions(InComponent->Dimensions),
		Thickness(InComponent->LineThickness),
		HoverThicknessMultiplier(InComponent->HoverSizeMultiplier),
		bEnableFlipping(InComponent->bEnableAxisFlip),
		bRemoveHiddenLines(InComponent->bRemoveHiddenLines)
	{
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		using namespace GizmoBoxComponentLocals;

		const FMatrix& LocalToWorldMatrix = GetLocalToWorld();
		FVector WorldOrigin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);

		FVector Points[8];   // box corners, order is 000, 100, 110, 010,  001, 101, 111, 011
		static const int Lines[12][2] = {
			{0,1}, {1,2}, {2,3}, {3,0},
			{4,5}, {5,6}, {6,7}, {7,4},
			{0,4}, {1,5}, {2,6}, {3,7}
		};

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
				bool bIsOrtho = !View->IsPerspectiveProjection();

				bool bWorldAxis = (bExternalWorldLocalState) ? (*bExternalWorldLocalState) : false;

				FVector UseDirectionX, UseDirectionY, UseDirectionZ;
				bool bFlippedX = false;
				bool bFlippedY = false;
				bool bFlippedZ = false;

				bool bIsViewDependent = (bExternalIsViewDependent) ? (*bExternalIsViewDependent) : false;
				GetBoxDirections(bIsViewDependent, View, WorldOrigin, DirectionX, DirectionY, DirectionZ, bEnableFlipping,
					bWorldAxis, [&LocalToWorldMatrix](const FVector& VectorIn) {
						return FVector{ LocalToWorldMatrix.TransformVector(VectorIn) };
					}, UseDirectionX, UseDirectionY, UseDirectionZ, bFlippedX, bFlippedY, bFlippedZ);

				FVector UseCenter(
					(bFlippedX) ? -LocalCenter.X : LocalCenter.X,
					(bFlippedY) ? -LocalCenter.Y : LocalCenter.Y,
					(bFlippedZ) ? -LocalCenter.Z : LocalCenter.Z
				);

				float LengthScale = 1;
				if (bIsViewDependent)
				{
					LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, WorldOrigin);
				}

				FVector WorldCenter = WorldOrigin
					+ LengthScale * LocalCenter.X * UseDirectionX
					+ LengthScale * LocalCenter.Y * UseDirectionY
					+ LengthScale * LocalCenter.Z * UseDirectionZ;
				//= LocalToWorldMatrix.TransformPosition(LengthScale*UseCenter);

				float UseThickness = (bExternalHoverState != nullptr && *bExternalHoverState == true) ?
					(HoverThicknessMultiplier * Thickness) : (Thickness);
				if (!bIsOrtho)
				{
					UseThickness *= (View->FOV / 90.0f);		// compensate for FOV scaling in Gizmos...
				}

				double DimensionX = LengthScale * Dimensions.X * 0.5f;
				double DimensionY = LengthScale * Dimensions.Y * 0.5f;
				double DimensionZ = LengthScale * Dimensions.Z * 0.5f;

				Points[0] = - DimensionX*UseDirectionX - DimensionY*UseDirectionY - DimensionZ*UseDirectionZ;
				Points[1] = + DimensionX*UseDirectionX - DimensionY*UseDirectionY - DimensionZ*UseDirectionZ;
				Points[2] = + DimensionX*UseDirectionX + DimensionY*UseDirectionY - DimensionZ*UseDirectionZ;
				Points[3] = - DimensionX*UseDirectionX + DimensionY*UseDirectionY - DimensionZ*UseDirectionZ;

				Points[4] = - DimensionX*UseDirectionX - DimensionY*UseDirectionY + DimensionZ*UseDirectionZ;
				Points[5] = + DimensionX*UseDirectionX - DimensionY*UseDirectionY + DimensionZ*UseDirectionZ;
				Points[6] = + DimensionX*UseDirectionX + DimensionY*UseDirectionY + DimensionZ*UseDirectionZ;
				Points[7] = - DimensionX*UseDirectionX + DimensionY*UseDirectionY + DimensionZ*UseDirectionZ;

				if (bRemoveHiddenLines)
				{
					FVector ViewDirection =
						View->IsPerspectiveProjection() ? WorldOrigin - View->ViewLocation : View->GetViewDirection();

					// find box corner direction that is most aligned with view direction. That's the corner we will hide.
					double MaxDot = -999999.;
					int MaxDotIndex = -1;
					for (int j = 0; j < 8; ++j)
					{
						double Dot = FVector::DotProduct(Points[j], ViewDirection);
						if (Dot > MaxDot)
						{
							MaxDot = Dot;
							MaxDotIndex = j;
						}
						Points[j] += WorldCenter;
					}
					for (int j = 0; j < 12; ++j)
					{
						if (Lines[j][0] != MaxDotIndex && Lines[j][1] != MaxDotIndex)
						{
							PDI->DrawLine(Points[Lines[j][0]], Points[Lines[j][1]], Color, SDPG_Foreground, UseThickness, 0.0f, true);
						}
					}
				}
				else
				{
					for (int j = 0; j < 8; ++j)
					{
						Points[j] += WorldCenter;
					}
					for (int j = 0; j < 12; ++j)
					{
						PDI->DrawLine(Points[Lines[j][0]], Points[Lines[j][1]], Color, SDPG_Foreground, UseThickness, 0.0f, true);
					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();

		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return false;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return IntCastChecked<uint32>(sizeof *this + GetAllocatedSize()); }
	SIZE_T GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	void SetExternalHoverState(bool* HoverState)
	{
		bExternalHoverState = HoverState;
	}

	void SetExternalWorldLocalState(bool* bWorldLocalState)
	{
		bExternalWorldLocalState = bWorldLocalState;
	}

	void SetExternalIsViewDependent(bool* bExternalIsViewDependentIn)
	{
		bExternalIsViewDependent = bExternalIsViewDependentIn;
	}

private:
	FLinearColor Color;
	FVector LocalCenter;
	FVector DirectionX, DirectionY, DirectionZ;;
	FVector Dimensions;
	float Thickness;
	float HoverThicknessMultiplier;
	bool bEnableFlipping = false;
	bool bRemoveHiddenLines = true;

	// set on Component for use in ::GetDynamicMeshElements()
	bool* bExternalHoverState = nullptr;
	bool* bExternalWorldLocalState = nullptr;
	bool* bExternalIsViewDependent = nullptr;
};



FPrimitiveSceneProxy* UGizmoBoxComponent::CreateSceneProxy()
{
	FGizmoBoxComponentSceneProxy* NewProxy = new FGizmoBoxComponentSceneProxy(this);
	NewProxy->SetExternalHoverState(&bHovering);
	NewProxy->SetExternalWorldLocalState(&bWorld);
	NewProxy->SetExternalIsViewDependent(&bIsViewDependent);
	return NewProxy;
}

FBoxSphereBounds UGizmoBoxComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FSphere(Origin, Dimensions.Size()).TransformBy(LocalToWorld));
}

bool UGizmoBoxComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	using namespace GizmoBoxComponentLocals;

	const FTransform& Transform = this->GetComponentToWorld();
	FTransform TransformToUse = Transform;
	if (bWorld)
	{
		TransformToUse.SetRotation(FQuat::Identity);
	}

	// transform points into local space
	FVector StartLocal = TransformToUse.InverseTransformPosition(Start);
	FVector EndLocal = TransformToUse.InverseTransformPosition(End);

	// transform into box-specific rotation space
	FQuat InvBoxRotation = Rotation.Inverse();
	StartLocal = InvBoxRotation * StartLocal;
	EndLocal = InvBoxRotation * EndLocal;

	bool bFlippedX = false; 
	bool bFlippedY = false;
	bool bFlippedZ = false;

	FVector WorldOrigin; // Only used if bIsViewDependent
	if (bIsViewDependent)
	{
		// It doesn't matter whether we use TransformToUse or Transform in this block since
		// the rotation component does not affect world origin and the vector transforms in
		// GetBoxDirections are only used if bWorld is false. We'll use the original transform
		// here just to line up with what our scene proxy is doing.

		WorldOrigin = Transform.TransformPosition(FVector::ZeroVector);

		FVector UseDirectionX, UseDirectionY, UseDirectionZ; // not used
		GetBoxDirections(bIsViewDependent, ToRawPtr(GizmoViewContext),
			WorldOrigin,
			Rotation * FVector(1, 0, 0), // DirectionX
			Rotation * FVector(0, 1, 0), // DirectionY
			Rotation * FVector(0, 0, 1), // DirectionZ
			bEnableAxisFlip, bWorld,
			[&Transform](const FVector& VectorIn) { return Transform.TransformVector(VectorIn); },
			UseDirectionX, UseDirectionY, UseDirectionZ, bFlippedX, bFlippedY, bFlippedZ);
	}

	FVector UseCenter(
		(bEnableAxisFlip && bFlippedX) ? -Origin.X : Origin.X,
		(bEnableAxisFlip && bFlippedY) ? -Origin.Y : Origin.Y,
		(bEnableAxisFlip && bFlippedZ) ? -Origin.Z : Origin.Z
	);

	float DynamicPixelToWorldScale = 1;

	if (bIsViewDependent)
	{
		DynamicPixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(
			GizmoViewContext, WorldOrigin);
	}

	UseCenter *= DynamicPixelToWorldScale;

	FVector ScaledDims = DynamicPixelToWorldScale * Dimensions;
	FBox Box(UseCenter - 0.5f * ScaledDims, UseCenter + 0.5f * ScaledDims);

	FVector Extent(SMALL_NUMBER, SMALL_NUMBER, SMALL_NUMBER);
	FVector HitLocal, NormalLocal; float HitTime;
	if (FMath::LineExtentBoxIntersection(Box, StartLocal, EndLocal, Extent, HitLocal, NormalLocal, HitTime) == false)
	{
		return false;
	}

	FVector HitWorld = TransformToUse.TransformPosition(Rotation * HitLocal);

	OutHit.Component = this;
	OutHit.ImpactPoint = HitWorld;
	OutHit.Distance = static_cast<float>(FVector::Distance(Start, HitWorld));
	//OutHit.ImpactNormal = ;
	
	return true;
}

void UGizmoBoxComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	//OutMaterials.Add(GEngine->VertexColorMaterial);
}
