// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoCircleComponent.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoCircleComponent)

namespace GizmoCircleComponentLocals
{
	const float RENDER_VISIBILITY_DOT_THRESHOLD = 0.05f;
	const float VIEW_PLANE_PARALLEL_DOT_THRESHOLD = 0.95f;

	template <typename SceneViewOrGizmoViewContext>
	void GetVisibility(const SceneViewOrGizmoViewContext* View, const FVector& ViewDirection,
		const FVector& GizmoPlaneWorldNormal, const FVector& WorldOrigin,
		bool& bRenderVisibilityOut, bool& bIsViewPlaneParallelOut)
	{
		FVector ViewPlaneDirection = View->GetViewDirection();

		bRenderVisibilityOut = FMath::Abs(ViewDirection.Dot(GizmoPlaneWorldNormal)) >= RENDER_VISIBILITY_DOT_THRESHOLD;
		bIsViewPlaneParallelOut = FMath::Abs(ViewPlaneDirection.Dot(GizmoPlaneWorldNormal)) >= VIEW_PLANE_PARALLEL_DOT_THRESHOLD;
	}

}

class FGizmoCircleComponentSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FGizmoCircleComponentSceneProxy(const UGizmoCircleComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent),
		Color(InComponent->Color),
		Normal(InComponent->Normal),
		Radius(InComponent->Radius),
		Thickness(InComponent->Thickness),
		NumSides(InComponent->NumSides),
		bViewAligned(InComponent->bViewAligned),
		bDrawFullCircle(InComponent->bDrawFullCircle),
		HoverThicknessMultiplier(InComponent->HoverSizeMultiplier)
	{
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		using namespace GizmoCircleComponentLocals;

		const FMatrix& LocalToWorldMatrix = GetLocalToWorld();
		FVector Origin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);
		FVector PlaneX, PlaneY;
		GizmoMath::MakeNormalPlaneBasis(Normal, PlaneX, PlaneY);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
				bool bIsOrtho = !View->IsPerspectiveProjection();
				FVector ViewVector = View->GetViewDirection();

				float LengthScale = 1;
				bool bIsViewDependent = (bExternalIsViewDependent) ? (*bExternalIsViewDependent) : false;
				if (bIsViewDependent)
				{
					LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, Origin);
				}

				double UseRadius = LengthScale * Radius;

				FLinearColor BackColor = FLinearColor(0.5f, 0.5f, 0.5f);
				float BackThickness = 0.5f;
				float UseThickness = (bExternalHoverState != nullptr && *bExternalHoverState == true) ?
					(HoverThicknessMultiplier * Thickness) : (Thickness);
				if (!bIsOrtho)
				{
					UseThickness *= (View->FOV / 90.0f);		// compensate for FOV scaling in Gizmos...
					BackThickness *= (View->FOV / 90.0f);		// compensate for FOV scaling in Gizmos...
				}

				if (bDrawFullCircle)
				{
					BackThickness = UseThickness;
					BackColor = Color;
				}

				const float	AngleDelta = 2.0f * PI / NumSides;

				if (bViewAligned)
				{
					FVector WorldOrigin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);
					WorldOrigin += 0.001 * ViewVector;
					FVector WorldPlaneX, WorldPlaneY;
					GizmoMath::MakeNormalPlaneBasis(ViewVector, WorldPlaneX, WorldPlaneY);

					FVector	LastVertex = WorldOrigin + WorldPlaneX * UseRadius;
					for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
					{
						float DeltaX = FMath::Cos(AngleDelta * (SideIndex + 1));
						float DeltaY = FMath::Sin(AngleDelta * (SideIndex + 1));
						const FVector DeltaVector = WorldPlaneX * DeltaX + WorldPlaneY * DeltaY;
						const FVector Vertex = WorldOrigin + UseRadius * DeltaVector;
						PDI->DrawLine(LastVertex, Vertex, Color, SDPG_Foreground, UseThickness, 0.0f, true);
						LastVertex = Vertex;
					}
				}
				else 
				{
					FVector WorldOrigin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);
					bool bWorldAxis = (bExternalWorldLocalState) ? (*bExternalWorldLocalState) : false;
					FVector WorldPlaneX = (bWorldAxis) ? PlaneX : FVector{ LocalToWorldMatrix.TransformVector(PlaneX) };
					FVector WorldPlaneY = (bWorldAxis) ? PlaneY : FVector{ LocalToWorldMatrix.TransformVector(PlaneY) };

					FVector PlaneWorldNormal = (bWorldAxis) ? Normal : FVector{ LocalToWorldMatrix.TransformVector(Normal) };

					// direction to origin of gizmo
					FVector GizmoViewDirection =
						(bIsOrtho) ? (View->GetViewDirection()) : (Origin - View->ViewLocation);
					GizmoViewDirection.Normalize();

					bool bRenderVisibility = true;
					bool bIsViewPlaneParallel = false;
					if (bIsViewDependent)
					{
						GetVisibility(View, GizmoViewDirection, PlaneWorldNormal, WorldOrigin, bRenderVisibility, bIsViewPlaneParallel);
					}

					if (bRenderVisibility)
					{
						if (bIsViewPlaneParallel)
						{
							FVector	LastVertex = WorldOrigin + WorldPlaneX * UseRadius;
							for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
							{
								float DeltaX = FMath::Cos(AngleDelta * (SideIndex + 1));
								float DeltaY = FMath::Sin(AngleDelta * (SideIndex + 1));
								const FVector DeltaVector = WorldPlaneX * DeltaX + WorldPlaneY * DeltaY;
								const FVector Vertex = WorldOrigin + UseRadius * DeltaVector;
								PDI->DrawLine(LastVertex, Vertex, Color, SDPG_Foreground, UseThickness, 0.0f, true);
								LastVertex = Vertex;
							}
						}
						else
						{
							FVector	LastVertex = WorldOrigin + WorldPlaneX * UseRadius;
							bool bLastVisible = FVector::DotProduct(WorldPlaneX, GizmoViewDirection) < 0;
							for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
							{
								float DeltaX = FMath::Cos(AngleDelta * (SideIndex + 1));
								float DeltaY = FMath::Sin(AngleDelta * (SideIndex + 1));
								const FVector DeltaVector = WorldPlaneX * DeltaX + WorldPlaneY * DeltaY;
								const FVector Vertex = WorldOrigin + UseRadius * DeltaVector;
								bool bVertexVisible = FVector::DotProduct(DeltaVector, GizmoViewDirection) < 0;
								if (bLastVisible && bVertexVisible)
								{
									PDI->DrawLine(LastVertex, Vertex, Color, SDPG_Foreground, UseThickness, 0.0f, true);
								}
								else
								{
									PDI->DrawLine(LastVertex, Vertex, BackColor, SDPG_Foreground, BackThickness, 0.0f, true);
								}
								bLastVisible = bVertexVisible;
								LastVertex = Vertex;
							}
						}
					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return false;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return IntCastChecked<uint32>(sizeof *this + GetAllocatedSize()); }
	SIZE_T GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	void SetExternalHoverState(bool* HoverState)
	{
		bExternalHoverState = HoverState;
	}

	void SetExternalWorldLocalState(bool* bWorldLocalState)
	{
		bExternalWorldLocalState = bWorldLocalState;
	}

	void SetExternalIsViewDependent(bool* bExternalIsViewDependentIn)
	{
		bExternalIsViewDependent = bExternalIsViewDependentIn;
	}

private:
	FLinearColor Color;
	FVector Normal;
	float Radius;
	float Thickness;
	int NumSides;
	bool bViewAligned;
	bool bDrawFullCircle;
	float HoverThicknessMultiplier;

	// set on Component for use in ::GetDynamicMeshElements()
	bool* bExternalHoverState = nullptr;
	bool* bExternalWorldLocalState = nullptr;
	bool* bExternalIsViewDependent = nullptr;
};




FPrimitiveSceneProxy* UGizmoCircleComponent::CreateSceneProxy()
{
	FGizmoCircleComponentSceneProxy* NewProxy = new FGizmoCircleComponentSceneProxy(this);
	NewProxy->SetExternalHoverState(&bHovering);
	NewProxy->SetExternalWorldLocalState(&bWorld);
	NewProxy->SetExternalIsViewDependent(&bIsViewDependent);
	return NewProxy;
}

FBoxSphereBounds UGizmoCircleComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FSphere(FVector::ZeroVector, Radius).TransformBy(LocalToWorld));
}

bool UGizmoCircleComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	using namespace GizmoCircleComponentLocals;

	const FTransform& Transform = this->GetComponentToWorld();
	FVector WorldOrigin = Transform.TransformPosition(FVector::ZeroVector);
	FVector WorldNormal = (bWorld) ? Normal : Transform.TransformVector(Normal);

	bool bRenderVisibility = true;
	bool bCircleIsViewPlaneParallel = false;
	float PixelToWorldScale = 1;
	if (bIsViewDependent)
	{
		FVector ViewDirection = GizmoViewContext->IsPerspectiveProjection() ? (WorldOrigin - GizmoViewContext->ViewLocation)
			: GizmoViewContext->GetViewDirection();
		ViewDirection.Normalize();

		GetVisibility(ToRawPtr(GizmoViewContext), ViewDirection, WorldNormal, WorldOrigin, bRenderVisibility, bCircleIsViewPlaneParallel);

		if (!bRenderVisibility)
		{
			return false;
		}

		PixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, WorldOrigin);
	}

	float LengthScale = PixelToWorldScale;
	float UseRadius = LengthScale * Radius;


	FRay Ray(Start, End - Start, false);

	// Find the intresection with the circle plane. Note that unlike the FMath version, GizmoMath::RayPlaneIntersectionPoint() 
	// checks that the ray isn't parallel to the plane.
	bool bIntersects;
	FVector HitPos;
	GizmoMath::RayPlaneIntersectionPoint(WorldOrigin, WorldNormal, Ray.Origin, Ray.Direction, bIntersects, HitPos);
	if (!bIntersects || Ray.GetParameter(HitPos) > Ray.GetParameter(End))
	{
		return false;
	}

	FVector NearestCircle;
	GizmoMath::ClosetPointOnCircle(HitPos, WorldOrigin, WorldNormal, UseRadius, NearestCircle);

	FVector NearestRay = Ray.ClosestPoint(NearestCircle);

	double Distance = FVector::Distance(NearestCircle, NearestRay);
	if (Distance > PixelHitDistanceThreshold * PixelToWorldScale)
	{
		return false;
	}

	// filter out hits on "back" of sphere that circle lies on
	if (bOnlyAllowFrontFacingHits && bCircleIsViewPlaneParallel == false)
	{
		bool bSphereIntersects = false;
		FVector SphereHitPoint;
		FVector RayToCirclePointDirection = (NearestCircle - Ray.Origin);
		RayToCirclePointDirection.Normalize();
		GizmoMath::RaySphereIntersection(
			WorldOrigin, UseRadius, Ray.Origin, RayToCirclePointDirection, bSphereIntersects, SphereHitPoint);
		if (bSphereIntersects)
		{
			if (FVector::DistSquared(SphereHitPoint, NearestCircle) > UseRadius*0.1f)
			{
				return false;
			}
		}
	}

	OutHit.Component = this;
	OutHit.Distance = static_cast<float>(FVector::Distance(Start, NearestRay));
	OutHit.ImpactPoint = NearestRay;
	return true;
}
