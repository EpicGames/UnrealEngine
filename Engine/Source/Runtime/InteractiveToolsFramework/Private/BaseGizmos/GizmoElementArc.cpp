// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementArc.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "DynamicMeshBuilder.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "VectorUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementArc)


static void DrawThickArc(FPrimitiveDrawInterface* PDI, const FVector& InCenter, const FVector& InAxis0, const FVector& InAxis1,
	double InOuterRadius, double InInnerRadius, const double InStartAngle, const double InEndAngle, const int32 InNumSegments,
	const FMaterialRenderProxy* MaterialRenderProxy, const FColor& InColor, const FColor& InLineColor, const double InLineThickness = 0.0)
{
	// Implementation copied from FWidget::DrawThickArc. This should eventually be moved to PrimDrawingUtils.h/cpp.

	if (InColor.A == 0)
	{
		return;
	}

	// Boost depth bias for lines to avoid z-fighting with the fill
	const float LineDepthBias = 1.0f + InLineThickness;

	const int32 NumPoints = FMath::TruncToInt32(InNumSegments * (InEndAngle - InStartAngle) / (UE_DOUBLE_PI / 2.0)) + 1;

	FColor TriangleColor = InColor;

	const FVector3f Axis1f = static_cast<FVector3f>(InAxis1);
	const FVector3f Axis0f = static_cast<FVector3f>(InAxis0);

	// Retain the higher precision version for rotation calculations
	const FVector ZAxis = InAxis0 ^ InAxis1;
	const FVector3f ZAxisf = Axis0f ^ Axis1f;
	FVector LastWorldVertex;

	// Whether to draw the lines on either end of the arc (the edges of the proverbial pizza slice).
	// We don't draw them if the arc forms a complete/closed circle, so we check if the NumPoints exceeds the NumSegments in 360 degrees (NumSegments is per 90 degrees).
	const bool bDrawEndLines = NumPoints <= (InNumSegments * 4);

	TArray<FDynamicMeshVertex> Vertices;
	Vertices.Reserve((NumPoints * 2) + 2);

	FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

	for (int32 RadiusIndex = 0; RadiusIndex < 2; ++RadiusIndex)
	{
		double Radius = (RadiusIndex == 0) ? InOuterRadius : InInnerRadius;
		double TCRadius = Radius / InOuterRadius;
		//Compute vertices for base circle.
		for (int32 VertexIndex = 0; VertexIndex <= NumPoints; VertexIndex++)
		{
			double Percent = VertexIndex / static_cast<double>(NumPoints);
			double Angle = FMath::Lerp(InEndAngle, InStartAngle, Percent);
			double AngleDeg = FRotator::ClampAxis(Angle * 180.f / PI);

			FVector VertexDir = InAxis0.RotateAngleAxis(AngleDeg, ZAxis);
			VertexDir.Normalize();

			FVector2f TC(static_cast<float>(TCRadius * FMath::Cos(Angle)), static_cast<float>(TCRadius * FMath::Sin(Angle)));

			// Keep the vertices in local space so that we don't lose precision when dealing with LWC
			// The local-to-world transform is handled in the MeshBuilder.Draw() call at the end of this function
			const FVector VertexPosition = VertexDir * Radius;
			FVector Normal = VertexPosition;
			Normal.Normalize();

			FDynamicMeshVertex MeshVertex;
			MeshVertex.Position = static_cast<FVector3f>(VertexPosition);
			MeshVertex.Color = TriangleColor;
			MeshVertex.TextureCoordinate[0] = TC;

			MeshVertex.SetTangents(
				Axis1f,
				-Axis0f,
				ZAxisf
			);

			Vertices.Emplace(MeshVertex); //Add bottom vertex

			// Push out the arc line borders so they dont z-fight with the mesh arcs
			// DrawLine needs vertices in world space, but this is fine because it takes FVectors and works with LWC well
			FVector EndLinePos = VertexPosition + InCenter;
			if (VertexIndex != 0)
			{
				PDI->DrawTranslucentLine(LastWorldVertex, EndLinePos, InLineColor, SDPG_Foreground, InLineThickness, LineDepthBias);
			}
			LastWorldVertex = EndLinePos;
		}
	}

	MeshBuilder.AddVertices(Vertices);

	if (bDrawEndLines)
	{
		check(Vertices.Num() == (NumPoints * 2) + 2);

		// From the first vert on the outer ring to the first vert on the inner ring
		FVector FirstEndLinePosOuter = static_cast<FVector>(Vertices[0].Position) + InCenter;
		FVector FirstEndLinePosInner = static_cast<FVector>(Vertices[NumPoints + 1].Position) + InCenter;
		PDI->DrawTranslucentLine(FirstEndLinePosOuter, FirstEndLinePosInner, InLineColor, SDPG_Foreground, InLineThickness, LineDepthBias);

		// From the last vert on the outer ring to the last vert on the inner ring
		FVector LastEndLinePosOuter = static_cast<FVector>(Vertices[NumPoints].Position) + InCenter;
		FVector LastEndLinePosInner = static_cast<FVector>(Vertices.Last().Position) + InCenter;
		PDI->DrawTranslucentLine(LastEndLinePosOuter, LastEndLinePosInner, InLineColor, SDPG_Foreground, InLineThickness, LineDepthBias);
	}

	//Add top/bottom triangles, in the style of a fan.
	int32 InnerVertexStartIndex = NumPoints + 1;
	for (int32 VertexIndex = 0; VertexIndex < NumPoints; VertexIndex++)
	{
		MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, InnerVertexStartIndex + VertexIndex);
		MeshBuilder.AddTriangle(VertexIndex + 1, InnerVertexStartIndex + VertexIndex + 1, InnerVertexStartIndex + VertexIndex);
	}

	MeshBuilder.Draw(PDI, FTranslationMatrix(InCenter), MaterialRenderProxy, SDPG_Foreground, 0.f);
}


UGizmoElementArc::UGizmoElementArc()
{
	LineThickness = 0.0f;
	HoverLineThicknessMultiplier = 1.0f;
	InteractLineThicknessMultiplier = 1.0f;
	SelectLineThicknessMultiplier = 1.0f;
	SubdueLineThicknessMultiplier = 1.0f;
}

void UGizmoElementArc::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (!bVisibleViewDependent)
	{
		return;
	}

	const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial();
	if (!UseMaterial)
	{
		return;
	}
	const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
	const FVector WorldAxis0 = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Axis0);
	const FVector WorldAxis1 = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Axis1);
	const FVector WorldNormal = WorldAxis0 ^ WorldAxis1;

	const FSceneView* View = RenderAPI->GetSceneView();

	bool bPartial = IsPartial(View, WorldCenter, WorldNormal);
	double Start, End;
	if (bPartial)
	{
		if (PartialEndAngle - PartialStartAngle <= 0)
		{
			return;
		}

		Start = PartialStartAngle;
		End = PartialEndAngle;
	}
	else
	{
		Start = 0.0;
		End = UE_DOUBLE_TWO_PI;
	}

	const double WorldOuterRadius = Radius * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const double WorldInnerRadius = InnerRadius * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
	const FColor VertexColor = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);
	const FColor LineColor = CurrentRenderState.GetCurrentLineColor().ToFColor(true);
	const double CurrentLineThickness = GetCurrentLineThickness(View->IsPerspectiveProjection(), View->FOV)
										* CurrentRenderState.LocalToWorldTransform.GetScale3D().X;

	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

	DrawThickArc(PDI, WorldCenter, WorldAxis0, WorldAxis1, WorldOuterRadius, WorldInnerRadius,
		Start, End, NumSegments, UseMaterial->GetRenderProxy(), VertexColor, LineColor, CurrentLineThickness);
}

FInputRayHit UGizmoElementArc::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		const FVector WorldCenter = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
		const FVector WorldAxis0 = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Axis0);
		const FVector WorldAxis1 = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Axis1);
		const FVector WorldNormal = WorldAxis0 ^ WorldAxis1;
		double HitDepth = -1.0;

		bool bPartial = IsPartial(ViewContext, WorldCenter, WorldNormal);
		const double PartialAngle = PartialEndAngle - PartialStartAngle;

		if (bPartial && PartialAngle <= UE_DOUBLE_SMALL_NUMBER)
		{
			return FInputRayHit();
		}

		// Intersect ray with plane in which arc lies.
		FPlane Plane(WorldCenter, WorldNormal);
		HitDepth = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);
		if (HitDepth < UE_DOUBLE_SMALL_NUMBER)
		{
			return FInputRayHit();
		}

		FVector HitPoint = RayOrigin + RayDirection * HitDepth;
		FVector HitVec = HitPoint - WorldCenter;

		// Determine whether hit point lies within the arc
		const double WorldOuterRadius = Radius * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X;
		const double WorldInnerRadius = InnerRadius * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X;
		const double Distance = HitVec.Length();
		double HitBufferMax = WorldOuterRadius + PixelHitThresholdAdjust;
		double HitBufferMin = WorldInnerRadius - PixelHitThresholdAdjust;

		if (Distance > HitBufferMax || Distance < HitBufferMin)
		{
			return FInputRayHit();
		}

		// Handle partial arc
		if (bPartial)
		{
			// Compute projected angle
			FVector WorldBeginAxis = WorldAxis0.RotateAngleAxis(PartialStartAngle, WorldNormal).GetSafeNormal();
			double HitAngle = UE::Geometry::VectorUtil::PlaneAngleSignedR(WorldBeginAxis, HitVec, WorldNormal); 

			if (HitAngle < UE_DOUBLE_SMALL_NUMBER)
			{
				HitAngle = UE_DOUBLE_TWO_PI + HitAngle;
			}

			if (HitAngle > PartialAngle)
			{
				return FInputRayHit();
			}
		}

		if (HitDepth >= UE_DOUBLE_SMALL_NUMBER)
		{
			FInputRayHit RayHit = MakeRayHit(HitDepth, OutLineTraceOutput);

			// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold) ...
			PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * FMath::Max(0.0, MinimumPixelHitDistanceThreshold);
			HitBufferMax = WorldOuterRadius + PixelHitThresholdAdjust;
			HitBufferMin = WorldInnerRadius - PixelHitThresholdAdjust;

			OutLineTraceOutput.bIsSurfaceHit = Distance <= HitBufferMax && Distance >= HitBufferMin;

			return RayHit;
		}
	}
	
	return FInputRayHit();
}

void UGizmoElementArc::SetInnerRadius(double InInnerRadius)
{
	InnerRadius = InInnerRadius;
}

double UGizmoElementArc::GetInnerRadius() const
{
	return InnerRadius;
}
