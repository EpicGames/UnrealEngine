// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementArrow.h"

#include "BaseGizmos/GizmoElementBox.h"
#include "BaseGizmos/GizmoElementCone.h"
#include "BaseGizmos/GizmoElementCylinder.h"
#include "BaseGizmos/GizmoElementSphere.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementArrow)

namespace UE::Private
{

void UpdatePixelThreshold(UGizmoElementBase* InGizmoElement, const float InPixelHitThreshold)
{
	if (InGizmoElement)
	{
		InGizmoElement->SetPixelHitDistanceThreshold(InPixelHitThreshold);
	}		
}

void UpdateMinPixelThreshold(UGizmoElementBase* InGizmoElement, const float InPixelHitThreshold)
{
	if (InGizmoElement)
	{
		InGizmoElement->SetMinimumPixelHitDistanceThreshold(InPixelHitThreshold);
	}		
}

using ThresholdGuard = TGuardValue_Bitfield_Cleanup<TFunction<void()>>;

}

UGizmoElementArrow::UGizmoElementArrow()
{
	bHitOwner = true;

	HeadType = EGizmoElementArrowHeadType::Cone;

	CylinderElement = NewObject<UGizmoElementCylinder>();
	UGizmoElementArrow::Add(CylinderElement);

	HeadElement = NewObject<UGizmoElementArrowHead>();
	HeadElement->SetType(HeadType);
	UGizmoElementArrow::Add(HeadElement);
}

void UGizmoElementArrow::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	check(RenderAPI);

	if (bUpdateArrowBody)
	{
		UpdateArrowBody();
	}

	if (bUpdateArrowHead)
	{
		UpdateArrowHead();
	}

	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Base, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		check(CylinderElement);
		CylinderElement->Render(RenderAPI, CurrentRenderState);

		check(HeadElement);
		HeadElement->Render(RenderAPI, CurrentRenderState);
	}
}

FInputRayHit UGizmoElementArrow::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	using namespace UE::Private;

	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Base, CurrentLineTraceState);

	if (!bHittableViewDependent)
	{
		return FInputRayHit();
	}

	ThresholdGuard Guard([this]()
	{
		UpdatePixelThreshold(CylinderElement, PixelHitDistanceThreshold);
		UpdatePixelThreshold(HeadElement, PixelHitDistanceThreshold);

		UpdateMinPixelThreshold(CylinderElement, MinimumPixelHitDistanceThreshold);
		UpdateMinPixelThreshold(HeadElement, MinimumPixelHitDistanceThreshold);
	});

	// update threshold if hitting the mask
	if (UGizmoElementBase* HitMaskGizmo = HitMask.Get())
	{
		const FInputRayHit MaskHit = HitMaskGizmo->LineTrace(ViewContext, LineTraceState, RayOrigin, RayDirection, OutLineTraceOutput);
		if (MaskHit.bHit)
		{
			static constexpr float NoPixelHitThreshold = 0.f;
			UpdatePixelThreshold(CylinderElement, NoPixelHitThreshold);
			UpdatePixelThreshold(HeadElement, NoPixelHitThreshold);
		}
	}
	
	check(CylinderElement);
	FInputRayHit Hit = CylinderElement->LineTrace(ViewContext, CurrentLineTraceState, RayOrigin, RayDirection, OutLineTraceOutput);

	// If we didn't hit the cylinder, OR it was hit, but not a surface hit, try the arrow head
	if (!Hit.bHit || !OutLineTraceOutput.bIsSurfaceHit)
	{
		check(HeadElement);
		FLineTraceOutput HeadLineTraceOutput = OutLineTraceOutput;
		FInputRayHit HeadHit = HeadElement->LineTrace(ViewContext, CurrentLineTraceState, RayOrigin, RayDirection, HeadLineTraceOutput);
		if (HeadHit.bHit && HeadLineTraceOutput.bIsSurfaceHit)
		{
			// If we're here, the cylinder was hit, but not on it's surface - but the head was hit directly on the surface.
			// This means we'll take priority over another element with a higher hit priority
			OutLineTraceOutput = HeadLineTraceOutput;
			Hit = HeadHit;
		}
	}

	if (Hit.bHit)
	{
		Hit.SetHitObject(this);
		Hit.HitIdentifier = PartIdentifier;
	}

	return Hit;
}

void UGizmoElementArrow::SetBase(const FVector& InBase)
{
	if (Base != InBase)
	{
		Base = InBase;
		bUpdateArrowBody = true;
		bUpdateArrowHead = true;
	}
}

FVector UGizmoElementArrow::GetBase() const
{
	return Base;
}

void UGizmoElementArrow::SetDirection(const FVector& InDirection)
{
	Direction = InDirection;
	Direction.Normalize();
	bUpdateArrowBody = true;
	bUpdateArrowHead = true;
}

FVector UGizmoElementArrow::GetDirection() const
{
	return Direction;
}

void UGizmoElementArrow::SetSideDirection(const FVector& InSideDirection)
{
	SideDirection = InSideDirection;
	SideDirection.Normalize();
	bUpdateArrowHead = true;
}

FVector UGizmoElementArrow::GetSideDirection() const
{
	return SideDirection;
}

void UGizmoElementArrow::SetBodyLength(float InBodyLength)
{
	if (BodyLength != InBodyLength)
	{
		BodyLength = InBodyLength;
		bUpdateArrowBody = true;
		bUpdateArrowHead = true;
	}
}

float UGizmoElementArrow::GetBodyLength() const
{
	return BodyLength;
}

void UGizmoElementArrow::SetBodyRadius(float InBodyRadius)
{
	if (BodyRadius != InBodyRadius)
	{
		BodyRadius = InBodyRadius;
		bUpdateArrowBody = true;
		bUpdateArrowHead = true;
	}
}

float UGizmoElementArrow::GetBodyRadius() const
{
	return BodyRadius;
}

void UGizmoElementArrow::SetHeadLength(float InHeadLength)
{
	if (HeadLength != InHeadLength)
	{
		HeadLength = InHeadLength;
		bUpdateArrowHead = true;
	}
}

float UGizmoElementArrow::GetHeadLength() const
{
	return HeadLength;
}

void UGizmoElementArrow::SetHeadRadius(float InHeadRadius)
{
	if (HeadRadius != InHeadRadius)
	{
		HeadRadius = InHeadRadius;
		bUpdateArrowHead = true;
	}
}

float UGizmoElementArrow::GetHeadRadius() const
{
	return HeadRadius;
}

void UGizmoElementArrow::SetNumSides(int32 InNumSides)
{
	if (NumSides != InNumSides)
	{
		NumSides = InNumSides;
		bUpdateArrowBody = true;
		bUpdateArrowHead = true;
	}
}

int32 UGizmoElementArrow::GetNumSides() const
{
	return NumSides;
}

void UGizmoElementArrow::SetEndCaps(bool InEndCaps)
{
	if (bEndCaps != InEndCaps)
	{
		bEndCaps = InEndCaps;
		bUpdateArrowHead = true;
	}
}

bool UGizmoElementArrow::GetEndCaps() const
{
	return bEndCaps;
}

void UGizmoElementArrow::SetPixelHitDistanceThreshold(float InPixelHitDistanceThreshold)
{
	if (PixelHitDistanceThreshold != InPixelHitDistanceThreshold)
	{
		PixelHitDistanceThreshold = InPixelHitDistanceThreshold;
		bUpdateArrowBody = true;
		bUpdateArrowHead = true;
	}
}

void UGizmoElementArrow::SetMinimumPixelHitDistanceThreshold(float InMinimumPixelHitDistanceThreshold)
{
	if (MinimumPixelHitDistanceThreshold != InMinimumPixelHitDistanceThreshold)
	{
		MinimumPixelHitDistanceThreshold = InMinimumPixelHitDistanceThreshold;
		bUpdateArrowBody = true;
		bUpdateArrowHead = true;
	}
}

void UGizmoElementArrow::SetHitMask(const TWeakObjectPtr<UGizmoElementBase>& InHitMask)
{
	HitMask = InHitMask;
}

void UGizmoElementArrow::SetHeadType(EGizmoElementArrowHeadType InHeadType)
{
	if (InHeadType != HeadType)
	{
		HeadType = InHeadType;
		UpdateArrowHead();
	}
}

EGizmoElementArrowHeadType UGizmoElementArrow::GetHeadType() const
{
	return HeadType;
}

void UGizmoElementArrow::SetIsDashed(bool bInDashing)
{
	if (ensure(CylinderElement))
	{
		CylinderElement->SetIsDashed(bInDashing);
	}
}

bool UGizmoElementArrow::IsDashed() const
{
	if (ensure(CylinderElement))
	{
		return CylinderElement->GetIsDashed();
	}

	return false;
}

void UGizmoElementArrow::SetDashParameters(const float InDashLength, const TOptional<float>& InGapLength)
{
	if (ensure(CylinderElement))
	{
		CylinderElement->SetDashParameters(InDashLength, InGapLength);
	}
}

void UGizmoElementArrow::GetDashParameters(float& OutDashLength, float& OutGapLength) const
{
	if (ensure(CylinderElement))
	{
		CylinderElement->GetDashParameters(OutDashLength, OutGapLength);
	}
}

void UGizmoElementArrow::UpdateArrowBody()
{
	CylinderElement->SetBase(FVector::ZeroVector);
	CylinderElement->SetDirection(Direction);
	CylinderElement->SetHeight(BodyLength);
	CylinderElement->SetNumSides(NumSides);
	CylinderElement->SetRadius(BodyRadius);
	CylinderElement->SetPixelHitDistanceThreshold(PixelHitDistanceThreshold);

	bUpdateArrowBody = false;
}

void UGizmoElementArrow::UpdateArrowHead()
{
	check(HeadElement);
	HeadElement->SetType(HeadType);
	HeadElement->SetCenter(Direction * BodyLength);
	HeadElement->SetDirection(Direction);
	HeadElement->SetSideDirection(SideDirection);
	HeadElement->SetLength(HeadLength);
	HeadElement->SetRadius(HeadRadius);
	HeadElement->SetNumSides(NumSides);
	HeadElement->SetElementInteractionState(ElementInteractionState);
	HeadElement->SetPixelHitDistanceThreshold(PixelHitDistanceThreshold);

	bUpdateArrowHead = false;
}
