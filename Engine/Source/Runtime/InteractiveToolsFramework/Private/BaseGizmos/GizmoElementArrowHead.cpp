// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoElementArrowHead.h"

#include "BaseGizmos/GizmoElementBox.h"
#include "BaseGizmos/GizmoElementCone.h"
#include "BaseGizmos/GizmoElementSphere.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementArrowHead)

namespace UE::InteractiveToolsFramework::Private
{
	namespace GizmoElementArrowHeadLocals
	{
		void UpdatePixelThreshold(UGizmoElementBase* InGizmoElement, const float InPixelHitThreshold)
		{
			if (InGizmoElement)
			{
				InGizmoElement->SetPixelHitDistanceThreshold(InPixelHitThreshold);
			}
		}

		using FThresholdGuard = TGuardValue_Bitfield_Cleanup<TFunction<void()>>;
	}
}

UGizmoElementArrowHead::UGizmoElementArrowHead()
{
	Type = EGizmoElementArrowHeadType::Cone;
	bHitOwner = true;

	ConeElement = NewObject<UGizmoElementCone>();
	BoxElement = nullptr;
	SphereElement = nullptr;

	UGizmoElementArrowHead::Add(ConeElement);
}

void UGizmoElementArrowHead::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	check(RenderAPI);

	if (bUpdate)
	{
		Update();
	}

	FRenderTraversalState CurrentRenderState(RenderState);
	const bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		auto RenderElement = [RenderAPI, &CurrentRenderState](UGizmoElementBase* InElement)
		{
			check(InElement);
			InElement->Render(RenderAPI, CurrentRenderState);
		};

		if (Type == EGizmoElementArrowHeadType::Cone)
		{
			RenderElement(ConeElement);
		}
		else if (Type == EGizmoElementArrowHeadType::Cube)
		{
			RenderElement(BoxElement);
		}
		else if (Type == EGizmoElementArrowHeadType::Sphere)
		{
			RenderElement(SphereElement);
		}
	}
}

FInputRayHit UGizmoElementArrowHead::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementArrowHeadLocals;

	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	const bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (!bHittableViewDependent)
	{
		return FInputRayHit();
	}

	FThresholdGuard Guard([this]()
	{
		UpdatePixelThreshold(ConeElement, PixelHitDistanceThreshold);
		UpdatePixelThreshold(BoxElement, PixelHitDistanceThreshold);
		UpdatePixelThreshold(SphereElement, PixelHitDistanceThreshold);
	});

	// update threshold if hitting the mask
	if (UGizmoElementBase* HitMaskGizmo = HitMask.Get())
	{
		const FInputRayHit MaskHit = HitMaskGizmo->LineTrace(ViewContext, LineTraceState, RayOrigin, RayDirection, OutLineTraceOutput);
		if (MaskHit.bHit)
		{
			static constexpr float NoPixelHitThreshold = 0.0f;
			UpdatePixelThreshold(ConeElement, NoPixelHitThreshold);
			UpdatePixelThreshold(BoxElement, NoPixelHitThreshold);
			UpdatePixelThreshold(SphereElement, NoPixelHitThreshold);
		}
	}

	auto LineTraceElement = [&](UGizmoElementBase* InElement) -> FInputRayHit
	{
		check(InElement);
		return InElement->LineTrace(ViewContext, CurrentLineTraceState, RayOrigin, RayDirection, OutLineTraceOutput);
	};

	FInputRayHit Hit;
	if (Type == EGizmoElementArrowHeadType::Cone)
	{
		Hit = LineTraceElement(ConeElement);
	}
	else if (Type == EGizmoElementArrowHeadType::Cube)
	{
		Hit = LineTraceElement(BoxElement);
	}
	else if (Type == EGizmoElementArrowHeadType::Sphere)
	{
		Hit = LineTraceElement(SphereElement);
	}

	if (Hit.bHit)
	{
		Hit.SetHitObject(this);
		Hit.HitIdentifier = PartIdentifier;
	}

	return Hit;
}

void UGizmoElementArrowHead::SetCenter(const FVector& InCenter)
{
	if (Center != InCenter)
	{
		Center = InCenter;
		bUpdate = true;
	}
}

FVector UGizmoElementArrowHead::GetCenter() const
{
	return Center;
}

void UGizmoElementArrowHead::SetDirection(const FVector& InDirection)
{
	Direction = InDirection;
	Direction.Normalize();
	bUpdate = true;
}

FVector UGizmoElementArrowHead::GetDirection() const
{
	return Direction;
}

void UGizmoElementArrowHead::SetSideDirection(const FVector& InSideDirection)
{
	SideDirection = InSideDirection;
	SideDirection.Normalize();
	bUpdate = true;
}

FVector UGizmoElementArrowHead::GetSideDirection() const
{
	return SideDirection;
}

void UGizmoElementArrowHead::SetLength(float InLength)
{
	if (Length != InLength)
	{
		Length = InLength;
		bUpdate = true;
	}
}

float UGizmoElementArrowHead::GetLength() const
{
	return Length;
}

void UGizmoElementArrowHead::SetRadius(float InRadius)
{
	if (Radius != InRadius)
	{
		Radius = InRadius;
		bUpdate = true;
	}
}

float UGizmoElementArrowHead::GetRadius() const
{
	return Radius;
}

void UGizmoElementArrowHead::SetNumSides(int32 InNumSides)
{
	if (NumSides != InNumSides)
    {
    	NumSides = InNumSides;
    	bUpdate = true;
    }
}

int32 UGizmoElementArrowHead::GetNumSides() const
{
	return NumSides;
}

void UGizmoElementArrowHead::SetType(EGizmoElementArrowHeadType InHeadType)
{
	if (InHeadType != Type)
	{
		// Removes the given element from the group, if the element is valid
		auto RemoveElementFromGroup = [&](UGizmoElementBase* InElement)
		{
			if (!InElement)
			{
				return;
			}

			UGizmoElementArrowHead::Remove(InElement);
			InElement = nullptr;
		};

		Type = InHeadType;

		if (Type == EGizmoElementArrowHeadType::Cone)
		{
			ConeElement = NewObject<UGizmoElementCone>();
			UGizmoElementArrowHead::Add(ConeElement);

			RemoveElementFromGroup(BoxElement);
			RemoveElementFromGroup(SphereElement);
		}
		else if (Type == EGizmoElementArrowHeadType::Cube)
		{
			BoxElement = NewObject<UGizmoElementBox>();
			UGizmoElementArrowHead::Add(BoxElement);

			RemoveElementFromGroup(ConeElement);
			RemoveElementFromGroup(SphereElement);
		}
		else if (Type == EGizmoElementArrowHeadType::Sphere)
		{
			SphereElement = NewObject<UGizmoElementSphere>();
			UGizmoElementArrowHead::Add(SphereElement);

			RemoveElementFromGroup(BoxElement);
			RemoveElementFromGroup(ConeElement);
		}

		Update();
	}
}

EGizmoElementArrowHeadType UGizmoElementArrowHead::GetType() const
{
	return Type;
}

void UGizmoElementArrowHead::SetPixelHitDistanceThreshold(float InPixelHitDistanceThreshold)
{
	if (PixelHitDistanceThreshold != InPixelHitDistanceThreshold)
	{
		PixelHitDistanceThreshold = InPixelHitDistanceThreshold;
		bUpdate = true;
	}
}

void UGizmoElementArrowHead::SetHitMask(const TWeakObjectPtr<UGizmoElementBase>& InHitMask)
{
	HitMask = InHitMask;
}

void UGizmoElementArrowHead::Update()
{
	static constexpr bool bEndCaps = true;

	auto UpdateCommon = [&](UGizmoElementBase* InElement)
	{
		InElement->SetVertexColor(GetVertexColor());
		InElement->SetHoverVertexColor(GetHoverVertexColor());
		InElement->SetInteractVertexColor(GetInteractVertexColor());
		InElement->SetSelectVertexColor(GetSelectVertexColor());
		InElement->SetSubdueVertexColor(GetSubdueVertexColor());

		InElement->SetMaterial(MeshRenderAttributes.Material.Value);
		InElement->SetHoverMaterial(MeshRenderAttributes.HoverMaterial.Value);
		InElement->SetInteractMaterial(MeshRenderAttributes.InteractMaterial.Value);
		InElement->SetSelectMaterial(MeshRenderAttributes.SelectMaterial.Value);
		InElement->SetSubdueMaterial(MeshRenderAttributes.SubdueMaterial.Value);
	};

	if (Type == EGizmoElementArrowHeadType::Cone)
	{
		if (ensure(ConeElement))
		{
			ConeElement->SetOrigin(Direction * (Length * 0.9f));
			ConeElement->SetDirection(-Direction);
			ConeElement->SetHeight(Length);
			ConeElement->SetRadius(Radius);
			ConeElement->SetNumSides(NumSides);
			ConeElement->SetElementInteractionState(ElementInteractionState);
			ConeElement->SetPixelHitDistanceThreshold(PixelHitDistanceThreshold);
			ConeElement->SetEndCaps(bEndCaps);

			UpdateCommon(ConeElement);
		}
	}
	else if (Type == EGizmoElementArrowHeadType::Cube)
	{
		if (ensure(BoxElement))
		{
			BoxElement->SetCenter(FVector::ZeroVector);
			BoxElement->SetUpDirection(Direction);
			BoxElement->SetSideDirection(SideDirection);
			BoxElement->SetDimensions(FVector(Length, Length, Length));
			BoxElement->SetElementInteractionState(ElementInteractionState);
			BoxElement->SetPixelHitDistanceThreshold(PixelHitDistanceThreshold);

			UpdateCommon(BoxElement);
		}
	}
	else if (Type == EGizmoElementArrowHeadType::Sphere)
	{
		if (ensure(SphereElement))
		{
			SphereElement->SetCenter(FVector::ZeroVector);
			SphereElement->SetRadius(Radius);
			SphereElement->SetNumSides(NumSides);
			SphereElement->SetElementInteractionState(ElementInteractionState);
			SphereElement->SetPixelHitDistanceThreshold(PixelHitDistanceThreshold);

			UpdateCommon(SphereElement);
		}
	}

	bUpdate = false;
}
