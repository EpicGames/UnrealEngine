// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementBase.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementBase)

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementBase, Log, All);

namespace GizmoElementBaseLocals
{
	template <class ViewType>
	void GetViewInfo(const ViewType* View,
		FVector& OutViewLocation, FVector& OutViewDirection, FVector& OutViewUp, bool& OutIsPerspectiveView)
	{
		OutIsPerspectiveView = View->IsPerspectiveProjection();
		OutViewLocation = View->ViewLocation;
		OutViewDirection = View->GetViewDirection();
		OutViewUp = View->GetViewUp();
	}
	
	static TAutoConsoleVariable<bool> CVarAllowFlattenedRendering(
		TEXT("ITF.Gizmos.AllowFlattenedRendering"),
		true,
		TEXT("When enabled, gizmo rendering will be flattened when possible, improving display at very large orthographic zoom levels"),
		ECVF_RenderThreadSafe);
}

bool UGizmoElementBase::GetViewDependentVisibility(const FSceneView* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const
{
	bool bIsPerspectiveView;
	FVector ViewLocation, ViewDirection, ViewUp;
	GizmoElementBaseLocals::GetViewInfo<FSceneView>(View, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
	return GetViewDependentVisibility(ViewLocation, ViewDirection, bIsPerspectiveView, InLocalToWorldTransform, InLocalCenter);
}

bool UGizmoElementBase::GetViewDependentVisibility(const UGizmoViewContext* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const
{
	bool bIsPerspectiveView;
	FVector ViewLocation, ViewDirection, ViewUp;
	GizmoElementBaseLocals::GetViewInfo<UGizmoViewContext>(View, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
	return GetViewDependentVisibility(ViewLocation, ViewDirection, bIsPerspectiveView, InLocalToWorldTransform, InLocalCenter);
}

bool UGizmoElementBase::GetViewAlignRot(const FSceneView* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter, FQuat& OutAlignRot) const
{
	bool bIsPerspectiveView;
	FVector ViewLocation, ViewDirection, ViewUp;
	GizmoElementBaseLocals::GetViewInfo<FSceneView>(View, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
	return GetViewAlignRot(ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView, InLocalToWorldTransform, InLocalCenter, OutAlignRot);
}

bool UGizmoElementBase::GetViewAlignRot(const UGizmoViewContext* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter, FQuat& OutAlignRot) const
{
	bool bIsPerspectiveView;
	FVector ViewLocation, ViewDirection, ViewUp;
	GizmoElementBaseLocals::GetViewInfo<UGizmoViewContext>(View, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
	return GetViewAlignRot(ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView, InLocalToWorldTransform, InLocalCenter, OutAlignRot);
}

bool UGizmoElementBase::GetViewDependentVisibility(const FVector& InViewLocation, const FVector& InViewDirection, bool bInPerspectiveView, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const
{
	if (ViewDependentType == EGizmoElementViewDependentType::None || ViewAlignType == EGizmoElementViewAlignType::PointOnly || ViewAlignType == EGizmoElementViewAlignType::PointEye)
	{
		return true;
	}

	FVector ViewDir;
	if (bInPerspectiveView)
	{
		const FVector WorldCenter = InLocalToWorldTransform.TransformPosition(InLocalCenter);
		ViewDir = WorldCenter - InViewLocation;
	}
	else
	{
		ViewDir = InViewDirection;
	}
	ViewDir.Normalize();

	const FVector WorldViewDependentAxis = InLocalToWorldTransform.TransformVectorNoScale(ViewDependentAxis);
	switch (ViewDependentType)
	{
		case EGizmoElementViewDependentType::None:
			return true;
		case EGizmoElementViewDependentType::Axis:
			return FMath::Abs(FVector::DotProduct(WorldViewDependentAxis, ViewDir)) < DefaultViewDependentAxialMaxCosAngleTol;
		case EGizmoElementViewDependentType::Plane:	
			return FMath::Abs(FVector::DotProduct(WorldViewDependentAxis, ViewDir)) > DefaultViewDependentPlanarMinCosAngleTol;
		default:
			// check that we didn't had any new type
			ensure(false);
			break;
	}

	return true;
}

bool UGizmoElementBase::GetViewAlignRot(const FVector& InViewLocation, const FVector& InViewDirection, const FVector& InViewUp, bool bInPerspectiveView, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter, FQuat& OutAlignRot) const
{
	FVector Scale = InLocalToWorldTransform.GetScale3D();

	if (ViewAlignType == EGizmoElementViewAlignType::None || !VerifyUniformScale(Scale))
	{
		return false;
	}

	FVector LocalViewDir;

	FTransform WorldToLocalTransform = InLocalToWorldTransform.Inverse();

	if (bInPerspectiveView && ViewAlignType != EGizmoElementViewAlignType::PointScreen)
	{
		FVector LocalViewLocation = WorldToLocalTransform.TransformPosition(InViewLocation);
		LocalViewDir = InLocalCenter - LocalViewLocation;
		LocalViewDir.Normalize();
	}
	else
	{
		FVector WorldViewDir = InViewDirection;
		LocalViewDir = WorldToLocalTransform.GetRotation().RotateVector(WorldViewDir);
		LocalViewDir.Normalize();
	}
		
	if (ViewAlignType == EGizmoElementViewAlignType::PointOnly)
	{
		OutAlignRot = FQuat::FindBetweenNormals(ViewAlignNormal, -LocalViewDir);
	}
	else if (ViewAlignType == EGizmoElementViewAlignType::PointEye || ViewAlignType == EGizmoElementViewAlignType::PointScreen)
	{
		FVector Right = ViewAlignAxis ^ ViewAlignNormal;
		Right.Normalize();
		FVector Up = ViewAlignNormal ^ Right;

		FVector LocalViewUp = WorldToLocalTransform.TransformVectorNoScale(InViewUp);
		FVector TargetFwd = -LocalViewDir;
		FVector TargetRight = LocalViewUp ^ TargetFwd;
		TargetRight.Normalize();
		FVector TargetUp = TargetFwd ^ TargetRight;

		OutAlignRot = GetAlignRotBetweenCoordSpaces(ViewAlignNormal, Right, Up, TargetFwd, TargetRight, TargetUp);
	}
	else if (ViewAlignType == EGizmoElementViewAlignType::Axial)
	{
		// if Axis and Dir are almost coincident, do not adjust the rotation
		if ((FMath::Abs(FVector::DotProduct(ViewAlignAxis, -LocalViewDir))) >= ViewAlignAxialMaxCosAngleTol)
		{
			return false;
		}

		FVector TargetRight = -LocalViewDir ^ ViewAlignAxis;
		TargetRight.Normalize();
		FVector TargetNormal = ViewAlignAxis ^ TargetRight;
		TargetNormal.Normalize();
		OutAlignRot = FQuat::FindBetweenNormals(ViewAlignNormal, TargetNormal);
	}

	return true;
}

void UGizmoElementBase::ApplyViewDepthOffset(const FSceneView* InView, const double& InViewDepthOffset, FTransform& InOutTransform) const
{
	bool bIsPerspectiveView;
	FVector ViewLocation, ViewDirection, ViewUp;
	GizmoElementBaseLocals::GetViewInfo<FSceneView>(InView, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
	ApplyViewDepthOffset(ViewLocation, ViewDirection, bIsPerspectiveView, InViewDepthOffset, InOutTransform);
}

void UGizmoElementBase::ApplyViewDepthOffset(const FVector& InViewLocation, const FVector& InViewDirection, const bool bInIsPerspectiveView, FTransform& InOutTransform) const
{
	ApplyViewDepthOffset(InViewLocation, InViewDirection, bInIsPerspectiveView, ViewDepthOffset, InOutTransform);
}

void UGizmoElementBase::ApplyViewDepthOffset(const FVector& InViewLocation, const FVector& InViewDirection, const bool bInIsPerspectiveView, const double& InViewDepthOffset, FTransform& InOutTransform) const
{
	if (bInIsPerspectiveView)
	{
		// Perspective views need to move the object away from the camera while also scaling so that they have the correct size.
		FVector CameraToObject = InOutTransform.GetLocation() - InViewLocation;
		const double DistanceToObject = CameraToObject.Length();
		
		// Clamp to camera near plane
		constexpr double MinDistance = UE_DOUBLE_KINDA_SMALL_NUMBER;
		if (DistanceToObject < MinDistance)
		{
			// Object is at or behind the camera - move along the forward vector without accounting for scale
			InOutTransform.SetLocation(InViewLocation + InViewDirection * InViewDepthOffset * InOutTransform.GetScale3D().X);
			return;
		}
		
		double TargetDistance = DistanceToObject + InViewDepthOffset;
		if (TargetDistance <= MinDistance)
		{
			// Prevent negative/zero
			TargetDistance = MinDistance;
		}
		
		// Calculate scale factor to maintain constant visual size
		const double ScaleFactor = TargetDistance / DistanceToObject;

		// Calculate the new position that maintains the same apparent position in view
		CameraToObject.Normalize(); // Normalize the vector to get direction

		// Update the transform with new location and scale
		InOutTransform.SetLocation(InViewLocation + CameraToObject * TargetDistance);
		InOutTransform.SetScale3D(InOutTransform.GetScale3D() * ScaleFactor);
	}
	else
	{
		// Orthographic cameras can simply shift the location along the camera forward axis without any scaling applied
		InOutTransform.SetLocation(InOutTransform.GetLocation() + InViewDirection * InViewDepthOffset);
	}
}

bool UGizmoElementBase::VerifyUniformScale(const FVector& Scale) const
{
	if (!FMath::IsNearlyEqual(Scale.X, Scale.Y, KINDA_SMALL_NUMBER) || !FMath::IsNearlyEqual(Scale.X, Scale.Z, KINDA_SMALL_NUMBER))
	{
		// Log one-time warning that non-uniform scale is not currently supported 
		static bool bNonUniformScaleWarning = true;
		if (bNonUniformScaleWarning)
		{
			UE_LOGF(LogGizmoElementBase, Warning, "Gizmo element library view-dependent alignment does not currently support non-uniform scale (%f %f %f).",
				Scale.X, Scale.Y, Scale.Z);
			bNonUniformScaleWarning = false;
		}
		return false;
	}
	return true;
}

FQuat UGizmoElementBase::GetAlignRotBetweenCoordSpaces(FVector SourceForward, FVector SourceRight, FVector SourceUp, FVector TargetForward, FVector TargetRight, FVector TargetUp) const
{
	FMatrix SourceToCanonical(
		FPlane(SourceForward.X, SourceRight.X, SourceUp.X, 0.0),
		FPlane(SourceForward.Y, SourceRight.Y, SourceUp.Y, 0.0),
		FPlane(SourceForward.Z, SourceRight.Z, SourceUp.Z, 0.0),
		FPlane::ZeroVector);

	FMatrix CanonicalToTarget(
		FPlane(TargetForward.X, TargetForward.Y, TargetForward.Z, 0.0),
		FPlane(TargetRight.X, TargetRight.Y, TargetRight.Z, 0.0),
		FPlane(TargetUp.X, TargetUp.Y, TargetUp.Z, 0.0),
		FPlane::ZeroVector);

	FMatrix SourceToTarget = SourceToCanonical * CanonicalToTarget;
	FQuat Result = SourceToTarget.ToQuat();
	Result.Normalize();
	return Result;
}

bool UGizmoElementBase::GetEnabledForInteractionState(const EGizmoElementInteractionState InElementInteractionState) const
{
	if (bEnabled)
	{
		switch (InElementInteractionState)
		{
		case EGizmoElementInteractionState::None:
			return bEnabledForDefaultState;		

		case EGizmoElementInteractionState::Hovering:
			return bEnabledForHoveringState;

		case EGizmoElementInteractionState::Interacting:	
			return bEnabledForInteractingState;

		case EGizmoElementInteractionState::Selected:	
			return bEnabledForSelectedState;

		case EGizmoElementInteractionState::Subdued:	
			return bEnabledForSubduedState;
		}
	}

	return false;
}

bool UGizmoElementBase::GetEnabledForViewProjection(bool bIsPerspectiveProjection) const
{
	if (bEnabled)
	{
		if (bIsPerspectiveProjection)
		{
			return bEnabledForPerspectiveProjection;
		}
		else
		{
			return bEnabledForOrthographicProjection;
		}
	}
	
	return false;
}

void UGizmoElementBase::ForEachSubElement(const TFunctionRef<void(UGizmoElementBase*)>& InFunc)
{
	for (UGizmoElementBase* Element : GetSubElements())
	{
		if (Element)
		{
			InFunc(Element);
		}
	}
}

void UGizmoElementBase::ForEachSubElementRecursive(const TFunctionRef<void(UGizmoElementBase*)>& InFunc) const
{
	TArray<TObjectPtr<UGizmoElementBase>> SubElements;
	if (GetSubElementsRecursive(SubElements))
	{
		for (UGizmoElementBase* Element : SubElements)
		{
			if (Element)
			{
				InFunc(Element);
			}
		}
	}
}

void UGizmoElementBase::ForEachSubElementRecursive(const TFunctionRef<void(UGizmoElementBase*)>& InFunc, const uint32 InPartId) const
{
	TArray<TObjectPtr<UGizmoElementBase>> SubElements;
	if (GetSubElementsRecursive(SubElements, InPartId))
	{
		for (UGizmoElementBase* Element : SubElements)
		{
			if (Element)
			{
				InFunc(Element);
			}
		}
	}
}

TConstArrayView<TObjectPtr<UGizmoElementBase>> UGizmoElementBase::GetSubElements() const
{
	// The default element has no sub-elements, so we do nothing and return and empty result
	return { };
}

bool UGizmoElementBase::GetSubElementsRecursive(TArray<TObjectPtr<UGizmoElementBase>>& OutElements) const
{
	OutElements.Append(GetSubElements());

	for (const TObjectPtr<UGizmoElementBase>& Element : GetSubElements())
	{
		if (Element)
		{
			Element->GetSubElementsRecursive(OutElements);
		}
	}

	return !OutElements.IsEmpty();
}

bool UGizmoElementBase::GetSubElementsRecursive(TArray<TObjectPtr<UGizmoElementBase>>& OutElements, const uint32 InPartId) const
{
	for (const TObjectPtr<UGizmoElementBase>& Element : GetSubElements())
	{
		if (Element)
		{
			if (Element->GetPartIdentifier() == InPartId)
			{
				OutElements.Add(Element);	
			}

			Element->GetSubElementsRecursive(OutElements, InPartId);
		}
	}

	return !OutElements.IsEmpty();
}

bool UGizmoElementBase::IsVisible(
	const FSceneView* View,
	EViewInteractionState InCurrentViewInteractionState,
	EGizmoElementInteractionState InCurrentInteractionState,
	const FTransform& InLocalToWorldTransform,
	const FVector& InLocalCenter) const
{
	return (GetVisibleState() &&
		(!IsVisibleFunction.IsSet() || IsVisibleFunction(View, InCurrentViewInteractionState, InCurrentInteractionState, InLocalToWorldTransform, InLocalCenter)) &&
		GetEnabledForInteractionState(InCurrentInteractionState) &&
		GetEnabledForViewProjection(View->IsPerspectiveProjection()) &&
		GetViewDependentVisibility(View, InLocalToWorldTransform, InLocalCenter));
}

bool UGizmoElementBase::IsVisible(const FSceneView* View, EGizmoElementInteractionState InCurrentInteractionState,
	const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const
{
	// Call with "All" view interaction state, as this (deprecated) call path is unaware of the actual state
	return IsVisible(View, EViewInteractionState::All, InCurrentInteractionState, InLocalToWorldTransform, InLocalCenter);
}

bool UGizmoElementBase::IsHittable(const UGizmoViewContext* ViewContext, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const
{
	return (GetHittableState() &&
		(!IsHittableFunction.IsSet() || IsHittableFunction(ViewContext, InLocalToWorldTransform, InLocalCenter)) &&
		GetEnabledForInteractionState(EGizmoElementInteractionState::None) &&
		GetEnabledForViewProjection(ViewContext->IsPerspectiveProjection()) &&
		(!GetVisibleState() || GetViewDependentVisibility(ViewContext, InLocalToWorldTransform, InLocalCenter)));
}

bool UGizmoElementBase::UpdateRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalCenter, FRenderTraversalState& InOutRenderState)
{
	FQuat AlignRot;
	bool bHasAlignRot;
	return UpdateRenderState(RenderAPI, InLocalCenter, InOutRenderState, bHasAlignRot, AlignRot);
}

bool UGizmoElementBase::UpdateRenderState(IToolsContextRenderAPI* RenderAPI, const FVector & InLocalCenter, FRenderTraversalState & InOutRenderState, bool& bOutHasAlignRot, FQuat& OutAlignRot)
{
	check(RenderAPI);
	const FSceneView* View = RenderAPI->GetSceneView();
	check(View);

	OutAlignRot = FQuat::Identity;
	bOutHasAlignRot = false;

	if (InOutRenderState.InteractionState == EGizmoElementInteractionState::None)
	{
		InOutRenderState.InteractionState = ElementInteractionState;
	}

	InOutRenderState.MeshRenderState.Update(MeshRenderAttributes);

	if (IsVisible(View, RenderAPI->GetViewInteractionState(), InOutRenderState.InteractionState, InOutRenderState.LocalToWorldTransform, InLocalCenter))
	{
		bool bIsPerspectiveView;
		FVector ViewLocation, ViewDirection, ViewUp;
		GizmoElementBaseLocals::GetViewInfo<FSceneView>(View, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
		bOutHasAlignRot = GetViewAlignRot(ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView, InOutRenderState.LocalToWorldTransform, InLocalCenter, OutAlignRot);

		InOutRenderState.LocalToWorldTransform = FTransform(OutAlignRot, InLocalCenter) * InOutRenderState.LocalToWorldTransform;

		if (!FMath::IsNearlyZero(ViewDepthOffset))
		{
			ApplyViewDepthOffset(ViewLocation, ViewDirection, bIsPerspectiveView, InOutRenderState.LocalToWorldTransform);
		}
	
		return true;
	}

	return false;
}

bool UGizmoElementBase::UpdateLineTraceState(const UGizmoViewContext* ViewContext, const FVector& InLocalCenter, FLineTraceTraversalState& InOutRenderState)
{
	FQuat AlignQuat;
	bool bHasAlignRot;
	return UpdateLineTraceState(ViewContext, InLocalCenter, InOutRenderState, bHasAlignRot, AlignQuat);
}

bool UGizmoElementBase::UpdateLineTraceState(const UGizmoViewContext* ViewContext, const FVector& InLocalCenter, FLineTraceTraversalState& InOutLineTraceState, bool& bOutHasAlignRot, FQuat& OutAlignRot)
{
	check(ViewContext);

	OutAlignRot = FQuat::Identity;
	bOutHasAlignRot = false;

	if (IsHittable(ViewContext, InOutLineTraceState.LocalToWorldTransform, InLocalCenter))
	{
		bool bIsPerspectiveView;
		FVector ViewLocation, ViewDirection, ViewUp;
		GizmoElementBaseLocals::GetViewInfo<UGizmoViewContext>(ViewContext, ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView);
		bOutHasAlignRot = GetViewAlignRot(ViewLocation, ViewDirection, ViewUp, bIsPerspectiveView, InOutLineTraceState.LocalToWorldTransform, InLocalCenter, OutAlignRot);

		InOutLineTraceState.LocalToWorldTransform = FTransform(OutAlignRot, InLocalCenter) * InOutLineTraceState.LocalToWorldTransform;

		if (!FMath::IsNearlyZero(ViewDepthOffset))
		{
			ApplyViewDepthOffset(ViewLocation, ViewDirection, bIsPerspectiveView, InOutLineTraceState.LocalToWorldTransform);
		}

		return true;
	}

	return false;
}

FInputRayHit UGizmoElementBase::MakeRayHit(const double InHitDepth, FLineTraceOutput& OutLineTraceOutput)
{
	FInputRayHit Hit(static_cast<float>(InHitDepth));
	Hit.SetHitObject(this);
	Hit.HitIdentifier = PartIdentifier;

	OutLineTraceOutput.HitPriority = HitPriority;

	return Hit;
}

bool UGizmoElementBase::GetVisibleState() const
{
	return (static_cast<uint8>(ElementState) & static_cast<uint8>(EGizmoElementState::Visible));
}

void UGizmoElementBase::FRenderTraversalState::Initialize(const UE::GizmoRenderingUtil::ISceneViewInterface& InView, const FTransform& InTransform, const double InDPIScale)
{
	LocalToWorldTransform = InTransform;
	PixelToWorldScale = UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten(&InView, InTransform.GetLocation(), 0.5, FlattenScale) * InDPIScale;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UGizmoElementBase::FRenderTraversalState::Initialize(const FSceneView* InSceneView, FTransform InTransform, const double InDPIScale)
{
	UE::GizmoRenderingUtil::FSceneViewWrapper Wrapper(*InSceneView);
	Initialize(Wrapper, InTransform, InDPIScale);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FScaleMatrix UGizmoElementBase::FRenderTraversalState::SafeFlattenMatrix() const
{
	return SafeFlattenMatrix(LocalToWorldTransform);
}

FScaleMatrix UGizmoElementBase::FRenderTraversalState::SafeFlattenMatrix(const FTransform& InLocalToWorldTransform) const
{
	if (GizmoElementBaseLocals::CVarAllowFlattenedRendering.GetValueOnAnyThread() && InLocalToWorldTransform.GetRotation().IsIdentity())
	{
		return FScaleMatrix(FlattenScale);
	}
	return FScaleMatrix(FVector::One());
}

void UGizmoElementBase::FLineTraceTraversalState::Initialize(const UGizmoViewContext* InGizmoViewContext, FTransform InTransform)
{
	check(InGizmoViewContext);

	LocalToWorldTransform = InTransform;
	bIsPerspectiveProjection = InGizmoViewContext->IsPerspectiveProjection();

	constexpr bool bUseDPIScaleFromViewContext = true;
	PixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale<double>(InGizmoViewContext, InTransform.GetLocation(), bUseDPIScaleFromViewContext);
}

void UGizmoElementBase::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	// Empty default implementation
}

FInputRayHit UGizmoElementBase::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection)
{
	FLineTraceOutput Unused;
	return LineTrace(ViewContext, LineTraceState, RayOrigin, RayDirection, Unused);
}

void UGizmoElementBase::SetVisibleState(bool bVisible)
{
	ElementState = (bVisible ? ElementState | EGizmoElementState::Visible : ElementState & (~EGizmoElementState::Visible));
}

bool UGizmoElementBase::GetHittableState() const
{
	return (static_cast<uint8>(ElementState) & static_cast<uint8>(EGizmoElementState::Hittable));
}

void UGizmoElementBase::SetHitPriority(const int32 InHitPriority)
{
	HitPriority = InHitPriority;
}

int32 UGizmoElementBase::GetHitPriority() const
{
	return HitPriority;
}

void UGizmoElementBase::SetHittableState(bool bHittable)
{
	ElementState = (bHittable ? ElementState | EGizmoElementState::Hittable : ElementState & (~EGizmoElementState::Hittable));
}

void UGizmoElementBase::SetEnabled(bool InEnabled)
{
	bEnabled = InEnabled;
}

bool UGizmoElementBase::GetEnabled() const
{
	return bEnabled;
}

void UGizmoElementBase::SetEnabledForPerspectiveProjection(bool bInEnabledForPerspectiveProjection)
{
	bEnabledForPerspectiveProjection = bInEnabledForPerspectiveProjection;
}

bool UGizmoElementBase::GetEnabledForPerspectiveProjection()
{
	return bEnabledForPerspectiveProjection;
}

void UGizmoElementBase::SetEnabledInOrthographicProjection(bool bInEnabledForOrthographicProjection)
{
	bEnabledForOrthographicProjection = bInEnabledForOrthographicProjection;
}

bool UGizmoElementBase::GetEnabledInOrthographicProjection()
{
	return bEnabledForOrthographicProjection;
}

void UGizmoElementBase::SetEnabledForDefaultState(bool bInEnabledForDefaultState)
{
	bEnabledForDefaultState = bInEnabledForDefaultState;
}

bool UGizmoElementBase::GetEnabledForDefaultState()
{
	return bEnabledForDefaultState;
}

void UGizmoElementBase::SetEnabledForHoveringState(bool bInEnabledForHoveringState)
{
	bEnabledForHoveringState = bInEnabledForHoveringState;
}

bool UGizmoElementBase::GetEnabledForHoveringState()
{
	return bEnabledForHoveringState;
}

void UGizmoElementBase::SetEnabledForInteractingState(bool bInEnabledForInteractingState)
{
	bEnabledForInteractingState = bInEnabledForInteractingState;
}

bool UGizmoElementBase::GetEnabledForInteractingState()
{
	return bEnabledForInteractingState;
}

void UGizmoElementBase::SetEnabledForSelectedState(bool bInEnabledForSelectedState)
{
	bEnabledForSelectedState = bInEnabledForSelectedState;
}

bool UGizmoElementBase::GetEnabledForSelectedState()
{
	return bEnabledForSelectedState;
}

void UGizmoElementBase::SetEnabledForSubduedState(bool bInEnabledForSubduedState)
{
	bEnabledForSubduedState = bInEnabledForSubduedState;
}

bool UGizmoElementBase::GetEnabledForSubduedState()
{
	return bEnabledForSubduedState;
}

void UGizmoElementBase::SetIsVisibleFunction(
	TUniqueFunction<bool(const FSceneView* InView, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState InCurrentInteractionState, const FTransform&
	InLocalToWorldTransform, const FVector& InLocalCenter)>&& InIsVisibleFunction)
{
	IsVisibleFunction = MoveTemp(InIsVisibleFunction);
}

void UGizmoElementBase::SetIsHittableFunction(
	TUniqueFunction<bool(const UGizmoViewContext* InViewContext, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter)>&& InIsHittableFunction)
{
	IsHittableFunction = MoveTemp(InIsHittableFunction);
}

void UGizmoElementBase::SetPartIdentifier(uint32 InPartIdentifier)
{
	PartIdentifier = InPartIdentifier;
}

uint32 UGizmoElementBase::GetPartIdentifier()
{
	return PartIdentifier;
}

uint32 UGizmoElementBase::GetPartIdentifier() const
{
	return PartIdentifier;
}

void UGizmoElementBase::SetPartIdentifier(uint32 InPartId, const bool bInOverrideUnsetChildren, const bool bInOverrideSet)
{
	if (bInOverrideSet || GetPartIdentifier() == 0)
	{
		// The base implementation has no children, so we just call the default Setter
		SetPartIdentifier(InPartId);	
	}
}

UGizmoElementBase* UGizmoElementBase::FindPartElement(const uint32 InPartId)
{
	if (GetPartIdentifier() == InPartId)
	{
		return this; // Return this element if the identifier matches
	}

	return nullptr;
}

const UGizmoElementBase* UGizmoElementBase::FindPartElement(const uint32 InPartId) const
{
	if (GetPartIdentifier() == InPartId)
	{
		return this; // Return this element if the identifier matches
	}

	return nullptr;
}

void UGizmoElementBase::SetElementState(EGizmoElementState InElementState)
{
	ElementState = InElementState;
}

EGizmoElementState UGizmoElementBase::GetElementState() const
{
	return ElementState;
}

void UGizmoElementBase::SetElementInteractionState(EGizmoElementInteractionState InElementInteractionState)
{
	ElementInteractionState = InElementInteractionState;
}

EGizmoElementInteractionState UGizmoElementBase::GetElementInteractionState() const
{
	return ElementInteractionState;
}


bool UGizmoElementBase::UpdatePartHittableState(bool bHittable, uint32 InPartIdentifier)
{
	return UpdatePartHittableState(bHittable, InPartIdentifier, false);
}

bool UGizmoElementBase::UpdatePartHittableState(bool bHittable, uint32 InPartIdentifier, bool bInAllowMultipleElements)
{
	if (InPartIdentifier != PartIdentifier)
	{
		return false;
	}

	SetHittableState(bHittable);
	return true;
}

TOptional<bool> UGizmoElementBase::GetPartHittableState(uint32 InPartIdentifier) const
{
	return InPartIdentifier == PartIdentifier ? TOptional<bool>(GetHittableState()) : TOptional<bool>();
}

bool UGizmoElementBase::UpdatePartVisibleState(bool bVisible, uint32 InPartIdentifier)
{
	return UpdatePartVisibleState(bVisible, InPartIdentifier, false);
}

bool UGizmoElementBase::UpdatePartVisibleState(bool bVisible, uint32 InPartIdentifier, bool bInAllowMultipleElements)
{
	if (InPartIdentifier != PartIdentifier)
	{
		return false;
	}

	SetVisibleState(bVisible);
	return true;
}

TOptional<bool> UGizmoElementBase::GetPartVisibleState(uint32 InPartIdentifier) const
{
	return InPartIdentifier == PartIdentifier ? TOptional<bool>(GetVisibleState()) : TOptional<bool>();
}

bool UGizmoElementBase::UpdatePartInteractionState(EGizmoElementInteractionState InInteractionState, uint32 InPartIdentifier)
{
	return UpdatePartInteractionState(InInteractionState, InPartIdentifier, false);
}

bool UGizmoElementBase::UpdatePartInteractionState(EGizmoElementInteractionState InInteractionState, uint32 InPartIdentifier, bool bInAllowMultipleElements)
{
	if (InPartIdentifier != PartIdentifier)
	{
		return false;
	}
	
	ElementInteractionState = InInteractionState;
	return true;
}

TOptional<EGizmoElementInteractionState> UGizmoElementBase::GetPartInteractionState(uint32 InPartIdentifier) const
{
	if (InPartIdentifier == PartIdentifier)
	{
		return TOptional<EGizmoElementInteractionState>(GetElementInteractionState());
	}

	return  TOptional<EGizmoElementInteractionState>();
}

void UGizmoElementBase::SetViewDependentType(EGizmoElementViewDependentType InViewDependentType)
{
	ViewDependentType = InViewDependentType;
}

EGizmoElementViewDependentType UGizmoElementBase::GetViewDependentType() const
{
	return ViewDependentType;
}

void UGizmoElementBase::SetViewDependentAngleTol(float InAngleTol)
{
	ViewDependentAngleTol = InAngleTol;
	ViewDependentAxialMaxCosAngleTol = FMath::Abs(FMath::Cos(ViewDependentAngleTol));
	ViewDependentPlanarMinCosAngleTol = FMath::Abs(FMath::Cos(HALF_PI + ViewDependentAngleTol));
}

float UGizmoElementBase::GetViewDependentAngleTol() const
{
	return ViewDependentAngleTol;
}

void UGizmoElementBase::SetViewDependentAxis(FVector InViewDependentAxis)
{
	ViewDependentAxis = InViewDependentAxis;
	ViewDependentAxis.Normalize();
}

FVector UGizmoElementBase::GetViewDependentAxis() const
{
	return ViewDependentAxis;
}

void UGizmoElementBase::SetViewAlignType(EGizmoElementViewAlignType InViewAlignType)
{
	ViewAlignType = InViewAlignType;
}

EGizmoElementViewAlignType UGizmoElementBase::GetViewAlignType() const
{
	return ViewAlignType;
}

void UGizmoElementBase::SetViewAlignAxis(FVector InViewAlignAxis)
{
	ViewAlignAxis = InViewAlignAxis;
	ViewAlignAxis.Normalize();
}

FVector UGizmoElementBase::GetViewAlignAxis() const
{
	return ViewAlignAxis;
}

void UGizmoElementBase::SetViewAlignNormal(FVector InViewAlignNormal)
{
	ViewAlignNormal = InViewAlignNormal;
	ViewAlignNormal.Normalize();
}

FVector UGizmoElementBase::GetViewAlignNormal() const
{
	return ViewAlignNormal;
}

void UGizmoElementBase::SetViewAlignAxialAngleTol(float InAngleTol)
{
	ViewAlignAxialAngleTol = InAngleTol;
	ViewAlignAxialMaxCosAngleTol = FMath::Abs(FMath::Cos(ViewAlignAxialAngleTol));
}

float UGizmoElementBase::GetViewAlignAxialAngleTol() const
{
	return ViewAlignAxialAngleTol;
}

void UGizmoElementBase::SetViewDepthOffset(float InViewDepthOffset)
{
	ViewDepthOffset = InViewDepthOffset;
}

float UGizmoElementBase::GetViewDepthOffset() const
{
	return ViewDepthOffset;
}

void UGizmoElementBase::SetPixelHitDistanceThreshold(float InPixelHitDistanceThreshold)
{
	PixelHitDistanceThreshold = InPixelHitDistanceThreshold;
}

float UGizmoElementBase::GetPixelHitDistanceThreshold() const
{
	return PixelHitDistanceThreshold;
}

void UGizmoElementBase::SetMinimumPixelHitDistanceThreshold(const float InMinimumPixelHitDistanceThreshold)
{
	MinimumPixelHitDistanceThreshold = InMinimumPixelHitDistanceThreshold;
}

float UGizmoElementBase::GetMinimumPixelHitDistanceThreshold() const
{
	return MinimumPixelHitDistanceThreshold;
}

void UGizmoElementBase::SetMaterial(TWeakObjectPtr<UMaterialInterface> InMaterial, bool InOverridesChildState)
{
	MeshRenderAttributes.Material.SetMaterial(InMaterial, InOverridesChildState);
}

const UMaterialInterface* UGizmoElementBase::GetMaterial() const
{
	return MeshRenderAttributes.Material.GetMaterial();
}

UMaterialInterface* UGizmoElementBase::GetMaterial()
{
	return MeshRenderAttributes.Material.GetMaterial();
}

bool UGizmoElementBase::DoesMaterialOverrideChildState() const
{
	return MeshRenderAttributes.Material.bOverridesChildState;
}

void UGizmoElementBase::ClearMaterial()
{
	MeshRenderAttributes.Material.Reset();
}

void UGizmoElementBase::SetHoverMaterial(TWeakObjectPtr<UMaterialInterface> InMaterial, bool InOverridesChildState)
{
	MeshRenderAttributes.HoverMaterial.SetMaterial(InMaterial, InOverridesChildState);
}

const UMaterialInterface* UGizmoElementBase::GetHoverMaterial() const
{
	return MeshRenderAttributes.HoverMaterial.GetMaterial();
}

UMaterialInterface* UGizmoElementBase::GetHoverMaterial()
{
	return MeshRenderAttributes.HoverMaterial.GetMaterial();
}

bool UGizmoElementBase::DoesHoverMaterialOverrideChildState() const
{
	return MeshRenderAttributes.HoverMaterial.bOverridesChildState;
}

void UGizmoElementBase::ClearHoverMaterial()
{
	MeshRenderAttributes.HoverMaterial.Reset();
}

void UGizmoElementBase::SetInteractMaterial(TWeakObjectPtr<UMaterialInterface> InMaterial, bool InOverridesChildState)
{
	MeshRenderAttributes.InteractMaterial.SetMaterial(InMaterial, InOverridesChildState);
}

const UMaterialInterface* UGizmoElementBase::GetInteractMaterial() const
{
	return MeshRenderAttributes.InteractMaterial.GetMaterial();
}

UMaterialInterface* UGizmoElementBase::GetInteractMaterial()
{
	return MeshRenderAttributes.InteractMaterial.GetMaterial();
}

bool UGizmoElementBase::DoesInteractMaterialOverrideChildState() const
{
	return MeshRenderAttributes.InteractMaterial.bOverridesChildState;
}

void UGizmoElementBase::ClearInteractMaterial()
{
	MeshRenderAttributes.InteractMaterial.Reset();
}

void UGizmoElementBase::SetSelectMaterial(TWeakObjectPtr<UMaterialInterface> InSelectMaterial, bool InOverridesChildState)
{
	MeshRenderAttributes.SelectMaterial.SetMaterial(InSelectMaterial, InOverridesChildState);
}

const UMaterialInterface* UGizmoElementBase::GetSelectMaterial() const
{
	return MeshRenderAttributes.SelectMaterial.GetMaterial();
}

UMaterialInterface* UGizmoElementBase::GetSelectMaterial()
{
	return MeshRenderAttributes.SelectMaterial.GetMaterial();
}

bool UGizmoElementBase::DoesSelectMaterialOverrideChildState() const
{
	return MeshRenderAttributes.SelectMaterial.bOverridesChildState;
}

void UGizmoElementBase::ClearSelectMaterial()
{
	MeshRenderAttributes.SelectMaterial.Reset();
}

void UGizmoElementBase::SetSubdueMaterial(TWeakObjectPtr<UMaterialInterface> InSubdueMaterial, bool InOverridesChildState)
{
	MeshRenderAttributes.SubdueMaterial.SetMaterial(InSubdueMaterial, InOverridesChildState);
}

const UMaterialInterface* UGizmoElementBase::GetSubdueMaterial() const
{
	return MeshRenderAttributes.SubdueMaterial.GetMaterial();
}

UMaterialInterface* UGizmoElementBase::GetSubdueMaterial()
{
	return MeshRenderAttributes.SubdueMaterial.GetMaterial();
}

bool UGizmoElementBase::DoesSubdueMaterialOverrideChildState() const
{
	return MeshRenderAttributes.SubdueMaterial.bOverridesChildState;
}

void UGizmoElementBase::ClearSubdueMaterial()
{
	MeshRenderAttributes.SubdueMaterial.Reset();
}

void UGizmoElementBase::SetVertexColor(FLinearColor InVertexColor, bool InOverridesChildState)
{
	MeshRenderAttributes.VertexColor.SetColor(InVertexColor, InOverridesChildState);
}

FLinearColor UGizmoElementBase::GetVertexColor() const
{
	return MeshRenderAttributes.VertexColor.GetColor();
}

bool UGizmoElementBase::HasVertexColor() const
{
	return MeshRenderAttributes.VertexColor.bHasValue;
}

bool UGizmoElementBase::DoesVertexColorOverrideChildState() const
{
	return MeshRenderAttributes.VertexColor.bOverridesChildState;
}

void UGizmoElementBase::ClearVertexColor()
{
	MeshRenderAttributes.VertexColor.Reset();
}

void UGizmoElementBase::SetHoverVertexColor(FLinearColor InVertexColor, bool InOverridesChildState)
{
	MeshRenderAttributes.HoverVertexColor.SetColor(InVertexColor, InOverridesChildState);
}

FLinearColor UGizmoElementBase::GetHoverVertexColor() const
{
	return MeshRenderAttributes.HoverVertexColor.GetColor();
}

bool UGizmoElementBase::HasHoverVertexColor() const
{
	return MeshRenderAttributes.HoverVertexColor.bHasValue;
}

bool UGizmoElementBase::DoesHoverVertexColorOverrideChildState() const
{
	return MeshRenderAttributes.HoverVertexColor.bOverridesChildState;
}

void UGizmoElementBase::ClearHoverVertexColor()
{
	MeshRenderAttributes.HoverVertexColor.Reset();
}

void UGizmoElementBase::SetInteractVertexColor(FLinearColor InInteractVertexColor, bool InOverridesChildState)
{
	MeshRenderAttributes.InteractVertexColor.SetColor(InInteractVertexColor, InOverridesChildState);
}

FLinearColor UGizmoElementBase::GetInteractVertexColor() const
{
	return MeshRenderAttributes.InteractVertexColor.GetColor();
}

bool UGizmoElementBase::HasInteractVertexColor() const
{
	return MeshRenderAttributes.InteractVertexColor.bHasValue;
}

bool UGizmoElementBase::DoesInteractVertexColorOverrideChildState() const
{
	return MeshRenderAttributes.InteractVertexColor.bOverridesChildState;
}

void UGizmoElementBase::ClearInteractVertexColor()
{
	MeshRenderAttributes.InteractVertexColor.Reset();
}

void UGizmoElementBase::SetSelectVertexColor(FLinearColor InVertexColor, bool InOverridesChildState)
{
	MeshRenderAttributes.SelectVertexColor.SetColor(InVertexColor, InOverridesChildState);
}

FLinearColor UGizmoElementBase::GetSelectVertexColor() const
{
	return MeshRenderAttributes.SelectVertexColor.GetColor();
}

bool UGizmoElementBase::HasSelectVertexColor() const
{
	return MeshRenderAttributes.SelectVertexColor.bHasValue;
}

bool UGizmoElementBase::DoesSelectVertexColorOverrideChildState() const
{
	return MeshRenderAttributes.SelectVertexColor.bOverridesChildState;
}

void UGizmoElementBase::ClearSelectVertexColor()
{
	MeshRenderAttributes.SelectVertexColor.Reset();
}

void UGizmoElementBase::SetSubdueVertexColor(FLinearColor InVertexColor, bool InOverridesChildState)
{
	MeshRenderAttributes.SubdueVertexColor.SetColor(InVertexColor, InOverridesChildState);
}

FLinearColor UGizmoElementBase::GetSubdueVertexColor() const
{
	return MeshRenderAttributes.SubdueVertexColor.GetColor();
}

bool UGizmoElementBase::HasSubdueVertexColor() const
{
	return MeshRenderAttributes.SubdueVertexColor.bHasValue;
}

bool UGizmoElementBase::DoesSubdueVertexColorOverrideChildState() const
{
	return MeshRenderAttributes.SubdueVertexColor.bOverridesChildState;
}

void UGizmoElementBase::ClearSubdueVertexColor()
{
	MeshRenderAttributes.SubdueVertexColor.Reset();
}
