// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementBox.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "Materials/MaterialInterface.h"
#include "InputState.h"
#include "Intersection/IntrRay3OrientedBox3.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementBox)

void UGizmoElementBox::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			FQuat LocalRotation = FRotationMatrix::MakeFromYZ(SideDirection, UpDirection).ToQuat();
			FTransform RenderLocalToWorldTransform = FTransform(LocalRotation) * CurrentRenderState.LocalToWorldTransform;
			const FVector HalfDimensions = Dimensions * 0.5;
			FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

			// Applies flatten scale in local space (so it doesn't affect translation)
			FMatrix RenderLocalToWorldMatrix = RenderLocalToWorldTransform.ToMatrixWithScale();
			RenderLocalToWorldMatrix = RenderLocalToWorldMatrix * CurrentRenderState.SafeFlattenMatrix();
			RenderLocalToWorldMatrix.SetOrigin(RenderLocalToWorldTransform.GetTranslation()); // Restore origin

			DrawBox(PDI, RenderLocalToWorldMatrix, HalfDimensions, UseMaterial->GetRenderProxy(), SDPG_Foreground);
		}
	}
}

FInputRayHit UGizmoElementBox::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		const FVector YAxis = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(SideDirection);
		const FVector ZAxis = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(UpDirection);
		const FVector XAxis = FVector::CrossProduct(YAxis, ZAxis);
		const FVector WorldCenter = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
		const double Scale = CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X;
		double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		const FVector WorldExtent = Dimensions * Scale * 0.5;

		double HitDepth = 0.0;
		UE::Geometry::TRay<double> Ray(RayOrigin, RayDirection);
		UE::Geometry::TFrame3<double> Frame(WorldCenter, XAxis, YAxis, ZAxis);
		UE::Geometry::TOrientedBox3<double> Box(Frame, WorldExtent + PixelHitThresholdAdjust);
		if (UE::Geometry::TIntrRay3OrientedBox3<double>::FindIntersection(Ray, Box, HitDepth))
		{
			FInputRayHit RayHit = MakeRayHit(HitDepth, OutLineTraceOutput);

			// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold) ...
			PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
			Box = UE::Geometry::TOrientedBox3<double>(Frame, WorldExtent + PixelHitThresholdAdjust); // Recreate the box with MinimumPixelHitDistanceThreshold
			if (UE::Geometry::TIntrRay3OrientedBox3<double>::FindIntersection(Ray, Box, HitDepth))
			{
				RayHit.HitDepth = HitDepth;
				OutLineTraceOutput.bIsSurfaceHit = true;
			}

			return RayHit;
		}
	}

	return FInputRayHit();
}

void UGizmoElementBox::SetCenter(const FVector& InCenter)
{
	Center = InCenter;
}

FVector UGizmoElementBox::GetCenter() const
{
	return Center;
}

void UGizmoElementBox::SetUpDirection(const FVector& InUpDirection)
{
	UpDirection = InUpDirection;
	UpDirection.Normalize();
}

FVector UGizmoElementBox::GetUpDirection() const
{
	return UpDirection;
}

void UGizmoElementBox::SetSideDirection(const FVector& InSideDirection)
{
	SideDirection = InSideDirection;
	SideDirection.Normalize();
}

FVector UGizmoElementBox::GetSideDirection() const
{
	return SideDirection;
}

FVector UGizmoElementBox::GetDimensions() const
{
	return Dimensions;
}

void UGizmoElementBox::SetDimensions(const FVector& InDimensions)
{
	Dimensions = InDimensions;
}


