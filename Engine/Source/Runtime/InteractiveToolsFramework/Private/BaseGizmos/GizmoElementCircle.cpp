// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementCircle.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "DynamicMeshBuilder.h"
#include "InputState.h"
#include "Intersection/IntersectionUtil.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementCircle)

namespace UE::InteractiveToolsFramework::Private
{
	namespace GizmoElementCircleLocals
	{
		// copy/paste of DrawDisc from PrimitiveDrawingUtils.h, with corrected (flat) normals
		void DrawDisc(FPrimitiveDrawInterface* PDI, const FMatrix& InLocalToWorld, FColor Color, double Radius, int32 NumSides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority)
		{
			check(NumSides >= 3);

			const float AngleDelta = 2.0f * UE_PI / NumSides;

			const FVector2D TC = FVector2D(0.0f, 0.0f);
			const float TCStep = 1.0f / NumSides;

			const FVector AxisX = FVector::RightVector;
			const FVector AxisY = FVector::UpVector;
			const FVector AxisZ = (AxisX) ^ AxisY;

			const FVector3f TangentX = static_cast<FVector3f>(AxisX);
			const FVector3f TangentY = static_cast<FVector3f>(-AxisY);
			const FVector3f TangentZ = static_cast<FVector3f>(AxisZ);

			FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

			// Add center vertex for a proper triangle fan.
			FDynamicMeshVertex CenterVertex;
			CenterVertex.Position = FVector3f::ZeroVector;
			CenterVertex.Color = Color;
			CenterVertex.TextureCoordinate[0] = FVector2f(TC);
			CenterVertex.SetTangents(
				TangentX,
				TangentY,
				TangentZ
			);
			const int32 CenterIndex = MeshBuilder.AddVertex(CenterVertex);
			const int32 PerimeterStartIndex = CenterIndex + 1;

			//Compute vertices for base circle.
			for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
			{
				const FVector Vertex = (AxisX * FMath::Cos(AngleDelta * (SideIndex)) + AxisY * FMath::Sin(AngleDelta * (SideIndex))) * Radius;

				FDynamicMeshVertex MeshVertex;
				MeshVertex.Position = static_cast<FVector3f>(Vertex);
				MeshVertex.Color = Color;
				MeshVertex.TextureCoordinate[0] = FVector2f(TC);
				MeshVertex.TextureCoordinate[0].X += TCStep * SideIndex;

				MeshVertex.SetTangents(
					TangentX,
					TangentY,
					TangentZ
				);

				MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
			}

			//Add top/bottom triangles, in the style of a fan.
			for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
			{
				const int32 V0 = CenterIndex;
				const int32 V1 = PerimeterStartIndex + SideIndex;
				const int32 V2 = PerimeterStartIndex + ((SideIndex + 1) % NumSides);

				MeshBuilder.AddTriangle(V0, V1, V2);
				MeshBuilder.AddTriangle(V0, V2, V1);
			}

			MeshBuilder.Draw(PDI, InLocalToWorld, MaterialRenderProxy, DepthPriority,0.f);
		}
	}
}

void UGizmoElementCircle::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}

	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		const double WorldRadius = GetCurrentRadius() * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;
		const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
		const FVector WorldAxis0 = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Axis0);
		const FVector WorldAxis1 = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Axis1);
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

		const FSceneView* View = RenderAPI->GetSceneView();
		if (!ensure(View))
		{
			return;
		}

		const float CurrentLineThickness = GetCurrentLineThickness(View->IsPerspectiveProjection(), View->FOV);

		if (bDrawMesh)
		{
			if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
			{
				using namespace UE::InteractiveToolsFramework::Private;

				// Offset in view-space to avoid z-fighting with the line.
				// Note: UpdateRenderState has already applied ViewDepthOffset to CurrentRenderState.LocalToWorldTransform,
				// so we only need to apply the additional offset to push the mesh behind the line.
				if (bDrawLine)
				{
					const double MeshViewDepthOffset = -CurrentLineThickness;
					ApplyViewDepthOffset(View, MeshViewDepthOffset, CurrentRenderState.LocalToWorldTransform);
				}

				// Rotate to match Axis0, Axis1
				const FMatrix AxisRotation = FRotationMatrix::MakeFromYZ(Axis0, Axis1);
				const FMatrix RenderLocalToWorldMatrix = AxisRotation * CurrentRenderState.LocalToWorldTransform.ToMatrixNoScale();

				const FColor VertexColor = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);

				GizmoElementCircleLocals::DrawDisc(PDI, RenderLocalToWorldMatrix, VertexColor, WorldRadius, NumSegments, UseMaterial->GetRenderProxy(), SDPG_Foreground);
			}
		}

		if (bDrawLine)
		{
			// Boost depth bias for lines to avoid z-fighting with the fill (of this or other elements)
			const float LineDepthBias = 1.0f + CurrentLineThickness;

			DrawCircle(PDI, WorldCenter, WorldAxis0, WorldAxis1, CurrentRenderState.GetCurrentLineColor(), WorldRadius, NumSegments, SDPG_Foreground, CurrentLineThickness, LineDepthBias, bScreenSpaceLine);
		}

	}
}

FInputRayHit UGizmoElementCircle::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		const FVector WorldCenter = CurrentLineTraceState.LocalToWorldTransform.GetLocation();
		const FVector WorldAxis0 = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Axis0);
		const FVector WorldAxis1 = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Axis1);
		const FVector WorldNormal = WorldAxis0 ^ WorldAxis1;
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		const double MinimumPixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
		double WorldRadius = CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X * Radius;

		// if ray is parallel to circle, no hit
		if (FMath::IsNearlyZero(FVector::DotProduct(WorldNormal, RayDirection)))
		{
			return FInputRayHit();
		}

		if (bHitMesh)
		{
			WorldRadius += PixelHitThresholdAdjust;

			UE::Geometry::FLinearIntersection Result;
			IntersectionUtil::RayCircleIntersection(RayOrigin, RayDirection, WorldCenter, WorldRadius, WorldNormal, Result);

			if (Result.intersects)
			{
				FInputRayHit RayHit = MakeRayHit(Result.parameter.Min, OutLineTraceOutput);

				// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold)
				WorldRadius -= PixelHitThresholdAdjust;
				WorldRadius += MinimumPixelHitThresholdAdjust;
				IntersectionUtil::RayCircleIntersection(RayOrigin, RayDirection, WorldCenter, WorldRadius, WorldNormal, Result);
				OutLineTraceOutput.bIsSurfaceHit = Result.intersects;

				return RayHit;
			}
		}
		else if (bHitLine)
		{
			FPlane Plane(WorldCenter, WorldNormal);
			double HitDepth = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);
			if (HitDepth < UE_DOUBLE_SMALL_NUMBER)
			{
				return FInputRayHit();
			}

			FVector HitPoint = RayOrigin + RayDirection * HitDepth;

			FVector NearestCirclePos;
			GizmoMath::ClosetPointOnCircle(HitPoint, WorldCenter, WorldNormal, static_cast<float>(WorldRadius), NearestCirclePos);

			FRay Ray(RayOrigin, RayDirection, true);
			FVector NearestRayPos = Ray.ClosestPoint(NearestCirclePos);

			const double HitBuffer = PixelHitThresholdAdjust + LineThickness;
			double Distance = FVector::Distance(NearestCirclePos, NearestRayPos);
			
			if (Distance <= HitBuffer)
			{
				FInputRayHit RayHit = MakeRayHit(HitDepth, OutLineTraceOutput);

				// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold)
				OutLineTraceOutput.bIsSurfaceHit = Distance < FMath::Max(CurrentLineTraceState.PixelToWorldScale, MinimumPixelHitThresholdAdjust + LineThickness);

				return RayHit;
			}
		}
	}

	return FInputRayHit();
}

void UGizmoElementCircle::SetDrawMesh(bool InDrawMesh)
{
	bDrawMesh = InDrawMesh;
}

bool UGizmoElementCircle::GetDrawMesh() const
{
	return bDrawMesh;
}

void UGizmoElementCircle::SetDrawLine(bool InDrawLine)
{
	bDrawLine = InDrawLine;
}

bool UGizmoElementCircle::GetDrawLine() const
{
	return bDrawLine;
}

void UGizmoElementCircle::SetHitMesh(bool InHitMesh)
{
	bHitMesh = InHitMesh;
}

bool UGizmoElementCircle::GetHitMesh() const
{
	return bHitMesh;
}

void UGizmoElementCircle::SetHitLine(bool InHitLine)
{
	bHitLine = InHitLine;
}

bool UGizmoElementCircle::GetHitLine() const
{
	return bHitLine;
}
