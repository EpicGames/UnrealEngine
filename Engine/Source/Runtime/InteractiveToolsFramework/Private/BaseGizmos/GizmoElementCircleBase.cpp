// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementCircleBase.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementCircleBase)

double UGizmoElementCircleBase::GetCurrentRadius() const
{
	auto GetMultipliedRadius = [this](const float InMultiplier) -> float
	{
		return (Radius > UE_SMALL_NUMBER ? Radius * InMultiplier : InMultiplier);
	};

	return GetMultipliedRadius(RadiusMultiplier.GetValueForState(ElementInteractionState));
}

bool UGizmoElementCircleBase::IsPartial(const FSceneView* View, const FVector& InWorldCenter, const FVector& InWorldNormal)
{
	return IsPartial(InWorldCenter, InWorldNormal, View->ViewLocation, View->GetViewDirection(),
		View->IsPerspectiveProjection());
}

bool UGizmoElementCircleBase::IsPartial(const UGizmoViewContext* View, const FVector& InWorldCenter, const FVector& InWorldNormal)
{
	return IsPartial(InWorldCenter, InWorldNormal, View->ViewLocation, View->GetViewDirection(),
		View->IsPerspectiveProjection());
}

bool UGizmoElementCircleBase::IsPartial(const FVector& InWorldCenter, const FVector& InWorldNormal,
	const FVector& InViewLocation, const FVector& InViewDirection, const bool bIsPerspectiveProjection)
{
	switch (PartialType)
	{
		case EGizmoElementPartialType::None:
			return false;
		case EGizmoElementPartialType::Partial:
			return true;
		case EGizmoElementPartialType::PartialViewDependent:
		{
			FVector ViewToCircleBaseCenter(InViewDirection);
			if (bIsPerspectiveProjection)
			{
				ViewToCircleBaseCenter = (InWorldCenter - InViewLocation).GetSafeNormal();
			}
			const double DotP = FMath::Abs(FVector::DotProduct(InWorldNormal, ViewToCircleBaseCenter));
			return (DotP <= PartialViewDependentMaxCosTol);
		}
		default:
			break;
	}

	return false;
}

void UGizmoElementCircleBase::SetCenter(const FVector& InCenter)
{
	Center = InCenter;
}

FVector UGizmoElementCircleBase::GetCenter() const
{
	return Center;
}

void UGizmoElementCircleBase::SetAxis0(const FVector& InAxis0)
{
	SetAxisTangent(InAxis0);
}

void UGizmoElementCircleBase::SetAxisBitangent(const FVector& InAxisTangent)
{
	Axis0 = InAxisTangent.GetSafeNormal();
}

FVector UGizmoElementCircleBase::GetAxis0() const
{
	return Axis1;
}

FVector UGizmoElementCircleBase::GetAxisBitangent() const
{
	return Axis0;
}

void UGizmoElementCircleBase::SetAxis1(const FVector& InAxis1)
{
	SetAxisTangent(InAxis1);
}

void UGizmoElementCircleBase::SetAxisTangent(const FVector& InAxisBitangent)
{
	Axis1 = InAxisBitangent.GetSafeNormal();
}

FVector UGizmoElementCircleBase::GetAxis1() const
{
	return Axis0;
}

FVector UGizmoElementCircleBase::GetAxisTangent() const
{
	return Axis1;
}

void UGizmoElementCircleBase::SetRadius(double InRadius)
{
	Radius = InRadius;
}

double UGizmoElementCircleBase::GetRadius() const
{
	return Radius;
}

void UGizmoElementCircleBase::SetHoverRadiusMultiplier(double InHoverRadiusMultiplier)
{
	RadiusMultiplier.Hover = InHoverRadiusMultiplier;
}

double UGizmoElementCircleBase::GetHoverRadiusMultiplier() const
{
	return RadiusMultiplier.GetHoverValue();
}

void UGizmoElementCircleBase::SetInteractRadiusMultiplier(double InInteractRadiusMultiplier)
{
	RadiusMultiplier.Interact = InInteractRadiusMultiplier;
}

double UGizmoElementCircleBase::GetInteractRadiusMultiplier() const
{
	return RadiusMultiplier.GetInteractValue();
}

void UGizmoElementCircleBase::SetSelectRadiusMultiplier(double InSelectRadiusMultiplier)
{
	RadiusMultiplier.Select = InSelectRadiusMultiplier;
}

double UGizmoElementCircleBase::GetSelectRadiusMultiplier() const
{
	return RadiusMultiplier.GetSelectValue();
}

void UGizmoElementCircleBase::SetSubdueRadiusMultiplier(double InSubdueRadiusMultiplier)
{
	RadiusMultiplier.Subdue = InSubdueRadiusMultiplier;
}

double UGizmoElementCircleBase::GetSubdueRadiusMultiplier() const
{
	return RadiusMultiplier.GetSubdueValue();
}

void UGizmoElementCircleBase::SetNumSegments(int32 InNumSegments)
{
	NumSegments = InNumSegments;
}

int32 UGizmoElementCircleBase::GetNumSegments() const
{
	return NumSegments;
}

void UGizmoElementCircleBase::SetPartialType(EGizmoElementPartialType InPartialType)
{
	PartialType = InPartialType;
}

EGizmoElementPartialType UGizmoElementCircleBase::GetPartialType() const
{
	return PartialType;
}

void UGizmoElementCircleBase::SetPartialStartAngle(double InPartialStartAngle)
{
	PartialStartAngle = InPartialStartAngle;
}

double UGizmoElementCircleBase::GetPartialStartAngle() const
{
	return PartialStartAngle;
}

void UGizmoElementCircleBase::SetPartialEndAngle(double InPartialEndAngle)
{
	PartialEndAngle = InPartialEndAngle;
}

double UGizmoElementCircleBase::GetPartialEndAngle() const
{
	return PartialEndAngle;
}

void UGizmoElementCircleBase::SetPartialViewDependentMaxCosTol(double InPartialViewDependentMaxCosTol)
{
	PartialViewDependentMaxCosTol = InPartialViewDependentMaxCosTol;
}

double UGizmoElementCircleBase::GetPartialViewDependentMaxCosTol() const
{
	return PartialViewDependentMaxCosTol;
}
