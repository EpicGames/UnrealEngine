// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementCone.h"

#include "GizmoPrivateUtil.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "DynamicMeshBuilder.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"

namespace GizmoConeComponentLocals
{
	// Custom DrawDisc with flat/constant normal
	void DrawDisc(FPrimitiveDrawInterface* InPDI, const FMatrix& InLocalToWorld, double InRadius, int32 InNumSides, const UMaterialInterface* InMaterial)
	{
		check(InNumSides >= 3);

		const float	AngleDelta = 2.0f * UE_PI / InNumSides;

		const FVector2f TextureCoordinate = FVector2f(0.0f, 0.0f);
		const FVector2f::FReal TextureCoordinateStep = 1.0f / InNumSides;

		FDynamicMeshBuilder MeshBuilder(InPDI->View->GetFeatureLevel());
		
		const FVector AxisX = FVector::RightVector;
		const FVector AxisY = FVector::UpVector;
		const FVector AxisZ = AxisX ^ AxisY;

		// Compute vertices for base circle.
		for (int32 SideIndex = 0; SideIndex < InNumSides; ++SideIndex)
		{
			const FVector Vertex = (AxisX * FMath::Cos(AngleDelta * (SideIndex)) + AxisY * FMath::Sin(AngleDelta * (SideIndex))) * InRadius;

			FDynamicMeshVertex MeshVertex;
			MeshVertex.Position = static_cast<FVector3f>(Vertex);
			MeshVertex.Color = FColor::White;
			MeshVertex.TextureCoordinate[0] = FVector2f(TextureCoordinate);
			MeshVertex.TextureCoordinate[0].X += TextureCoordinateStep * SideIndex;

			MeshVertex.SetTangents(
			FVector3f(AxisY),
			FVector3f(-AxisX),
			FVector3f(AxisZ)
			);

			MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
		}

		// Add top/bottom triangles, in the style of a fan.
		for (int32 SideIndex = 0; SideIndex < InNumSides - 1; ++SideIndex)
		{
			constexpr int32 V0 = 0;
			const int32 V1 = SideIndex;
			const int32 V2 = (SideIndex + 1);

			MeshBuilder.AddTriangle(V0, V1, V2);
			MeshBuilder.AddTriangle(V0, V2, V1);
		}

		MeshBuilder.Draw(InPDI, InLocalToWorld, InMaterial->GetRenderProxy(), SDPG_Foreground,0.0f);
	}
}

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementCone)

void UGizmoElementCone::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Origin, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			FQuat Rotation = FRotationMatrix::MakeFromX(Direction).ToQuat();
			const FVector Scale(Height);
	
			FTransform RenderLocalToWorldTransform = FTransform(Rotation, FVector::ZeroVector, Scale) * CurrentRenderState.LocalToWorldTransform;
			const float ConeSide = FMath::Sqrt(Height * Height + Radius * Radius);
			const float HeightOverConeSide = Height / ConeSide;
			const float Angle = FMath::Acos(HeightOverConeSide);
			FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

			// Applies flatten scale in local space (so it doesn't affect translation)
			FMatrix RenderLocalToWorldMatrix = RenderLocalToWorldTransform.ToMatrixWithScale();
			RenderLocalToWorldMatrix = RenderLocalToWorldMatrix * CurrentRenderState.SafeFlattenMatrix();
			RenderLocalToWorldMatrix.SetOrigin(RenderLocalToWorldTransform.GetTranslation()); // Restore origin

			DrawCone(PDI, RenderLocalToWorldMatrix, Angle, Angle, NumSides, false, FColor::White, UseMaterial->GetRenderProxy(), SDPG_Foreground);
	
			if (bEndCaps)
			{
				// The resulting length of the arrow/cone is altered according to the angle, so we need to account for it here to get an accurate bottom location
				const float Offset = Height * HeightOverConeSide;
				const FVector Location = CurrentRenderState.LocalToWorldTransform.TransformPosition(Direction * Offset);
				RenderLocalToWorldMatrix.RemoveScaling();
				RenderLocalToWorldMatrix.SetOrigin(Location);
				GizmoConeComponentLocals::DrawDisc(PDI, RenderLocalToWorldMatrix, Radius * HeightOverConeSide * CurrentRenderState.LocalToWorldTransform.GetScale3D().X, NumSides, UseMaterial);
			}
		}
	}
}

FInputRayHit UGizmoElementCone::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Origin, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		bool bIntersects = false;
		double RayParam = 0.0;

		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		const double ConeSide = FMath::Sqrt(Height * Height + Radius * Radius);
		const double CosAngle = Height / ConeSide;
		double WorldHeight = Height * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust * 2.0;
		const FVector WorldDirection = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Direction);
		FVector WorldOrigin = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector) - WorldDirection * PixelHitThresholdAdjust;

		// due to numerical imprecision, the ray origin needs to be clamped in ortho views
		// (cf. UEditorInteractiveToolsContext::GetRayFromMousePos)
		FVector ClampedRayOrigin(RayOrigin);
		const double DepthBias = UE::GizmoUtil::ClampRayOrigin(ViewContext, ClampedRayOrigin, RayDirection);

		GizmoMath::RayConeIntersection(
			WorldOrigin,
			WorldDirection,
			CosAngle,
			WorldHeight,
			ClampedRayOrigin, RayDirection,
			bIntersects, RayParam);

		if (bIntersects)
		{
			// add the depth bias if any
			RayParam += DepthBias;

			FInputRayHit RayHit = MakeRayHit(RayParam, OutLineTraceOutput);
			
			// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold) ...
			const double MinimumPixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
			WorldHeight = Height * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X + MinimumPixelHitThresholdAdjust * 2.0;
			WorldOrigin = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector) - WorldDirection * MinimumPixelHitThresholdAdjust;

			GizmoMath::RayConeIntersection(
				WorldOrigin,
				WorldDirection,
				CosAngle,
				WorldHeight,
				ClampedRayOrigin, RayDirection,
				bIntersects, RayParam);

			if (bIntersects)
			{
				RayHit.HitDepth = RayParam + DepthBias;
				OutLineTraceOutput.bIsSurfaceHit = true;
			}

			return RayHit;
		}
	}

	return FInputRayHit();
}

void UGizmoElementCone::SetOrigin(const FVector& InOrigin)
{
	Origin = InOrigin;
}

FVector UGizmoElementCone::GetOrigin() const
{
	return Origin;
}

void UGizmoElementCone::SetDirection(const FVector& InDirection)
{
	Direction = InDirection;
	Direction.Normalize();
}

FVector UGizmoElementCone::GetDirection() const
{
	return Direction;
}

void UGizmoElementCone::SetHeight(float InHeight)
{
	Height = InHeight;
}

float UGizmoElementCone::GetHeight() const
{
	return Height;
}

void UGizmoElementCone::SetRadius(float InRadius)
{
	Radius = InRadius;
}

float UGizmoElementCone::GetRadius() const
{
	return Radius;
}

void UGizmoElementCone::SetNumSides(int32 InNumSides)
{
	NumSides = InNumSides;
}

int32 UGizmoElementCone::GetNumSides() const
{
	return NumSides;
}

void UGizmoElementCone::SetEndCaps(bool InEndCaps)
{
	bEndCaps = InEndCaps;
}

bool UGizmoElementCone::GetEndCaps() const
{
	return bEndCaps;
}
