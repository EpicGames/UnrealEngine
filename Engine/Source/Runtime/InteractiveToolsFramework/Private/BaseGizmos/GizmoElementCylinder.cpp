// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementCylinder.h"

#include "Algo/ForEach.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "DynamicMeshBuilder.h"
#include "GizmoPrivateUtil.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"
#include "SphereTypes.h"
#include "ToolDataVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementCylinder)

namespace GizmoElementCylinderLocals
{
	constexpr int32 MaxNumDashes = 512; // Arbitrary maximum to prevent overflow in the case of very long cylinders

	// Copy/Paste, with added Vertex Color, @see: PrimitiveDrawingUtil.h, DrawCylinder(...);
	void DrawCylinder(
		class FPrimitiveDrawInterface* PDI,
		const FMatrix& CylToWorld,
		const FVector& Base,
		const FVector& XAxis,
		const FVector& YAxis,
		const FVector& ZAxis,
		double Radius,
		double TopRadius,
		double HalfHeight,
		uint32 Sides,
		const FMaterialRenderProxy* MaterialInstance,
		uint8 DepthPriority,
		const FColor& InColor)
	{
		TArray<FDynamicMeshVertex> MeshVerts;
		TArray<uint32> MeshIndices;
		BuildTaperedCylinderVerts(Base, XAxis, YAxis, ZAxis, Radius, TopRadius, HalfHeight, Sides, MeshVerts, MeshIndices);

		Algo::ForEach(MeshVerts, [InColor](FDynamicMeshVertex& InVertex)
		{
			InVertex.Color = InColor;
		});

		FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
		MeshBuilder.AddVertices(MeshVerts);
		MeshBuilder.AddTriangles(MeshIndices);

		MeshBuilder.Draw(PDI, CylToWorld, MaterialInstance, DepthPriority);
	}
}

void UGizmoElementCylinder::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Base, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			// Line is too small to draw
			if (FMath::IsNearlyZero(Height))
			{
				return;
			}

			const FQuat Rotation = FRotationMatrix::MakeFromZ(Direction).ToQuat();
			const double HalfHeight = Height * 0.5;
			const FVector OriginOffset = Direction * HalfHeight;

			const FTransform RenderLocalToWorldTransform = FTransform(Rotation, OriginOffset) * CurrentRenderState.LocalToWorldTransform;

			// Applies flatten scale in local space (so it doesn't affect translation)
			FMatrix RenderLocalToWorldMatrix = RenderLocalToWorldTransform.ToMatrixWithScale();
			RenderLocalToWorldMatrix = RenderLocalToWorldMatrix * CurrentRenderState.SafeFlattenMatrix(RenderLocalToWorldTransform);
			RenderLocalToWorldMatrix.SetOrigin(RenderLocalToWorldTransform.GetTranslation()); // Restore origin

			FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

			const FColor Color = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);

			const FVector WorldOrigin = CurrentRenderState.LocalToWorldTransform.GetLocation();
			const FVector WorldDirection = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Direction);
			const double WorldHeight = Height * CurrentRenderState.LocalToWorldTransform.GetScale3D().X;

			// Get screen-space endpoints (used for visibility and for reparameterization, both world and screen-space)
            FVector ScreenStartPoint, ScreenEndPoint;
            double LineFractionInView = 1.0;
            double WorldStartFraction = 0.0;
			GizmoMath::GetScreenSpaceLineStartEnd(RenderAPI->GetSceneView(), WorldOrigin, WorldDirection, WorldHeight, ScreenStartPoint, ScreenEndPoint, LineFractionInView, WorldStartFraction);

			const float HeightInView = Height * LineFractionInView;

			// Screen-space dashed: need to reparameterize screen fractions -> world fractions and scale radii per segment, only used in screen-space path
			const float ScreenSpaceHeight = FVector2D::Distance(FVector2D(ScreenStartPoint), FVector2D(ScreenEndPoint));

			if (bIsDashed)
			{
				GizmoMath::FDashLineSettings DashLineSettings;
				DashLineSettings.MaterialInterface = UseMaterial;
				DashLineSettings.Radius = Radius;
				DashLineSettings.Color = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);
				DashLineSettings.RenderLocalToWorldMatrix = RenderLocalToWorldMatrix;
				DashLineSettings.DashGapLength = DashGapLength;
				DashLineSettings.DashLength = DashLength;
				DashLineSettings.ScreenStartPoint = ScreenStartPoint;
				DashLineSettings.ScreenEndPoint = ScreenEndPoint;
				DashLineSettings.ScaledLength = Height;
				DashLineSettings.LineFractionInView = LineFractionInView;
				DashLineSettings.WorldStartFraction = WorldStartFraction;

				if (bScreenSpace)
				{
					DashLineSettings.TotalLength = ScreenSpaceHeight;
					DashLineSettings.ParamToWorldFunction = [StartDepth = ScreenStartPoint.Z, EndDepth = ScreenEndPoint.Z](const double& InParam)
					{
						// Converts a screen-space fraction (0-1) to a world-space fraction (0-1), so that, for example,
						// the mid-point will appear to correctly to the viewer even if it's not the actual world-space mid-point.

						// Clamp screen fraction to normalized range
						const double ClampedParam = FMath::Clamp(InParam, 0.0, 1.0);

						return (ClampedParam * StartDepth) / (((1.0 - ClampedParam) * EndDepth) + (ClampedParam * StartDepth));
					};

					double DepthGapCorrection = 1.0;
					const double ReferencePointDepth = RenderState.DepthAtPixelToWorldReferencePoint;
					if (ReferencePointDepth > 0)
					{
						// When PixelToWorldScale is not sampled at StartState, StartDepth and ReferencePointDepth are not the same (see UGizmoElementTranslateGroup::UpdateDeltaRenderState)
						DepthGapCorrection = ScreenStartPoint.Z / ReferencePointDepth;
					}
					DashLineSettings.ParamToRadiusScaleFunction = [DepthGapCorrection, StartDepth = ScreenStartPoint.Z, EndDepth = ScreenEndPoint.Z](const double& InParam)
					{
						return DepthGapCorrection * GizmoMath::GetPixelToWorldScaleAtScreenFraction(StartDepth, EndDepth, InParam);
					};

					GizmoMath::BuildDashedLine(RenderAPI->GetPrimitiveDrawInterface(), DashLineSettings);
				
					//BuildDashes(ScreenSpaceDashPolicy);
				}
				else
				{
					//FDashPolicy WorldSpaceDashPolicy;
					DashLineSettings.TotalLength = HeightInView;
					DashLineSettings.ParamToWorldFunction = [](const double& InParam) { return InParam; }; // Linear mapping in world-space
					DashLineSettings.ParamToRadiusScaleFunction = [](const double&) { return 1.0; }; // Constant radius in world-space

					GizmoMath::BuildDashedLine(RenderAPI->GetPrimitiveDrawInterface(), DashLineSettings);
					//BuildDashes(WorldSpaceDashPolicy);
				}
			}
			else // Solid, not dashed
			{
				if (bScreenSpace)
				{
					constexpr double Start = 0.0;
					constexpr double End = 1.0;

					const double RadiusScaleStart = GizmoMath::GetPixelToWorldScaleAtScreenFraction(ScreenStartPoint.Z, ScreenEndPoint.Z, Start);
					const double RadiusScaleEnd = GizmoMath::GetPixelToWorldScaleAtScreenFraction(ScreenStartPoint.Z, ScreenEndPoint.Z, End);

					GizmoElementCylinderLocals::DrawCylinder(
						PDI,
						RenderLocalToWorldMatrix,
						(-FVector::UpVector * HalfHeight) + FVector::UpVector * HalfHeight,
						FVector::XAxisVector,
						FVector::YAxisVector,
						FVector::ZAxisVector,
						Radius * RadiusScaleStart,
						Radius * RadiusScaleEnd,
						HalfHeight,
						NumSides,
						UseMaterial->GetRenderProxy(),
						SDPG_Foreground,
						Color);
				}
				else
				{
					GizmoElementCylinderLocals::DrawCylinder(
						PDI,
						RenderLocalToWorldMatrix,
						(-FVector::UpVector * HalfHeight) + FVector::UpVector * HalfHeight,
						FVector::XAxisVector,
						FVector::YAxisVector,
						FVector::ZAxisVector,
						Radius,
						Radius,
						HalfHeight,
						NumSides,
						UseMaterial->GetRenderProxy(),
						SDPG_Foreground,
						Color);
				}
			}
		}
	}
}

FInputRayHit UGizmoElementCylinder::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Base, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		bool bIntersects = false;
		double RayParam = 0.0;
		
		double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		double WorldHeight = Height * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust * 2.0;
		double WorldRadius = Radius * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X + PixelHitThresholdAdjust;
		const FVector WorldDirection = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Direction);
		const FVector LocalCenter = Direction * Height * 0.5;
		const FVector WorldCenter = CurrentLineTraceState.LocalToWorldTransform.TransformPosition(LocalCenter);

		// due to numerical imprecision, the ray origin needs to be clamped in ortho views
		// (cf. UEditorInteractiveToolsContext::GetRayFromMousePos)
		FVector ClampedRayOrigin(RayOrigin);
		const double DepthBias = UE::GizmoUtil::ClampRayOrigin(ViewContext, ClampedRayOrigin, RayDirection);
		
		GizmoMath::RayCylinderIntersection(
			WorldCenter,
			WorldDirection,
			WorldRadius,
			WorldHeight,
			ClampedRayOrigin, RayDirection,
			bIntersects, RayParam);

		if (bIntersects)
		{
			// add the depth bias if any
			RayParam += DepthBias;

			FInputRayHit RayHit = MakeRayHit(RayParam, OutLineTraceOutput);

			UE::Geometry::TRay<double> Ray(ClampedRayOrigin, RayDirection);
			FVector HitPoint = Ray.PointAt(RayParam);

			// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold) ...
			// ... by negating the threshold
			WorldHeight -= PixelHitThresholdAdjust * 2.0;
			WorldRadius -= PixelHitThresholdAdjust;

			const double MinimumPixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
			WorldHeight += MinimumPixelHitThresholdAdjust * 2.0;
			WorldRadius += MinimumPixelHitThresholdAdjust;

			// Clamp both height and radius to a minimum of PixelToWorldScale to ensure hittability
			WorldHeight = FMath::Max(CurrentLineTraceState.PixelToWorldScale, WorldHeight);
			WorldRadius = FMath::Max(CurrentLineTraceState.PixelToWorldScale, WorldRadius);

			GizmoMath::RayCylinderIntersection(
				WorldCenter,
				WorldDirection,
				WorldRadius,
				WorldHeight,
				ClampedRayOrigin, RayDirection,
				bIntersects, RayParam);

			// We hit the surface
			if (bIntersects)
			{
				RayHit.HitDepth = RayParam + DepthBias;
				OutLineTraceOutput.bIsSurfaceHit = true;
			}

			return RayHit;
		}
	}

	return FInputRayHit();
}

void UGizmoElementCylinder::SetBase(const FVector& InBase)
{
	Base = InBase;
}

FVector UGizmoElementCylinder::GetBase() const
{
	return Base;
}

void UGizmoElementCylinder::SetDirection(const FVector& InDirection)
{
	Direction = InDirection;
	Direction.Normalize();
}

FVector UGizmoElementCylinder::GetDirection() const
{
	return Direction;
}

void UGizmoElementCylinder::SetHeight(float InHeight)
{
	Height = InHeight;
}

float UGizmoElementCylinder::GetHeight() const
{
	return Height;
}

void UGizmoElementCylinder::SetRadius(float InRadius)
{
	Radius = InRadius;
}

float UGizmoElementCylinder::GetRadius() const
{
	return Radius;
}

void UGizmoElementCylinder::SetNumSides(int32 InNumSides)
{
	NumSides = InNumSides;
}

int32 UGizmoElementCylinder::GetNumSides() const
{
	return NumSides;
}

void UGizmoElementCylinder::SetIsDashed(bool bInDashing)
{
	bIsDashed = bInDashing;
}

bool UGizmoElementCylinder::GetIsDashed() const
{
	return bIsDashed;
}

void UGizmoElementCylinder::SetDashParameters(const float InDashLength, const TOptional<float>& InGapLength)
{
	DashLength = InDashLength;
	DashGapLength = InGapLength.Get(InDashLength * 0.5f);
}

void UGizmoElementCylinder::GetDashParameters(float& OutDashLength, float& OutGapLength) const
{
	OutDashLength = DashLength;
	OutGapLength = DashGapLength;
}

void UGizmoElementCylinder::SetScreenSpace(bool bInScreenSpace)
{
	bScreenSpace = bInScreenSpace;
}

bool UGizmoElementCylinder::GetScreenSpace() const
{
	return bScreenSpace;
}
