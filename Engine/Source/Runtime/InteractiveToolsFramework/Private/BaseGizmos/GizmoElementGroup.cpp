// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoElementGroup.h"
#include "BaseGizmos/GizmoViewContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementGroup)

void UGizmoElementGroupBase::ApplyUniformConstantScaleToTransform(double PixelToWorldScale, FTransform& InOutLocalToWorldTransform) const
{
	return ApplyUniformConstantScaleToTransform(PixelToWorldScale, InOutLocalToWorldTransform, false);
}

void UGizmoElementGroupBase::ApplyUniformConstantScaleToTransform(double PixelToWorldScale, FTransform& InOutLocalToWorldTransform, const bool bInForceApply) const
{
	double Scale = InOutLocalToWorldTransform.GetScale3D().X;
	if (bInForceApply || bConstantScale)
	{
		Scale *= PixelToWorldScale;
	}
	InOutLocalToWorldTransform.SetScale3D(FVector(Scale, Scale, Scale));
}

void UGizmoElementGroupBase::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		ApplyUniformConstantScaleToTransform(CurrentRenderState.PixelToWorldScale, CurrentRenderState.LocalToWorldTransform);

		// Continue render even if not visible so all transforms will be cached 
		// for subsequent line tracing.
		ForEachSubElement([RenderAPI, &CurrentRenderState](UGizmoElementBase* Element)
		{
			Element->Render(RenderAPI, CurrentRenderState);
		});
	}
}

void UGizmoElementGroupBase::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, FVector::ZeroVector, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		ApplyUniformConstantScaleToTransform(CurrentRenderState.PixelToWorldScale, CurrentRenderState.LocalToWorldTransform);

		// Continue render even if not visible so all transforms will be cached
		// for subsequent line tracing.
		ForEachSubElement([Canvas, RenderAPI, &CurrentRenderState](UGizmoElementBase* Element)
		{
			Element->DrawHUD(Canvas, RenderAPI, CurrentRenderState);
		});
	}
}

FInputRayHit UGizmoElementGroupBase::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FInputRayHit Hit;

	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, FVector::ZeroVector, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		// Evaluates purely by distance (to the ray origin).
		struct FHitEvaluatorByDistance
		{
			/** Replaces the given Existing hit with the New hit if it's a better candidate. */
			static void UpdateHit(
				const EGizmoElementHitSortType InSortType,
				FInputRayHit& OutTargetOrExistingHit,
				FLineTraceOutput& OutTargetOrExistingLineTraceOutput,
				const FInputRayHit& InNewHit,
				const FLineTraceOutput& InNewLineTraceOutput)
			{
				// Early-out if the new hit is not valid
				if (!InNewHit.bHit)
				{
					return;
				}

				const bool bIsExistingHitValid = OutTargetOrExistingHit.bHit;
				bool bUseNewHit = !bIsExistingHitValid; // Use the new hit by default if the existing hit is not valid

				// If the existing hit was valid, check if the new is better
				if (bIsExistingHitValid)
				{
					const bool bIsNewCloserHit = InNewHit.HitDepth < OutTargetOrExistingHit.HitDepth;
					bUseNewHit = bIsNewCloserHit;
				}

				if (bUseNewHit)
				{
					OutTargetOrExistingHit = InNewHit;
					OutTargetOrExistingLineTraceOutput = InNewLineTraceOutput;
				}
			}
		};

		// Evaluates by priority, whether it's a surface hit (vs. within a tolerance), and by distance
		struct FHitEvaluatorByPriority
		{
			/** Replaces the given Existing hit with the New hit if it's a better candidate. */
			static void UpdateHit(
				const EGizmoElementHitSortType InSortType,
				FInputRayHit& OutTargetOrExistingHit,
				FLineTraceOutput& OutTargetOrExistingLineTraceOutput,
				const FInputRayHit& InNewHit,
				const FLineTraceOutput& InNewLineTraceOutput)
			{
				// Early-out if the new hit is not valid
				if (!InNewHit.bHit)
				{
					return;
				}

				const bool bIsExistingHitValid = OutTargetOrExistingHit.bHit;
				bool bUseNewHit = !bIsExistingHitValid; // Use the new hit by default if the existing hit is not valid

				// If the existing hit was valid, check if the new is better
				if (bIsExistingHitValid)
				{
					const bool bTestPriority = EnumHasAnyFlags(InSortType, EGizmoElementHitSortType::Priority);
					const bool bTestSurface = EnumHasAnyFlags(InSortType, EGizmoElementHitSortType::Surface);
					const bool bTestClosest = EnumHasAnyFlags(InSortType, EGizmoElementHitSortType::Closest);

					const int32 ExistingPriority = OutTargetOrExistingLineTraceOutput.HitPriority;
					const int32 NewPriority = InNewLineTraceOutput.HitPriority;

					const bool bIsExistingSurfaceHit = OutTargetOrExistingLineTraceOutput.bIsSurfaceHit;
					const bool bIsNewSurfaceHit = InNewLineTraceOutput.bIsSurfaceHit || !bTestSurface;

					const double ExistingHitDepth = OutTargetOrExistingHit.HitDepth;
					const double NewHitDepth = InNewHit.HitDepth;

					const auto ShouldUseNewHit = [&]() -> bool
					{
						// Negative priorities are ALWAYS behind positive priorities, even if exact hits.
						if (bTestPriority && NewPriority < 0 && ExistingPriority >= 0)
						{
							return false;
						}

						// Prefer exact surface hits
						if (bTestSurface)
						{
							if (bIsNewSurfaceHit && !bIsExistingSurfaceHit)
							{
								// New is exact, existing wasn't, so use new
								return true;
							}

							if (!bIsNewSurfaceHit && bIsExistingSurfaceHit)
							{
								// Existing hit is exact, new isn't, so keep existing
								return false;	
							}
						}

						// Higher priority win's (unless we already have an exact surface hit)
						if (bTestPriority && NewPriority > ExistingPriority)
						{
							return !(bTestSurface && bIsExistingSurfaceHit);
						}

						// Priorities are the same, so if we're testing closest, use that
						if (bTestClosest && (!bTestPriority || NewPriority == ExistingPriority))
						{
							return NewHitDepth < ExistingHitDepth;
						}

						return false;
					};

					bUseNewHit = ShouldUseNewHit();
				}

				if (bUseNewHit)
				{
					OutTargetOrExistingHit = InNewHit;
					OutTargetOrExistingLineTraceOutput = InNewLineTraceOutput;
				}
			}
		};

		ApplyUniformConstantScaleToTransform(CurrentLineTraceState.PixelToWorldScale, CurrentLineTraceState.LocalToWorldTransform);

		using FUpdateHitFunc = void(*)(
			const EGizmoElementHitSortType InSortType,
			FInputRayHit& OutTargetOrExistingHit,
			FLineTraceOutput& OutTargetOrExistingLineTraceOutput,
			const FInputRayHit& InNewHit,
			const FLineTraceOutput& InNewLineTraceOutput);

		FUpdateHitFunc UpdateHit = nullptr;

		if (CurrentLineTraceState.HitSortType == EGizmoElementHitSortType::Closest)
		{
			UpdateHit = FHitEvaluatorByDistance::UpdateHit;
		}
		else // @note: Add other sort types as needed
		{
			UpdateHit = FHitEvaluatorByPriority::UpdateHit;
		}

		if (ensure(UpdateHit))
		{
			ForEachSubElement([&UpdateHit, ViewContext, &CurrentLineTraceState, &RayOrigin, &RayDirection, &Hit, &OutLineTraceOutput](UGizmoElementBase* Element)
			{
				FLineTraceOutput NewLineTraceOutput;
				const FInputRayHit NewHit = Element->LineTrace(ViewContext, CurrentLineTraceState, RayOrigin, RayDirection, NewLineTraceOutput);
				if (!NewHit.bHit)
				{
					return;
				}

				UpdateHit(
					CurrentLineTraceState.HitSortType,
					Hit, OutLineTraceOutput,
					NewHit, NewLineTraceOutput);
			});

			if (bHitOwner && Hit.bHit)
			{
				Hit.SetHitObject(this);
				Hit.HitIdentifier = PartIdentifier;
			}
		}
	}

	return Hit;
}

void UGizmoElementGroupBase::SetPartIdentifier(uint32 InPartId, const bool bInOverrideUnsetChildren, const bool bInOverrideSet)
{
	if (!bInOverrideUnsetChildren)
	{
		Super::SetPartIdentifier(InPartId, bInOverrideUnsetChildren);
	}

	// Only set if the part identifier is current Default (0), and bInOverrideSet is true 
	if (bInOverrideSet || GetPartIdentifier() == 0)
	{
		UGizmoElementBase::SetPartIdentifier(InPartId);
	}

	ForEachSubElement([InPartId, bInOverrideUnsetChildren](UGizmoElementBase* Element)
	{
		// By default, bOverrideChildIfSet is true, but we only want that for the root element
		constexpr bool bOverrideChildIfSet = false;
		Element->SetPartIdentifier(InPartId, bInOverrideUnsetChildren, bOverrideChildIfSet);
	});
}

UGizmoElementBase* UGizmoElementGroupBase::FindPartElement(const uint32 InPartId)
{
	for (UGizmoElementBase* Element : Elements)
	{
		if (Element && Element->GetPartIdentifier() == InPartId)
		{
			return Element;
		}
	}

	return nullptr;
}

const UGizmoElementBase* UGizmoElementGroupBase::FindPartElement(const uint32 InPartId) const
{
	for (UGizmoElementBase* Element : Elements)
	{
		if (Element && Element->GetPartIdentifier() == InPartId)
		{
			return Element;
		}
	}

	return nullptr;
}

void UGizmoElementGroupBase::Add(UGizmoElementBase* InElement)
{
	if (!Elements.Contains(InElement))
	{
		Elements.Add(InElement);
	}
}

void UGizmoElementGroupBase::Remove(UGizmoElementBase* InElement)
{
	if (InElement)
	{
		int32 Index;
		if (Elements.Find(InElement, Index))
		{
			Elements.RemoveAtSwap(Index);
		}
	}
}

TConstArrayView<TObjectPtr<UGizmoElementBase>> UGizmoElementGroupBase::GetSubElements() const
{
	return Elements;
}

bool UGizmoElementGroupBase::UpdatePartVisibleState(bool bVisible, uint32 InPartIdentifier, bool bInAllowMultipleElements)
{
	bool bWasUpdated = false;

	if (Super::UpdatePartVisibleState(bVisible, InPartIdentifier, bInAllowMultipleElements))
	{
		bWasUpdated = true;
		if (bWasUpdated && !bInAllowMultipleElements)
		{
			return true;
		}
	}

	for (UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{
			bWasUpdated |= Element->UpdatePartVisibleState(bVisible, InPartIdentifier, bInAllowMultipleElements);
			if (bWasUpdated && !bInAllowMultipleElements)
			{
				return true;
			}
		}
	}

	return bWasUpdated;
}

TOptional<bool> UGizmoElementGroupBase::GetPartVisibleState(uint32 InPartIdentifier) const
{
	TOptional<bool> Result = Super::GetPartVisibleState(InPartIdentifier);
	if (Result.IsSet())
	{
		return Result;
	}

	for (UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{
			Result = Element->GetPartVisibleState(InPartIdentifier);
			if (Result.IsSet())
			{
				return Result;
			}
		}
	}

	return TOptional<bool>();
}


bool UGizmoElementGroupBase::UpdatePartHittableState(bool bHittable, uint32 InPartIdentifier, bool bInAllowMultipleElements)
{
	bool bWasUpdated = false;

	if (Super::UpdatePartHittableState(bHittable, InPartIdentifier, bInAllowMultipleElements))
	{
		bWasUpdated = true;
		if (bWasUpdated && !bInAllowMultipleElements)
		{
			return true;
		}
	}

	for (UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{
			bWasUpdated |= Element->UpdatePartHittableState(bHittable, InPartIdentifier, bInAllowMultipleElements);
			if (bWasUpdated && !bInAllowMultipleElements)
			{
				return true;
			}
		}
	}

	return bWasUpdated;
}

TOptional<bool> UGizmoElementGroupBase::GetPartHittableState(uint32 InPartIdentifier) const
{
	TOptional<bool> Result = Super::GetPartHittableState(InPartIdentifier);
	if (Result.IsSet())
	{
		return Result;
	}

	for (UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{
			Result = Element->GetPartHittableState(InPartIdentifier);
			if (Result.IsSet())
			{
				return Result;
			}
		}
	}

	return TOptional<bool>();
}

bool UGizmoElementGroupBase::UpdatePartInteractionState(EGizmoElementInteractionState InInteractionState, uint32 InPartIdentifier, bool bInAllowMultipleElements)
{
	bool bWasUpdated = false;

	if (Super::UpdatePartInteractionState(InInteractionState, InPartIdentifier, bInAllowMultipleElements))
	{
		bWasUpdated = true;
		if (bWasUpdated && !bInAllowMultipleElements)
		{
			return true;
		}
	}

	for (UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{
			bWasUpdated |= Element->UpdatePartInteractionState(InInteractionState, InPartIdentifier, bInAllowMultipleElements);
			if (bWasUpdated && !bInAllowMultipleElements)
			{
				return true;
			}
		}
	}

	return bWasUpdated;
}


TOptional<EGizmoElementInteractionState> UGizmoElementGroupBase::GetPartInteractionState(uint32 InPartIdentifier) const
{
	TOptional<EGizmoElementInteractionState> Result = Super::GetPartInteractionState(InPartIdentifier);
	if (Result.IsSet())
	{
		return Result;
	}

	for (UGizmoElementBase* Element : Elements)
	{
		if (Element)
		{ 
			Result = Element->GetPartInteractionState(InPartIdentifier);
			if (Result.IsSet())
			{
				return Result;
			}
		}
	}

	return TOptional<EGizmoElementInteractionState>();
}

void UGizmoElementGroupBase::SetConstantScale(bool bInConstantScale)
{
	bConstantScale = bInConstantScale;
}

bool UGizmoElementGroupBase::GetConstantScale() const
{
	return bConstantScale;
}

void UGizmoElementGroupBase::SetHitOwner(bool bInHitOwner)
{
	bHitOwner = bInHitOwner;
}

bool UGizmoElementGroupBase::GetHitOwner() const
{
	return bHitOwner;
}
