// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoElementHitTargets.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/TransformProxy.h"
#include "BaseGizmos/ViewBasedTransformAdjusters.h"
#include "Engine/EngineTypes.h"    // FHitResult

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementHitTargets)

namespace UE::InteractiveToolsFramework::Private
{
	namespace GizmoElementHitTargetLocals
	{
		constexpr bool bAllowMultiElementParts = false;
		
		// Toggles between the provided state and "None" state for the given gizmo element.
		void ToggleElementInteractionState(UGizmoElementBase* InGizmoElement, const EGizmoElementInteractionState InState, const bool bInStateEnable)
		{
			if (!InGizmoElement)
			{
				return;
			}

			InGizmoElement->SetElementInteractionState(bInStateEnable ? InState : EGizmoElementInteractionState::None);
		}

		// Toggles between the provided state and "None" state for the given gizmo element.
		void ToggleElementPartInteractionState(UGizmoElementBase* InGizmoElement, const uint32 InPartId, const EGizmoElementInteractionState InState, const bool bInStateEnable, const bool bInAllowMultipleElements = false)
		{
			if (!InGizmoElement)
			{
				return;
			}

			InGizmoElement->UpdatePartInteractionState(bInStateEnable ? InState : EGizmoElementInteractionState::None, InPartId, bInAllowMultipleElements);
		}
	}
}

FInputRayHit UGizmoElementHitTarget::IsHit(const FInputDeviceRay& ClickPos) const
{
	if (GizmoElement && GizmoViewContext && GizmoTransformProxy && (!Condition || Condition(ClickPos)))
	{
		FTransform Transform = GizmoTransformProxy->GetTransform();
		if (TransformAdjuster.IsValid())
		{
			Transform = TransformAdjuster->GetAdjustedComponentToWorld(*GizmoViewContext, Transform);
		}

		UGizmoElementBase::FLineTraceTraversalState LineTraceState;
		LineTraceState.Initialize(GizmoViewContext, Transform);

		UGizmoElementBase::FLineTraceOutput LineTraceOutput;		
		return GizmoElement->LineTrace(GizmoViewContext, LineTraceState, ClickPos.WorldRay.Origin, ClickPos.WorldRay.Direction, LineTraceOutput);
	}
	return FInputRayHit();
}

void UGizmoElementHitTarget::UpdateHoverState(bool bHovering)
{
	if (!GizmoElement)
	{
		return;
	}

	if (bHovering)
	{
		GizmoElement->SetElementInteractionState(EGizmoElementInteractionState::Hovering);
	}
	// If this element is interacting, do not update the hover state. This is necessary because when transitioning
	// from hovering to interacting, the gizmo framework updates interacting to true before updating hovering to false.
	else if (GizmoElement->GetElementInteractionState() != EGizmoElementInteractionState::Interacting)
	{
		GizmoElement->SetElementInteractionState(EGizmoElementInteractionState::None);
	}
}

void UGizmoElementHitTarget::UpdateInteractingState(bool bInteracting)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	ToggleElementInteractionState(GizmoElement, EGizmoElementInteractionState::Interacting, bInteracting);
}

void UGizmoElementHitTarget::UpdateSelectedState(bool bInSelected)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	ToggleElementInteractionState(GizmoElement, EGizmoElementInteractionState::Selected, bInSelected);
}

void UGizmoElementHitTarget::UpdateSubdueState(bool bSubdued)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	ToggleElementInteractionState(GizmoElement, EGizmoElementInteractionState::Subdued, bSubdued);
}

void UGizmoElementHitTarget::SetTransformAdjuster(const TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster>& InAdjuster)
{
	TransformAdjuster = InAdjuster;
}

TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> UGizmoElementHitTarget::GetTransformAdjuster() const
{
	return TransformAdjuster;
}

UGizmoElementHitTarget* UGizmoElementHitTarget::Construct(UGizmoElementBase* InGizmoElement, UGizmoViewContext* InGizmoViewContext, UObject* Outer)
{
	UGizmoElementHitTarget* NewHitTarget = NewObject<UGizmoElementHitTarget>(Outer);
	NewHitTarget->GizmoElement = InGizmoElement;
	NewHitTarget->GizmoViewContext = InGizmoViewContext;
	return NewHitTarget;
}

FInputRayHit UGizmoElementHitMultiTarget::IsHit(const FInputDeviceRay& ClickPos) const
{
	if (GizmoElement && GizmoViewContext && GizmoTransformProxy && (!Condition || Condition(ClickPos)))
	{
		FTransform Transform = GizmoTransformProxy->GetTransform();
		if (TransformAdjuster.IsValid())
		{
			Transform = TransformAdjuster->GetAdjustedComponentToWorld(*GizmoViewContext, Transform);
		}

		UGizmoElementBase::FLineTraceTraversalState LineTraceState;
		LineTraceState.Initialize(GizmoViewContext, Transform);

		UGizmoElementBase::FLineTraceOutput LineTraceOutput;
		return GizmoElement->LineTrace(GizmoViewContext, LineTraceState, ClickPos.WorldRay.Origin, ClickPos.WorldRay.Direction, LineTraceOutput);
	}
	return FInputRayHit();
}

void UGizmoElementHitMultiTarget::UpdateHoverState(bool bInHovering, uint32 PartIdentifier)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;
	
	return UpdateHoverState(bInHovering, PartIdentifier, bAllowMultiElementParts);
}

void UGizmoElementHitMultiTarget::UpdateHoverState(bool bInHovering, uint32 PartIdentifier, bool bInAllowMultipleElements)
{
	if (!GizmoElement)
	{
		return;
	}

	if (bInHovering)
	{
		GizmoElement->UpdatePartInteractionState(EGizmoElementInteractionState::Hovering, PartIdentifier, bInAllowMultipleElements);
	}
	else
	{
		// If this element is interacting, do not update the hover state. This is necessary because when transitioning
		// from hovering to interacting, the gizmo framework updates interacting to true before updating hovering to false.
		TOptional<EGizmoElementInteractionState> InteractionStateResult = GizmoElement->GetPartInteractionState(PartIdentifier);
		if (InteractionStateResult.IsSet() && InteractionStateResult.GetValue() == EGizmoElementInteractionState::Interacting)
		{
			return;
		}

		GizmoElement->UpdatePartInteractionState(EGizmoElementInteractionState::None, PartIdentifier, bInAllowMultipleElements);
	}
}

void UGizmoElementHitMultiTarget::UpdateInteractingState(bool bInInteracting, uint32 PartIdentifier)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	UpdateInteractingState(bInInteracting, PartIdentifier, bAllowMultiElementParts);
}

void UGizmoElementHitMultiTarget::UpdateInteractingState(bool bInInteracting, uint32 PartIdentifier, bool bInAllowMultipleElements)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	ToggleElementPartInteractionState(GizmoElement, PartIdentifier, EGizmoElementInteractionState::Interacting, bInInteracting, bInAllowMultipleElements);
}

void UGizmoElementHitMultiTarget::UpdateSelectedState(bool bInSelected, uint32 PartIdentifier)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	UpdateSelectedState(bInSelected, PartIdentifier, bAllowMultiElementParts);
}

void UGizmoElementHitMultiTarget::UpdateSelectedState(bool bInSelected, uint32 PartIdentifier, bool bInAllowMultipleElements)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	ToggleElementPartInteractionState(GizmoElement, PartIdentifier, EGizmoElementInteractionState::Selected, bInSelected, bInAllowMultipleElements);
}

void UGizmoElementHitMultiTarget::UpdateSubdueState(bool bInSubdued, uint32 PartIdentifier)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	UpdateSubdueState(bInSubdued, PartIdentifier, bAllowMultiElementParts);
}

void UGizmoElementHitMultiTarget::UpdateSubdueState(bool bInSubdued, uint32 PartIdentifier, bool bInAllowMultipleElements)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;
	
	ToggleElementPartInteractionState(GizmoElement, PartIdentifier, EGizmoElementInteractionState::Subdued, bInSubdued, bInAllowMultipleElements);
}

void UGizmoElementHitMultiTarget::UpdateHittableState(bool bHittable, uint32 PartIdentifier)
{
	using namespace UE::InteractiveToolsFramework::Private::GizmoElementHitTargetLocals;

	UpdateHittableState(bHittable, PartIdentifier, bAllowMultiElementParts);
}

void UGizmoElementHitMultiTarget::UpdateHittableState(bool bHittable, uint32 PartIdentifier, bool bInAllowMultipleElements)
{
	if (!GizmoElement)
	{
		return;
	}

	GizmoElement->UpdatePartHittableState(bHittable, PartIdentifier, bInAllowMultipleElements);
}

void UGizmoElementHitMultiTarget::SetTransformAdjuster(const TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster>& InAdjuster)
{
	TransformAdjuster = InAdjuster;
}

TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> UGizmoElementHitMultiTarget::GetTransformAdjuster() const
{
	return TransformAdjuster;
}

UGizmoElementHitMultiTarget* UGizmoElementHitMultiTarget::Construct(UGizmoElementBase* InGizmoElement, UGizmoViewContext* InGizmoViewContext, UObject* Outer)
{
	UGizmoElementHitMultiTarget* NewHitTarget = NewObject<UGizmoElementHitMultiTarget>(Outer);
	NewHitTarget->GizmoElement = InGizmoElement;
	NewHitTarget->GizmoViewContext = InGizmoViewContext;
	return NewHitTarget;
}
