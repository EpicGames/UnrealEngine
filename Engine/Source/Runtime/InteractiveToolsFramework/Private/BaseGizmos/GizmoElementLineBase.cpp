// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementLineBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementLineBase)

DEFINE_LOG_CATEGORY_STATIC(LogGizmoElementLineBase, Log, All);

bool UGizmoElementLineBase::UpdateRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState)
{
	InOutRenderState.LineRenderState.Update(LineRenderAttributes);

	return Super::UpdateRenderState(RenderAPI, InLocalOrigin, InOutRenderState);
}

float UGizmoElementLineBase::GetCurrentLineThickness(bool bPerspectiveView, float InViewFOV) const
{
	float CurrentLineThickness;

	auto GetMultipliedLineThickness = [this](float InMultiplier) -> float
	{
		return (LineThickness > UE_SMALL_NUMBER ? LineThickness * InMultiplier : InMultiplier);
	};

	switch (ElementInteractionState)
	{
	case EGizmoElementInteractionState::Hovering:
		CurrentLineThickness = GetMultipliedLineThickness(HoverLineThicknessMultiplier);
		break;

	case EGizmoElementInteractionState::Interacting:
		CurrentLineThickness = GetMultipliedLineThickness(InteractLineThicknessMultiplier);
		break;

	case EGizmoElementInteractionState::Selected:
		CurrentLineThickness = GetMultipliedLineThickness(SelectLineThicknessMultiplier);
		break;

	case EGizmoElementInteractionState::Subdued:
		CurrentLineThickness = GetMultipliedLineThickness(SubdueLineThicknessMultiplier);
		break;

	default:
		CurrentLineThickness = LineThickness;
		break;
	}

	if (bPerspectiveView)
	{
		CurrentLineThickness *= (InViewFOV / 90.0f);		// compensate for FOV scaling in Gizmos...
	}

	return CurrentLineThickness;
}

void UGizmoElementLineBase::SetLineThickness(float InLineThickness)
{
	if (InLineThickness < 0.0f)
	{
		UE_LOGF(LogGizmoElementLineBase, Warning, "Invalid gizmo element line thickness %f, will be set to 0.0.", InLineThickness);
		LineThickness = 0.0f;
	}
	else
	{
		LineThickness = InLineThickness;
	}
}

float UGizmoElementLineBase::GetLineThickness() const
{
	return LineThickness;
}

void UGizmoElementLineBase::SetHoverLineThicknessMultiplier(float InHoverLineThicknessMultiplier)
{
	HoverLineThicknessMultiplier = InHoverLineThicknessMultiplier;
}

float UGizmoElementLineBase::GetHoverLineThicknessMultiplier() const
{
	return HoverLineThicknessMultiplier;
}

void UGizmoElementLineBase::SetInteractLineThicknessMultiplier(float InInteractLineThicknessMultiplier)
{
	InteractLineThicknessMultiplier = InInteractLineThicknessMultiplier;
}

float UGizmoElementLineBase::GetInteractLineThicknessMultiplier() const
{
	return InteractLineThicknessMultiplier;
}

void UGizmoElementLineBase::SetSelectLineThicknessMultiplier(float InSelectLineThicknessMultiplier)
{
	SelectLineThicknessMultiplier = InSelectLineThicknessMultiplier;
}

float UGizmoElementLineBase::GetSelectLineThicknessMultiplier() const
{
	return SelectLineThicknessMultiplier;
}

void UGizmoElementLineBase::SetSubdueLineThicknessMultiplier(float InSubdueLineThicknessMultiplier)
{
	SubdueLineThicknessMultiplier = InSubdueLineThicknessMultiplier;
}

float UGizmoElementLineBase::GetSubdueLineThicknessMultiplier() const
{
	return SubdueLineThicknessMultiplier;
}

void UGizmoElementLineBase::SetScreenSpaceLine(bool bInScreenSpaceLine)
{
	bScreenSpaceLine = bInScreenSpaceLine;
}

bool UGizmoElementLineBase::GetScreenSpaceLine() const
{
	return bScreenSpaceLine;
}

void UGizmoElementLineBase::SetLineColor(FLinearColor InLineColor, bool InOverridesChildState)
{
	LineRenderAttributes.LineColor.SetColor(InLineColor, InOverridesChildState);
}

FLinearColor UGizmoElementLineBase::GetLineColor() const
{
	return LineRenderAttributes.LineColor.GetColor();
}

bool UGizmoElementLineBase::HasLineColor() const
{
	return LineRenderAttributes.LineColor.bHasValue;
}

bool UGizmoElementLineBase::DoesLineColorOverrideChildState() const
{
	return LineRenderAttributes.LineColor.bOverridesChildState;
}

void UGizmoElementLineBase::ClearLineColor()
{
	LineRenderAttributes.LineColor.Reset();
}

void UGizmoElementLineBase::SetHoverLineColor(FLinearColor InHoverLineColor, bool InOverridesChildState)
{
	LineRenderAttributes.HoverLineColor.SetColor(InHoverLineColor, InOverridesChildState);
}

FLinearColor UGizmoElementLineBase::GetHoverLineColor() const
{
	return LineRenderAttributes.HoverLineColor.GetColor();
}
bool UGizmoElementLineBase::HasHoverLineColor() const
{
	return LineRenderAttributes.HoverLineColor.bHasValue;
}

bool UGizmoElementLineBase::DoesHoverLineColorOverrideChildState() const
{
	return LineRenderAttributes.HoverLineColor.bOverridesChildState;
}

void UGizmoElementLineBase::ClearHoverLineColor()
{
	LineRenderAttributes.HoverLineColor.Reset();
}

void UGizmoElementLineBase::SetInteractLineColor(FLinearColor InInteractLineColor, bool InOverridesChildState)
{
	LineRenderAttributes.InteractLineColor.SetColor(InInteractLineColor, InOverridesChildState);
}

FLinearColor UGizmoElementLineBase::GetInteractLineColor() const
{
	return LineRenderAttributes.InteractLineColor.GetColor();
}
bool UGizmoElementLineBase::HasInteractLineColor() const
{
	return LineRenderAttributes.InteractLineColor.bHasValue;
}

bool UGizmoElementLineBase::DoesInteractLineColorOverrideChildState() const
{
	return LineRenderAttributes.InteractLineColor.bOverridesChildState;
}

void UGizmoElementLineBase::ClearInteractLineColor()
{
	LineRenderAttributes.InteractLineColor.Reset();
}

void UGizmoElementLineBase::SetSelectLineColor(FLinearColor InColor, bool InOverridesChildState)
{
	LineRenderAttributes.SelectLineColor.SetColor(InColor, InOverridesChildState);
}

FLinearColor UGizmoElementLineBase::GetSelectLineColor() const
{
	return LineRenderAttributes.SelectLineColor.GetColor();
}

bool UGizmoElementLineBase::HasSelectLineColor() const
{
	return LineRenderAttributes.SelectLineColor.bHasValue;
}

bool UGizmoElementLineBase::DoesSelectLineColorOverrideChildState() const
{
	return LineRenderAttributes.SelectLineColor.bOverridesChildState;
}

void UGizmoElementLineBase::ClearSelectLineColor()
{
	LineRenderAttributes.SelectLineColor.Reset();
}

void UGizmoElementLineBase::SetSubdueLineColor(FLinearColor InColor, bool InOverridesChildState)
{
	LineRenderAttributes.SubdueLineColor.SetColor(InColor, InOverridesChildState);
}

FLinearColor UGizmoElementLineBase::GetSubdueLineColor() const
{
	return LineRenderAttributes.SubdueLineColor.GetColor();
}

bool UGizmoElementLineBase::HasSubdueLineColor() const
{
	return LineRenderAttributes.SubdueLineColor.bHasValue;
}

bool UGizmoElementLineBase::DoesSubdueLineColorOverrideChildState() const
{
	return LineRenderAttributes.SubdueLineColor.bOverridesChildState;
}

void UGizmoElementLineBase::ClearSubdueLineColor()
{
	LineRenderAttributes.SubdueLineColor.Reset();
}
