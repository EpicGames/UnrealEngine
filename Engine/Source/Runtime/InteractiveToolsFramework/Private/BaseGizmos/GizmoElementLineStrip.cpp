// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementLineStrip.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "Math/Ray.h"
#include "PrimitiveDrawingUtils.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementLineStrip)

void UGizmoElementLineStrip::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}
	
	FRenderTraversalState CurrentRenderState(RenderState);
	const bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Base, CurrentRenderState);

	if (bVisibleViewDependent && Vertices.Num() > 1)
	{
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		ComputeProjectedVertices(RenderState.LocalToWorldTransform, true);

		const FSceneView* View = RenderAPI->GetSceneView();
		if (!ensure(View))
		{
			return;
		}

		const float CurrentLineThickness = GetCurrentLineThickness(View->IsPerspectiveProjection(), View->FOV);
		const FColor LineColor = CurrentRenderState.GetCurrentLineColor().ToFColor(true);

		// Boost depth bias for lines to avoid z-fighting with the fill (of this or other elements)
		const float LineDepthBias = 1.0f + CurrentLineThickness;
		
		if(bDrawLineStrip)
		{
			for(int32 Index = 0; Index < ProjectedVertices.Num() - 1; Index++)
			{
				const FVector& A = ProjectedVertices[Index];
				const FVector& B = ProjectedVertices[Index+1];
				PDI->DrawLine(A, B, LineColor, SDPG_Foreground, CurrentLineThickness, LineDepthBias, bScreenSpaceLine);
			}
		}
		else
		{
			for(int32 Index = 0; Index < ProjectedVertices.Num(); Index+=2)
			{
				const FVector& A = ProjectedVertices[Index];
				const FVector& B = ProjectedVertices[Index+1];
				PDI->DrawLine(A, B, LineColor, SDPG_Foreground, CurrentLineThickness, LineDepthBias, bScreenSpaceLine);
			}
		}
	}
}

FInputRayHit UGizmoElementLineStrip::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	const bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Base, CurrentLineTraceState);

	if (bHittableViewDependent && Vertices.Num() > 1)
	{
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		ComputeProjectedVertices(CurrentLineTraceState.LocalToWorldTransform, false);

		const FRay Ray(RayOrigin, RayDirection);
		const FVector RayB = Ray.Origin + Ray.Direction * 10000.0;
		auto Intersect = [Ray, RayB](const FVector& A, const FVector&B, double InTolerance, double& Distance) -> bool
		{
			FVector HitPointA = Ray.Origin;
			FVector HitPointB = A;
			FMath::SegmentDistToSegmentSafe(Ray.Origin, RayB, A, B, HitPointA, HitPointB);
			if(HitPointA.Equals(Ray.Origin))
			{
				return false;
			}

			Distance = FVector::Distance(HitPointA, HitPointB);
			return Distance < InTolerance;
		};

		double HitDepth = 0;
		if(bDrawLineStrip)
		{
			for(int32 Index = 0; Index < ProjectedVertices.Num() - 1; Index++)
			{
				const FVector& A = ProjectedVertices[Index];
				const FVector& B = ProjectedVertices[Index+1];
				if(Intersect(A, B, PixelHitThresholdAdjust, HitDepth))
				{
					return MakeRayHit(HitDepth, OutLineTraceOutput);
				}
			}
		}
		else
		{
			for(int32 Index = 0; Index < ProjectedVertices.Num(); Index+=2)
			{
				const FVector& A = ProjectedVertices[Index];
				const FVector& B = ProjectedVertices[Index+1];
				if(Intersect(A, B, PixelHitThresholdAdjust, HitDepth))
				{
					return MakeRayHit(HitDepth, OutLineTraceOutput);
				}
			}
		}
	}

	return FInputRayHit();
}

void UGizmoElementLineStrip::SetVertices(const TArrayView<const FVector>& InVertices)
{
	Vertices = InVertices;
}

const TArray<FVector>& UGizmoElementLineStrip::GetVertices() const
{
	return Vertices;
}

void UGizmoElementLineStrip::SetBase(FVector InBase)
{
	Base = InBase;
}

FVector UGizmoElementLineStrip::GetBase() const
{
	return Base;
}

void UGizmoElementLineStrip::SetUpDirection(const FVector& InUpDirection)
{
	UpDirection = InUpDirection.GetSafeNormal();
}

FVector UGizmoElementLineStrip::GetUpDirection() const
{
	return UpDirection;
}

void UGizmoElementLineStrip::SetSideDirection(const FVector& InSideDirection)
{
	SideDirection = InSideDirection.GetSafeNormal();
}

FVector UGizmoElementLineStrip::GetSideDirection() const
{
	return SideDirection;
}

void UGizmoElementLineStrip::SetDrawLineStrip(bool InDrawLineStrip)
{
	bDrawLineStrip = InDrawLineStrip;
}

bool UGizmoElementLineStrip::GetDrawLineStrip() const
{
	return bDrawLineStrip;
}

void UGizmoElementLineStrip::ComputeProjectedVertices(const FTransform& InLocalToWorldTransform, bool bIncludeBase)
{
	const FVector WorldUpAxis = InLocalToWorldTransform.TransformVectorNoScale(UpDirection);
    const FVector WorldSideAxis = InLocalToWorldTransform.TransformVectorNoScale(SideDirection);
    const FVector WorldNormal = FVector::CrossProduct(WorldSideAxis, WorldUpAxis);
    const FVector WorldCenter = InLocalToWorldTransform.TransformPosition(bIncludeBase ? Base : FVector::ZeroVector);
    const FVector WorldScale = InLocalToWorldTransform.GetScale3D();

    ProjectedVertices.Reset();
    ProjectedVertices.Reserve(Vertices.Num());

    for(int32 Index = 0; Index < Vertices.Num(); Index++)
    {
    	FVector Vertex = Vertices[Index] * WorldScale;
    	ProjectedVertices.Add(WorldCenter + WorldNormal * Vertex.X + WorldSideAxis * Vertex.Y + WorldUpAxis * Vertex.Z);
    }
}

