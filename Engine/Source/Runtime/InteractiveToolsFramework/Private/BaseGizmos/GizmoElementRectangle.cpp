// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementRectangle.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementRectangle)

void UGizmoElementRectangle::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	if (!ensure(RenderAPI))
	{
		return;
	}
	
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		const FVector WorldUpAxis = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(UpDirection);
		const FVector WorldSideAxis = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(SideDirection);
		const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
		const float WorldWidth = static_cast<float>(Width * CurrentRenderState.LocalToWorldTransform.GetScale3D().X);
		const float WorldHeight = static_cast<float>(Height * CurrentRenderState.LocalToWorldTransform.GetScale3D().X);

		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

		if (bDrawMesh)
		{
			if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
			{
				FColor VertexColor = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);
				DrawRectangleMesh(PDI, WorldCenter, WorldUpAxis, WorldSideAxis, VertexColor, WorldWidth, WorldHeight, UseMaterial->GetRenderProxy(), SDPG_Foreground);
			}
		}

		if (bDrawLine)
		{
			const FSceneView* View = RenderAPI->GetSceneView();
			if (!ensure(View))
			{
				return;
			}

			float CurrentLineThickness = GetCurrentLineThickness(View->IsPerspectiveProjection(), View->FOV);

			// Boost depth bias for lines to avoid z-fighting with the fill (of this or other elements)
			const float LineDepthBias = 1.0f + CurrentLineThickness;

			FColor LineColor = CurrentRenderState.GetCurrentLineColor().ToFColor(true);
			DrawRectangle(PDI, WorldCenter, WorldUpAxis, WorldSideAxis, LineColor, WorldWidth, WorldHeight, SDPG_Foreground, CurrentLineThickness, LineDepthBias, bScreenSpaceLine);
		}
	}
}

FInputRayHit UGizmoElementRectangle::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		FTransform LocalToWorldTransform = CurrentLineTraceState.LocalToWorldTransform;
		const FVector WorldUpAxis = LocalToWorldTransform.TransformVectorNoScale(UpDirection);
		const FVector WorldSideAxis = LocalToWorldTransform.TransformVectorNoScale(SideDirection);
		const FVector WorldNormal = FVector::CrossProduct(WorldUpAxis, WorldSideAxis);
		const FVector WorldCenter = LocalToWorldTransform.TransformPosition(FVector::ZeroVector);
		const double Scale = LocalToWorldTransform.GetScale3D().X;
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		double WorldHeight = Scale * Height + PixelHitThresholdAdjust * 2.0;
		double WorldWidth = Scale * Width + PixelHitThresholdAdjust * 2.0;
		const FVector Base = WorldCenter - WorldUpAxis * WorldHeight * 0.5 - WorldSideAxis * WorldWidth * 0.5;

		// if ray is parallel to rectangle, no hit
		if (FMath::IsNearlyZero(FVector::DotProduct(WorldNormal, RayDirection)))
		{
			return FInputRayHit();
		}

		if (bHitMesh)
		{
			FPlane Plane(Base, WorldNormal);
			double HitDepth = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);
			if (HitDepth < UE_DOUBLE_SMALL_NUMBER)
			{
				return FInputRayHit();
			}

			FVector HitPoint = RayOrigin + RayDirection * HitDepth;
			FVector HitOffset = HitPoint - Base;
			double HdU = FVector::DotProduct(HitOffset, WorldUpAxis);
			double HdS = FVector::DotProduct(HitOffset, WorldSideAxis);

			// clip to rectangle dimensions
			if (HdU >= UE_DOUBLE_SMALL_NUMBER && HdU <= WorldHeight && HdS >= UE_DOUBLE_SMALL_NUMBER && HdS <= WorldWidth)
			{
				FInputRayHit RayHit = MakeRayHit(HitDepth, OutLineTraceOutput);

				// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold)
				const double MinimumPixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
				WorldHeight = Scale * Height + MinimumPixelHitThresholdAdjust * 2.0;
				WorldWidth = Scale * Width + MinimumPixelHitThresholdAdjust * 2.0;
				OutLineTraceOutput.bIsSurfaceHit = HdU <= WorldHeight && HdS <= WorldWidth;

				return RayHit;
			}
		}
		else if (bHitLine)
		{
			FPlane Plane(Base, WorldNormal);
			double HitDepth = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);
			if (HitDepth < UE_DOUBLE_SMALL_NUMBER)
			{
				return FInputRayHit();
			}

			FVector HitPoint = RayOrigin + RayDirection * HitDepth;
			FVector HitOffset = HitPoint - Base;
			double HdU = FVector::DotProduct(HitOffset, WorldUpAxis);
			double HdS = FVector::DotProduct(HitOffset, WorldSideAxis);

			double HitBuffer = PixelHitThresholdAdjust + LineThickness;

			// determine if the hit is within pixel tolerance of the edges of rectangle
			if (HdU >= UE_DOUBLE_SMALL_NUMBER && HdU <= WorldHeight && HdS >= UE_DOUBLE_SMALL_NUMBER && HdS <= WorldWidth &&
				(HdS <= HitBuffer || HdS >= WorldWidth - HitBuffer ||
				 HdU <= HitBuffer || HdU >= WorldHeight - HitBuffer))
			{
				FInputRayHit RayHit = MakeRayHit(HitDepth, OutLineTraceOutput);

				// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold)
				const double MinimumPixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
				HitBuffer = MinimumPixelHitThresholdAdjust + LineThickness;
				OutLineTraceOutput.bIsSurfaceHit = (HdS <= HitBuffer || HdS >= WorldWidth - HitBuffer ||
					HdU <= HitBuffer || HdU >= WorldHeight - HitBuffer);

				return RayHit;
			}
		}
	}

	return FInputRayHit();
}

void UGizmoElementRectangle::SetCenter(FVector InCenter)
{
	Center = InCenter;
}

FVector UGizmoElementRectangle::GetCenter() const
{
	return Center;
}

void UGizmoElementRectangle::SetWidth(float InWidth)
{
	Width = InWidth;
}

float UGizmoElementRectangle::GetWidth() const
{
	return Width;
}

void UGizmoElementRectangle::SetHeight(float InHeight)
{
	Height = InHeight;
}

float UGizmoElementRectangle::GetHeight() const
{
	return Height;
}

void UGizmoElementRectangle::SetUpDirection(const FVector& InUpDirection)
{
	UpDirection = InUpDirection.GetSafeNormal();
}

FVector UGizmoElementRectangle::GetUpDirection() const
{
	return UpDirection;
}

void UGizmoElementRectangle::SetSideDirection(const FVector& InSideDirection)
{
	SideDirection = InSideDirection.GetSafeNormal();
}

FVector UGizmoElementRectangle::GetSideDirection() const
{
	return SideDirection;
}

void UGizmoElementRectangle::SetDrawMesh(bool InDrawMesh)
{
	bDrawMesh = InDrawMesh;
}
bool UGizmoElementRectangle::GetDrawMesh() const
{
	return bDrawMesh;
}

void UGizmoElementRectangle::SetDrawLine(bool InDrawLine)
{
	bDrawLine = InDrawLine;
}

bool UGizmoElementRectangle::GetDrawLine() const
{
	return bDrawLine;
}

void UGizmoElementRectangle::SetHitMesh(bool InHitMesh)
{
	bHitMesh = InHitMesh;
}

bool UGizmoElementRectangle::GetHitMesh() const
{
	return bHitMesh;
}

void UGizmoElementRectangle::SetHitLine(bool InHitLine)
{
	bHitLine = InHitLine;
}

bool UGizmoElementRectangle::GetHitLine() const
{
	return bHitLine;
}

