// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementRenderState.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementRenderState)

FLinearColor FGizmoElementColorAttribute::GetColor() const
{
	if (bHasValue)
	{
		return Value;
	}
	return DefaultColor;
}

void FGizmoElementColorAttribute::SetColor(FLinearColor InColor, bool InOverridesChildState)
{
	Value = InColor;
	bHasValue = true;
	bOverridesChildState = InOverridesChildState;
}

void FGizmoElementColorAttribute::Reset()
{
	Value = DefaultColor;
	bHasValue = false;
	bOverridesChildState = false;
}

void FGizmoElementColorAttribute::UpdateState(const FGizmoElementColorAttribute& InChildColorAttribute)
{
	if (InChildColorAttribute.bHasValue && !(bOverridesChildState && bHasValue))
	{
		Value = InChildColorAttribute.Value;
		bHasValue = true;
		bOverridesChildState = InChildColorAttribute.bOverridesChildState;
	}
}

const UMaterialInterface* FGizmoElementMaterialAttribute::GetMaterial() const
{
	if (Value.IsValid())
	{
		return Value.Get();
	}
	return nullptr;
}

UMaterialInterface* FGizmoElementMaterialAttribute::GetMaterial()
{
	if (Value.IsValid())
	{
		return Value.Get();
	}
	return nullptr;
}

void FGizmoElementMaterialAttribute::SetMaterial(TWeakObjectPtr<UMaterialInterface> InMaterial, bool InOverridesChildState)
{
	Value = InMaterial;
	bOverridesChildState = InOverridesChildState;
}

void FGizmoElementMaterialAttribute::Reset()
{
	Value = nullptr;
	bOverridesChildState = false;
}

void FGizmoElementMaterialAttribute::UpdateState(const FGizmoElementMaterialAttribute& InChildMaterialAttribute)
{
	if (InChildMaterialAttribute.Value != nullptr && !(bOverridesChildState && Value != nullptr))
	{
		Value = InChildMaterialAttribute.Value;
		bOverridesChildState = InChildMaterialAttribute.bOverridesChildState;
	}
}

const UMaterialInterface* FGizmoElementMeshRenderStateAttributes::GetMaterial(EGizmoElementInteractionState InteractionState)
{
	return GetMaterialInternal(InteractionState);
}

const UMaterialInterface* FGizmoElementMeshRenderStateAttributes::GetMaterial(EGizmoElementInteractionState InteractionState) const
{
	return GetMaterialInternal(InteractionState);
}

const UMaterialInterface* FGizmoElementMeshRenderStateAttributes::GetMaterialInternal(EGizmoElementInteractionState InteractionState) const
{
	switch (InteractionState)
	{
	case EGizmoElementInteractionState::Hovering:
		return HoverMaterial.GetMaterial();

	case EGizmoElementInteractionState::Interacting:
		return InteractMaterial.GetMaterial();

	case EGizmoElementInteractionState::Selected:
		return SelectMaterial.GetMaterial() ? SelectMaterial.GetMaterial() : InteractMaterial.GetMaterial();

	case EGizmoElementInteractionState::Subdued:
		return SubdueMaterial.GetMaterial() ? SubdueMaterial.GetMaterial() : Material.GetMaterial();

	default:
		break;
	}

	return Material.GetMaterial();
}

FLinearColor FGizmoElementMeshRenderStateAttributes::GetVertexColor(EGizmoElementInteractionState InteractionState)
{
	return GetVertexColorInternal(InteractionState);
}

FLinearColor FGizmoElementMeshRenderStateAttributes::GetVertexColor(EGizmoElementInteractionState InteractionState) const
{
	return GetVertexColorInternal(InteractionState);
}

FLinearColor FGizmoElementMeshRenderStateAttributes::GetVertexColorInternal(EGizmoElementInteractionState InteractionState) const
{
	switch (InteractionState)
	{
	case EGizmoElementInteractionState::Hovering:
		return HoverVertexColor.GetColor();

	case EGizmoElementInteractionState::Interacting:
		return InteractVertexColor.GetColor();

	case EGizmoElementInteractionState::Selected:
		return SelectVertexColor.bHasValue ? SelectVertexColor.GetColor() : InteractVertexColor.GetColor();

	case EGizmoElementInteractionState::Subdued:
		return SubdueVertexColor.bHasValue ? SubdueVertexColor.GetColor() : VertexColor.GetColor();

	default:
		break;
	}

	return VertexColor.GetColor();
}

void FGizmoElementMeshRenderStateAttributes::Update(FGizmoElementMeshRenderStateAttributes& InChildAttributes)
{
	Material.UpdateState(InChildAttributes.Material);
	HoverMaterial.UpdateState(InChildAttributes.HoverMaterial);
	InteractMaterial.UpdateState(InChildAttributes.InteractMaterial);
	SelectMaterial.UpdateState(InChildAttributes.SelectMaterial);
	SubdueMaterial.UpdateState(InChildAttributes.SubdueMaterial);

	VertexColor.UpdateState(InChildAttributes.VertexColor);
	HoverVertexColor.UpdateState(InChildAttributes.HoverVertexColor);
	InteractVertexColor.UpdateState(InChildAttributes.InteractVertexColor);
	SelectVertexColor.UpdateState(InChildAttributes.SelectVertexColor);
	SubdueVertexColor.UpdateState(InChildAttributes.SubdueVertexColor);
}

FLinearColor FGizmoElementLineRenderStateAttributes::GetLineColor(EGizmoElementInteractionState InteractionState)
{
	return GetLineColorInternal(InteractionState);
}

FLinearColor FGizmoElementLineRenderStateAttributes::GetLineColor(EGizmoElementInteractionState InteractionState) const
{
	return GetLineColorInternal(InteractionState);
}

void FGizmoElementLineRenderStateAttributes::Update(FGizmoElementLineRenderStateAttributes& InChildAttributes)
{
	LineColor.UpdateState(InChildAttributes.LineColor);
	HoverLineColor.UpdateState(InChildAttributes.HoverLineColor);
	InteractLineColor.UpdateState(InChildAttributes.InteractLineColor);
	SelectLineColor.UpdateState(InChildAttributes.SelectLineColor);
	SubdueLineColor.UpdateState(InChildAttributes.SubdueLineColor);
}

FLinearColor FGizmoElementLineRenderStateAttributes::GetLineColorInternal(EGizmoElementInteractionState InteractionState) const
{
	switch (InteractionState)
	{
	case EGizmoElementInteractionState::Hovering:
		return HoverLineColor.GetColor();

	case EGizmoElementInteractionState::Interacting:
		return InteractLineColor.GetColor();

	case EGizmoElementInteractionState::Selected:
		return SelectLineColor.bHasValue ? SelectLineColor.GetColor() : InteractLineColor.GetColor();

	case EGizmoElementInteractionState::Subdued:
		return SubdueLineColor.bHasValue ? SubdueLineColor.GetColor() : LineColor.GetColor();

	default:
		break;
	}

	return LineColor.GetColor();
}
