// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoElementShared.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementShared)

namespace UE::InteractiveToolsFramework::Private
{
	namespace GizmoElementSharedLocals
	{
		/** Shared functionality to reduce boilerplate! */
		template <typename StructType, typename ValueType>
		static ValueType GetValueForState(const StructType& InPerStateValue, const EGizmoElementInteractionState InState)
		{
			switch (InState)
			{
			case EGizmoElementInteractionState::Hovering:
				return InPerStateValue.GetHoverValue();

			case EGizmoElementInteractionState::Interacting:
				return InPerStateValue.GetInteractValue();

			case EGizmoElementInteractionState::Selected:
				return InPerStateValue.GetSelectValue();

			case EGizmoElementInteractionState::Subdued:
				return InPerStateValue.GetSubdueValue();

			case EGizmoElementInteractionState::None:
			default:
				return InPerStateValue.GetDefaultValue();
			}
		}
	}
}

double FGizmoPerStateValueDouble::GetValueForState(const EGizmoElementInteractionState InState) const
{
	return UE::InteractiveToolsFramework::Private::GizmoElementSharedLocals::GetValueForState<FGizmoPerStateValueDouble, double>(*this, InState);
}

const FLinearColor& FGizmoPerStateValueLinearColor::GetValueForState(const EGizmoElementInteractionState InState) const
{
	return UE::InteractiveToolsFramework::Private::GizmoElementSharedLocals::GetValueForState<FGizmoPerStateValueLinearColor, const FLinearColor&>(*this, InState);
}
