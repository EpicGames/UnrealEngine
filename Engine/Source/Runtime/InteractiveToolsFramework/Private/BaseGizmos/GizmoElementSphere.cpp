// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementSphere.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "Materials/MaterialInterface.h"
#include "InputState.h"
#include "Math/Ray.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementSphere)

void UGizmoElementSphere::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
			DrawSphere(
				PDI,
				CurrentRenderState.LocalToWorldTransform.GetTranslation(),
				CurrentRenderState.LocalToWorldTransform.Rotator(),
				Radius * CurrentRenderState.LocalToWorldTransform.GetScale3D(),
				NumSides,
				NumSides,
				UseMaterial->GetRenderProxy(), SDPG_Foreground);
		}
	}
}

FInputRayHit UGizmoElementSphere::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		const FVector WorldCenter = CurrentLineTraceState.LocalToWorldTransform.GetTranslation();
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		const double Scale = CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X;
		const double WorldExtent = Scale * Radius + PixelHitThresholdAdjust;

		const FRay Ray(RayOrigin, RayDirection);
		const double HitDepth = Ray.Dist(WorldCenter);

		if(HitDepth < WorldExtent)
		{
			return MakeRayHit(HitDepth, OutLineTraceOutput);
		}
	}

	return FInputRayHit();
}

void UGizmoElementSphere::SetCenter(const FVector& InCenter)
{
	Center = InCenter;
}

FVector UGizmoElementSphere::GetCenter() const
{
	return Center;
}

void UGizmoElementSphere::SetRadius(float InRadius)
{
	Radius = InRadius;
}

float UGizmoElementSphere::GetRadius() const
{
	return Radius;
}

void UGizmoElementSphere::SetNumSides(int32 InNumSides)
{
	NumSides = InNumSides;
}

int32 UGizmoElementSphere::GetNumSides() const
{
	return NumSides;
}

