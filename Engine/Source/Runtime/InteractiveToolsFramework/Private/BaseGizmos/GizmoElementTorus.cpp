// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementTorus.h"

#include "GizmoPrivateUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "InputState.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "VectorUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementTorus)

void UGizmoElementTorus::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Center, CurrentRenderState);

	if (bVisibleViewDependent)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			const FVector WorldCenter = CurrentRenderState.LocalToWorldTransform.GetLocation();
			const FVector Normal = Axis0 ^ Axis1; 
			const FVector WorldNormal = CurrentRenderState.LocalToWorldTransform.TransformVectorNoScale(Normal);

			const bool bPartial = IsPartial(RenderAPI->GetSceneView(), WorldCenter, WorldNormal);
			FVector BeginAxis = bPartial ? Axis0.RotateAngleAxis(PartialStartAngle, Normal).GetSafeNormal() : Axis0;

			const float PartialAngle = static_cast<float>(PartialEndAngle - PartialStartAngle);			
			if (PartialAngle <= 0.0f)
			{
				return;
			}

			FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

			FVector TorusSideAxis = Normal ^ BeginAxis;
			TorusSideAxis.Normalize();

			const FColor Color = CurrentRenderState.GetCurrentVertexColor().ToFColor(true);

			// Applies flatten scale in local space (so it doesn't affect translation)
			FMatrix RenderLocalToWorldMatrix = CurrentRenderState.LocalToWorldTransform.ToMatrixWithScale();
			RenderLocalToWorldMatrix = RenderLocalToWorldMatrix * CurrentRenderState.SafeFlattenMatrix();
			RenderLocalToWorldMatrix.SetOrigin(CurrentRenderState.LocalToWorldTransform.GetTranslation()); // Restore origin

			DrawTorus(PDI, RenderLocalToWorldMatrix,
				BeginAxis, TorusSideAxis, Color, GetCurrentRadius(), GetCurrentInnerRadius(), NumSegments, NumInnerSlices,
				UseMaterial->GetRenderProxy(), SDPG_Foreground, bPartial, PartialAngle, bEndCaps);
		}
	}
}

//
// This method approximates ray-torus intersection by intersecting the ray with the plane in which the torus 
// lies, then determining a hit point closest to the linear circle defined by torus center and torus outer radius.
// 
// If the torus lies at a glancing angle, the ray-torus intersection is performed against cylinders approximating
// the shape of the torus.
//
FInputRayHit UGizmoElementTorus::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Center, CurrentLineTraceState);

	if (bHittableViewDependent)
	{
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;
		const double WorldOuterRadius = GetCurrentRadius() * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X;
		const double WorldInnerRadius = GetCurrentInnerRadius() * CurrentLineTraceState.LocalToWorldTransform.GetScale3D().X;
		const FVector WorldCenter = CurrentLineTraceState.LocalToWorldTransform.GetLocation();
		const FVector WorldAxis0 = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Axis0);
		const FVector WorldAxis1 = CurrentLineTraceState.LocalToWorldTransform.TransformVectorNoScale(Axis1);
		const FVector WorldNormal = WorldAxis0 ^ WorldAxis1;
		double HitDepth = -1.0;
		bool bIsExactHit = false;

		bool bPartial = IsPartial(ViewContext, WorldCenter, WorldNormal);

		FVector WorldBeginAxis;
		const double PartialAngle = PartialEndAngle - PartialStartAngle;
		if (bPartial)
		{
			if (PartialAngle <= 0)
			{
				return FInputRayHit();
			}

			WorldBeginAxis = WorldAxis0.RotateAngleAxis(PartialStartAngle, WorldNormal).GetSafeNormal();
		}
		else
		{
			WorldBeginAxis = WorldAxis0;
		}

		if (IsGlancingAngle(ViewContext, WorldCenter, WorldNormal))
		{
			// If the torus lies at a glancing angle, the ray-torus intersection is performed against cylinders approximating
			// the shape of the torus.
			static constexpr int32 NumFullTorusCylinders = 16;
			static constexpr double AngleDelta = UE_DOUBLE_TWO_PI / static_cast<double>(NumFullTorusCylinders);
			const int32 NumCylinders = bPartial ? FMath::CeilToInt32(NumFullTorusCylinders * PartialAngle / UE_DOUBLE_TWO_PI) : NumFullTorusCylinders;

			const FVector ViewDirection = ViewContext->GetViewDirection();
			FVector VectorA = WorldBeginAxis; 
			FVector VectorB = VectorA.RotateAngleAxisRad(AngleDelta, WorldNormal);

			const double CylinderRadius = WorldInnerRadius + PixelHitThresholdAdjust;
			double CylinderHeight = (VectorB - VectorA).Length() * WorldOuterRadius;

			if (FMath::IsNearlyZero(CylinderHeight))
			{
				return FInputRayHit();
			}

			const double MinimumPixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * MinimumPixelHitDistanceThreshold;
			const double ExactCylinderRadius = FMath::Max(CurrentLineTraceState.PixelToWorldScale, WorldInnerRadius + MinimumPixelHitThresholdAdjust);

			// due to numerical imprecision, the ray origin needs to be clamped in ortho views
			// (cf. UEditorInteractiveToolsContext::GetRayFromMousePos)
			FVector ClampedRayOrigin(RayOrigin);
			const double DepthBias = UE::GizmoUtil::ClampRayOrigin(ViewContext, ClampedRayOrigin, RayDirection);

			bool bDidIntersect = false;
			
			// Line trace against a set of cylinders approximating the shape of the torus
			for (int32 i = 0; i < NumCylinders; i++)
			{
				if (i > 0)
				{
					VectorA = VectorB;
					VectorB = VectorA.RotateAngleAxisRad(AngleDelta, WorldNormal);
				}

				if (i == NumCylinders - 1)
				{
					CylinderHeight = bPartial ? FMath::Fmod(PartialAngle, AngleDelta) * CylinderHeight : CylinderHeight;
				}

				const FVector CylinderDirection = (VectorB - VectorA).GetSafeNormal();
				const FVector CylinderCenter = WorldCenter + (VectorA * WorldOuterRadius) + CylinderDirection * CylinderHeight * 0.5;			
				bool bIntersects = false;
				double RayParam = -1.0;

				GizmoMath::RayCylinderIntersection(
					CylinderCenter,
					CylinderDirection,
					CylinderRadius,
					CylinderHeight,
					ClampedRayOrigin, RayDirection,
					bIntersects, RayParam);

				bDidIntersect |= bIntersects; 
				
				// Update with closest hit depth
				if (bIntersects && (HitDepth < UE_DOUBLE_SMALL_NUMBER || HitDepth > RayParam))
				{
					HitDepth = RayParam;

					// We successfully hit, now check if we intersect the actual surface (rather than just within the PixelHitDistanceThreshold)
					GizmoMath::RayCylinderIntersection(
						CylinderCenter,
						CylinderDirection,
						ExactCylinderRadius,
						CylinderHeight,
						ClampedRayOrigin, RayDirection,
						bIntersects, RayParam);

					bIsExactHit = bIntersects;
				}
			}

			if (bDidIntersect)
			{
				// add the depth bias if any
				HitDepth += DepthBias;
			}
		}
		else
		{
			// Intersect ray with plane in which torus lies.
			FPlane Plane(WorldCenter, WorldNormal);
			HitDepth = FMath::RayPlaneIntersectionParam(RayOrigin, RayDirection, Plane);
			if (HitDepth < UE_DOUBLE_SMALL_NUMBER)
			{
				return FInputRayHit();
			}

			FVector HitPoint = RayOrigin + RayDirection * HitDepth;

			// Find the closest point on the circle to the intersection point
			FVector NearestCirclePos;
			GizmoMath::ClosetPointOnCircle(HitPoint, WorldCenter, WorldNormal, static_cast<float>(WorldOuterRadius), NearestCirclePos);

			// Find the closest point on the ray to the circle and determine if it is within the torus
			FRay Ray(RayOrigin, RayDirection, true);
			FVector NearestRayPos = Ray.ClosestPoint(NearestCirclePos);

			const double HitBuffer = PixelHitThresholdAdjust + WorldInnerRadius;
			double Distance = FVector::Distance(NearestCirclePos, NearestRayPos);

			if (Distance > HitBuffer)
			{
				return FInputRayHit();
			}

			bIsExactHit = (Distance <= FMath::Max(CurrentLineTraceState.PixelToWorldScale * 2.0f, WorldInnerRadius));

			// Handle partial torus
			if (bPartial)
			{
				// Compute projected angle
				FVector HitVec = (NearestCirclePos - WorldCenter).GetSafeNormal();
				double HitAngle = UE::Geometry::VectorUtil::PlaneAngleSignedR(WorldBeginAxis, HitVec, WorldNormal); 

				if (HitAngle < UE_DOUBLE_SMALL_NUMBER)
				{
					HitAngle = UE_DOUBLE_TWO_PI + HitAngle;
				}

				if (HitAngle > PartialAngle)
				{
					return FInputRayHit();
				}
			}

			// Adjust hit depth to return the closest hit point's depth
			HitDepth = (NearestRayPos - RayOrigin).Length();
		}

		if (HitDepth >= UE_DOUBLE_SMALL_NUMBER)
		{
			FInputRayHit RayHit = MakeRayHit(HitDepth, OutLineTraceOutput);
			OutLineTraceOutput.bIsSurfaceHit = bIsExactHit;

			return RayHit;
		}
	}
	
	return FInputRayHit();
}

double UGizmoElementTorus::GetCurrentInnerRadius() const
{
	auto GetMultipliedInnerRadius = [this](const float InMultiplier) -> float
	{
		return (InnerRadius > UE_SMALL_NUMBER ? InnerRadius * InMultiplier : InMultiplier);
	};

	return GetMultipliedInnerRadius(InnerRadiusMultiplier.GetValueForState(ElementInteractionState));
}

bool UGizmoElementTorus::IsGlancingAngle(const UGizmoViewContext* View, const FVector& InWorldCenter, const FVector& InWorldNormal)
{
	check(View);
	FVector ViewToCircleBaseCenter;
	if (View->IsPerspectiveProjection())
	{
		ViewToCircleBaseCenter = (InWorldCenter - View->ViewLocation).GetSafeNormal();
	}
	else
	{
		ViewToCircleBaseCenter = View->GetViewDirection();
	}

	// Determine if the ray direction is at a glancing angle by using a minimum cos angle based on the
	// angle between the vector from arc center to ring center and the vector from arc center to ring edge.
	double MinCosAngle =
		FMath::Max(InnerRadius * 1.5 / FMath::Sqrt(Radius * Radius + InnerRadius * InnerRadius),
			DefaultViewDependentPlanarMinCosAngleTol);

	double DotP = FMath::Abs(FVector::DotProduct(InWorldNormal, ViewToCircleBaseCenter));
	return (DotP <= MinCosAngle);
}

void UGizmoElementTorus::SetInnerRadius(double InInnerRadius)
{
	InnerRadius = InInnerRadius;
}

double UGizmoElementTorus::GetInnerRadius() const
{
	return InnerRadius;
}

void UGizmoElementTorus::SetHoverInnerRadiusMultiplier(double InHoverInnerRadiusMultiplier)
{
	InnerRadiusMultiplier.Hover = InHoverInnerRadiusMultiplier;
}

double UGizmoElementTorus::GetHoverInnerRadiusMultiplier() const
{
	return InnerRadiusMultiplier.GetHoverValue();
}

void UGizmoElementTorus::SetInteractInnerRadiusMultiplier(double InInteractInnerRadiusMultiplier)
{
	InnerRadiusMultiplier.Interact = InInteractInnerRadiusMultiplier;
}

double UGizmoElementTorus::GetInteractInnerRadiusMultiplier() const
{
	return InnerRadiusMultiplier.GetInteractValue();
}

void UGizmoElementTorus::SetSelectInnerRadiusMultiplier(double InSelectInnerRadiusMultiplier)
{
	InnerRadiusMultiplier.Select = InSelectInnerRadiusMultiplier;
}

double UGizmoElementTorus::GetSelectInnerRadiusMultiplier() const
{
	return InnerRadiusMultiplier.GetSelectValue();
}

void UGizmoElementTorus::SetSubdueInnerRadiusMultiplier(double InSubdueInnerRadiusMultiplier)
{
	InnerRadiusMultiplier.Subdue = InSubdueInnerRadiusMultiplier;
}

double UGizmoElementTorus::GetSubdueInnerRadiusMultiplier() const
{
	return InnerRadiusMultiplier.GetSubdueValue();
}

void UGizmoElementTorus::SetNumInnerSlices(int32 InNumInnerSlices)
{
	NumInnerSlices = InNumInnerSlices;
}

int32 UGizmoElementTorus::GetNumInnerSlices() const
{
	return NumInnerSlices;
}

void UGizmoElementTorus::SetEndCaps(bool InEndCaps)
{
	bEndCaps = InEndCaps;
}

bool UGizmoElementTorus::GetEndCaps() const
{
	return bEndCaps;
}

