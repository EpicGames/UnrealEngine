// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/GizmoElementTriangleList.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "Materials/MaterialInterface.h"
#include "InputState.h"
#include "Math/Ray.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMeshBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoElementTriangleList)

void UGizmoElementTriangleList::Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState)
{
	FRenderTraversalState CurrentRenderState(RenderState);
	const bool bVisibleViewDependent = UpdateRenderState(RenderAPI, Base, CurrentRenderState);

	if (bVisibleViewDependent && Vertices.Num() > 2)
	{
		if (const UMaterialInterface* UseMaterial = CurrentRenderState.GetCurrentMaterial())
		{
			FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
			ComputeProjectedVertices(RenderState.LocalToWorldTransform, true);

			FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
			for(int32 Index = 0; Index < ProjectedVertices.Num() - 2;)
			{
				const FDynamicMeshVertex A((FVector3f)ProjectedVertices[Index++]);
				const FDynamicMeshVertex B((FVector3f)ProjectedVertices[Index++]);
				const FDynamicMeshVertex C((FVector3f)ProjectedVertices[Index++]);
				MeshBuilder.AddTriangle(MeshBuilder.AddVertex(A), MeshBuilder.AddVertex(B), MeshBuilder.AddVertex(C));
			}
			MeshBuilder.Draw(PDI,FMatrix::Identity,UseMaterial->GetRenderProxy(),SDPG_Foreground,0.f);
		}
	}
}

FInputRayHit UGizmoElementTriangleList::LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput)
{
	FLineTraceTraversalState CurrentLineTraceState(LineTraceState);
	const bool bHittableViewDependent = UpdateLineTraceState(ViewContext, Base, CurrentLineTraceState);

	if (bHittableViewDependent && Vertices.Num() > 2)
	{
		ComputeProjectedVertices(LineTraceState.LocalToWorldTransform, false);
		const double PixelHitThresholdAdjust = CurrentLineTraceState.PixelToWorldScale * PixelHitDistanceThreshold;

		const FRay Ray(RayOrigin, RayDirection);

		double HitDepth = 0.0;
		for(int32 Index = 0; Index < ProjectedVertices.Num() - 2;)
		{
			const FVector& A = ProjectedVertices[Index++];
			const FVector& B = ProjectedVertices[Index++];
			const FVector& C = ProjectedVertices[Index++];
			if (Intersect(Ray, A, B, C, PixelHitThresholdAdjust, HitDepth))
			{
				return MakeRayHit(HitDepth, OutLineTraceOutput);
			}
		}
	}

	return FInputRayHit();
}

void UGizmoElementTriangleList::SetVertices(const TArrayView<const FVector>& InVertices)
{
	Vertices = InVertices;
}

const TArray<FVector>& UGizmoElementTriangleList::GetVertices() const
{
	return Vertices;
}

void UGizmoElementTriangleList::SetBase(FVector InBase)
{
	Base = InBase;
}

FVector UGizmoElementTriangleList::GetBase() const
{
	return Base;
}

void UGizmoElementTriangleList::SetUpDirection(const FVector& InUpDirection)
{
	UpDirection = InUpDirection.GetSafeNormal();
}

FVector UGizmoElementTriangleList::GetUpDirection() const
{
	return UpDirection;
}

void UGizmoElementTriangleList::SetSideDirection(const FVector& InSideDirection)
{
	SideDirection = InSideDirection.GetSafeNormal();
}

FVector UGizmoElementTriangleList::GetSideDirection() const
{
	return SideDirection;
}

void UGizmoElementTriangleList::ComputeProjectedVertices(const FTransform& InLocalToWorldTransform, bool bIncludeBase)
{
	const FVector WorldUpAxis = InLocalToWorldTransform.TransformVectorNoScale(UpDirection);
	const FVector WorldSideAxis = InLocalToWorldTransform.TransformVectorNoScale(SideDirection);
    const FVector WorldNormal = FVector::CrossProduct(WorldSideAxis, WorldUpAxis);
	const FVector WorldCenter = InLocalToWorldTransform.TransformPosition(Base);
	const FVector WorldScale = InLocalToWorldTransform.GetScale3D();

	ProjectedVertices.Reset();
	ProjectedVertices.Reserve(Vertices.Num());

	for(int32 Index = 0; Index < Vertices.Num(); Index++)
	{
		FVector Vertex = Vertices[Index] * WorldScale;
		ProjectedVertices.Add(WorldCenter + WorldNormal * Vertex.X + WorldSideAxis * Vertex.Y + WorldUpAxis * Vertex.Z);
	}
}

bool UGizmoElementTriangleList::Intersect(const FRay& Ray, const FVector& InA, const FVector& InB, const FVector& InC, double Tolerance, double& Distance)
{
	const FVector Center = (InA + InB + InC) / 3.0;
	const FVector A = InA + (InA - Center).GetSafeNormal() * Tolerance;
	const FVector B = InB + (InB - Center).GetSafeNormal() * Tolerance;
	const FVector C = InC + (InC - Center).GetSafeNormal() * Tolerance;
	const FVector BA = A - B;
	const FVector CB = B - C;
	const FVector TriNormal = BA ^ CB;

	FVector Intersection;
	bool bCollide = FMath::SegmentPlaneIntersection(
		Ray.Origin,
		Ray.Origin + Ray.Direction * 10000.0,
		FPlane(A, TriNormal), Intersection);
	if (!bCollide)
	{
		return false;
	}

	const FVector BaryCentric = FMath::ComputeBaryCentric2D(Intersection, A, B, C);
	if (BaryCentric.X > 0.0f && BaryCentric.Y > 0.0f && BaryCentric.Z > 0.0f)
	{
		Distance = 0.0;
		return true;
	}
	return false;
}