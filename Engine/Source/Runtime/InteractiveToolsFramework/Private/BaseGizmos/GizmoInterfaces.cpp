// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseGizmos/GizmoMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoInterfaces)



void IGizmoAxisSource::GetAxisFrame(
	FVector& PlaneNormalOut, FVector& PlaneAxis1Out, FVector& PlaneAxis2Out) const
{
	PlaneNormalOut = GetDirection();
	if (HasTangentVectors())
	{
		GetTangentVectors(PlaneAxis1Out, PlaneAxis2Out);
	}
	else
	{
		GizmoMath::MakeNormalPlaneBasis(PlaneNormalOut, PlaneAxis1Out, PlaneAxis2Out);
	}
}

