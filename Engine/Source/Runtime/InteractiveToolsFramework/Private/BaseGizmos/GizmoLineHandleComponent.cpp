// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoLineHandleComponent.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoLineHandleComponent)



class FGizmoLineHandleComponentSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FGizmoLineHandleComponentSceneProxy(const UGizmoLineHandleComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent),
		Color(InComponent->Color),
		Normal(InComponent->Normal),
		Direction(InComponent->Direction),
		HandleSize(InComponent->HandleSize),
		Thickness(InComponent->Thickness),
		bBoundaryOnly(false),
		bImageScale(InComponent->bImageScale),
		HoverThicknessMultiplier(InComponent->HoverSizeMultiplier)
	{
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		FVector LocalOffset = Direction;
		if (ExternalDistance != nullptr)
		{
			LocalOffset *= (*ExternalDistance);
		}

		const FMatrix& LocalToWorldMatrix = GetLocalToWorld();
		FVector WorldOrigin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);

		float IntervalMarkerSize = HandleSize;
		FVector WorldIntervalEnd = LocalToWorldMatrix.TransformPosition(LocalOffset + IntervalMarkerSize * Normal);
		FVector WorldDiskOrigin = LocalToWorldMatrix.TransformPosition(LocalOffset);
		FVector WorldBaseOrigin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
				bool bIsOrtho = !View->IsPerspectiveProjection();
				FVector UpVector = View->GetViewUp();
				FVector ViewVector = View->GetViewDirection();

				float LengthScale = 1;
				bool bIsViewDependent = (bExternalIsViewDependent) ? (*bExternalIsViewDependent) : false;
				if (bIsViewDependent)
				{
					LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, WorldDiskOrigin);
				}

				FVector ScaledIntevalStart = -LengthScale * (WorldIntervalEnd - WorldDiskOrigin) + WorldDiskOrigin;
				FVector ScaledIntevalEnd = LengthScale * (WorldIntervalEnd - WorldDiskOrigin) + WorldDiskOrigin;
				FVector ScaledDiskOrigin = 0.5 * (ScaledIntevalStart + ScaledIntevalEnd);

				float UseThickness = (bExternalHoverState != nullptr && *bExternalHoverState == true) ?
					(HoverThicknessMultiplier * Thickness) : (Thickness);
				if (!bIsOrtho)
				{
					UseThickness *= (View->FOV / 90.0f);		// compensate for FOV scaling in Gizmos...
				}

				// From base origin to disk origin
				PDI->DrawLine(WorldBaseOrigin, ScaledDiskOrigin, Color, SDPG_Foreground, UseThickness, 0.0f, true);
				// Draw the interval marker
				PDI->DrawLine(ScaledIntevalStart, ScaledIntevalEnd, Color, SDPG_Foreground, 2 * UseThickness, 0.0f, true);
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return false;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return IntCastChecked<uint32>( sizeof *this + GetAllocatedSize() ); }
	SIZE_T GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }


	void SetExternalHoverState(bool* HoverState)
	{
		bExternalHoverState = HoverState;
	}

	void SetExternalWorldLocalState(bool* bWorldLocalState)
	{
		bExternalWorldLocalState = bWorldLocalState;
	}

	void SetExternalIsViewDependent(bool* bExternalIsViewDependentIn)
	{
		bExternalIsViewDependent = bExternalIsViewDependentIn;
	}

	void SetLengthScale(float* Distance)
	{
		ExternalDistance = Distance;
	}

private:
	FLinearColor Color;
	FVector Normal;
	FVector Direction;
	float HandleSize;
	float Thickness;
	bool bBoundaryOnly;
	bool bImageScale;
	float HoverThicknessMultiplier;

	// set on Component for use in ::GetDynamicMeshElements()
	bool* bExternalHoverState = nullptr;
	bool* bExternalWorldLocalState = nullptr;
	float* ExternalDistance = nullptr;
	bool* bExternalIsViewDependent = nullptr;
};


FPrimitiveSceneProxy* UGizmoLineHandleComponent::CreateSceneProxy()
{
	FGizmoLineHandleComponentSceneProxy* NewProxy = new FGizmoLineHandleComponentSceneProxy(this);
	NewProxy->SetExternalHoverState(&bHovering);
	NewProxy->SetExternalWorldLocalState(&bWorld);
	NewProxy->SetLengthScale(&Length);
	NewProxy->SetExternalIsViewDependent(&bIsViewDependent);
	return NewProxy;
}

FBoxSphereBounds UGizmoLineHandleComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// the handle looks like
	//                         ------|
	// where '------' has length "Length" and '|' is of length 2*Handlesize
	//  
	float Radius = FMath::Sqrt(Length * Length + HandleSize * HandleSize);
	return FBoxSphereBounds(FSphere(FVector::ZeroVector, Radius).TransformBy(LocalToWorld));
}

bool UGizmoLineHandleComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	const FTransform& Transform = this->GetComponentToWorld();
	FVector WorldBaseOrigin = Transform.TransformPosition(FVector::ZeroVector);
	float PixelToWorldScale = 1;
	if (bIsViewDependent)
	{
		PixelToWorldScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(GizmoViewContext, WorldBaseOrigin);
	}

	float LengthScale = (bImageScale) ? PixelToWorldScale : 1.f;
	double UseHandleSize = LengthScale * HandleSize;
	FVector LocalOffset = Length * Direction;

	FVector HandleDir = (bWorld) ? Normal : Transform.TransformVector(Normal);
	FVector WorldHandleOrigin = Transform.TransformPosition(LocalOffset);

	FVector BaseToHandle = WorldHandleOrigin - WorldBaseOrigin;

	// where the handle crosses the connecting line
	FVector ScaledHandleOrigin = LengthScale * BaseToHandle + WorldBaseOrigin;

	// start and end point of the handle.
	FVector HandleStart = ScaledHandleOrigin + LengthScale * HandleDir;
	FVector HandleEnd = ScaledHandleOrigin - LengthScale * HandleDir;

	FVector NearestOnHandle, NearestOnLine;
	FMath::SegmentDistToSegmentSafe(HandleStart, HandleEnd, Start, End, NearestOnHandle, NearestOnLine);
	double Distance = FVector::Distance(NearestOnHandle, NearestOnLine);
	if (Distance > PixelHitDistanceThreshold * PixelToWorldScale)
	{
		return false;
	}

	OutHit.Component = this;
	OutHit.Distance = static_cast<float>(FVector::Distance(Start, NearestOnLine));
	OutHit.ImpactPoint = NearestOnLine;
	return true;

}
