// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoMath.h"

#include "Algo/ForEach.h"
#include "DynamicMeshBuilder.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveDrawInterface.h"
#include "SceneManagement.h"
#include "SceneView.h"

FVector GizmoMath::ProjectPointOntoLine(
	const FVector& Point,
	const FVector& LineOrigin, const FVector& LineDirection)
{
	double ProjectionParam = FVector::DotProduct((Point - LineOrigin), LineDirection);
	return LineOrigin + ProjectionParam * LineDirection;
}


void GizmoMath::NearestPointOnLine(
	const FVector& LineOrigin, const FVector& LineDirection,
	const FVector& QueryPoint,
	FVector& NearestPointOut, float& LineParameterOut)
{
	check(LineDirection.IsNormalized());
	double LineParameter = FVector::DotProduct( (QueryPoint - LineOrigin), LineDirection);
	NearestPointOut = LineOrigin + LineParameter * LineDirection;
	LineParameterOut = (float)LineParameter;
}


void GizmoMath::NearestPointOnLineToRay(
	const FVector& LineOrigin, const FVector& LineDirection,
	const FVector& RayOrigin, const FVector& RayDirection,
	FVector& NearestLinePointOut, float& LineParameterOut,
	FVector& NearestRayPointOut, float& RayParameterOut)
{
	FVector kDiff = LineOrigin - RayOrigin;
	double a01 = -FVector::DotProduct(LineDirection, RayDirection);
	double b0 = FVector::DotProduct(kDiff, LineDirection);
	double c = kDiff.SizeSquared();
	double det = FMath::Abs((double)1 - a01 * a01);
	double b1, s0, s1;

	if (det >= SMALL_NUMBER)
	{
		b1 = -FVector::DotProduct(kDiff, RayDirection);
		s1 = a01 * b0 - b1;

		if (s1 >= (double)0)
		{
			// Two interior points are closest, one on Line and one on Ray
			double invDet = ((double)1) / det;
			s0 = (a01 * b1 - b0) * invDet;
			s1 *= invDet;
		}
		else
		{
			// Origin of Ray and interior point of Line are closest.
			s0 = -b0;
			s1 = (double)0;
		}
	}
	else
	{
		// Lines are parallel, closest pair with one point at Ray origin.
		s0 = -b0;
		s1 = (double)0;
	}

	NearestLinePointOut = LineOrigin + s0 * LineDirection;
	NearestRayPointOut = RayOrigin + s1 * RayDirection;
	LineParameterOut = (float)s0;
	RayParameterOut = (float)s1;
}




void GizmoMath::RayPlaneIntersectionPoint(
	const FVector& PlaneOrigin, const FVector& PlaneNormal,
	const FVector& RayOrigin, const FVector& RayDirection,
	bool& bIntersectsOut, FVector& PlaneIntersectionPointOut)
{
	bIntersectsOut = false;
	PlaneIntersectionPointOut = PlaneOrigin;

	double PlaneEquationD = -FVector::DotProduct(PlaneOrigin, PlaneNormal);
	double NormalDot = FVector::DotProduct(RayDirection, PlaneNormal);

	if (FMath::Abs(NormalDot) < SMALL_NUMBER)
	{
		return;
	}

	double RayParam = -( FVector::DotProduct(RayOrigin, PlaneNormal) + PlaneEquationD) / NormalDot;
	if (RayParam < 0)
	{
		return;
	}

	PlaneIntersectionPointOut = RayOrigin + RayParam * RayDirection;
	bIntersectsOut = true;
}


void GizmoMath::RaySphereIntersection(
	const FVector& SphereOrigin, const float SphereRadius,
	const FVector& RayOrigin, const FVector& RayDirection,
	bool& bIntersectsOut, FVector& SphereIntersectionPointOut)
{
	bIntersectsOut = false;
	SphereIntersectionPointOut = RayOrigin;
	 
	FVector DeltaPos = RayOrigin - SphereOrigin;
	double a0 = DeltaPos.SizeSquared() - (double)SphereRadius*(double)SphereRadius;
	double a1 = FVector::DotProduct(RayDirection, DeltaPos);
	double discr = a1 * a1 - a0;
	if (discr > 0)   // intersection only when roots are real
	{
		bIntersectsOut = true;
		double root = FMath::Sqrt(discr);
		double NearRayParam = -a1 + root;		// isn't it always this one?
		double NearRayParam2 = -a1 - root;
		double UseRayParam = FMath::Min(NearRayParam, NearRayParam2);
		SphereIntersectionPointOut = RayOrigin + UseRayParam * RayDirection;
	}
}


template<typename RealType>
void GizmoMath::RayCylinderIntersection(
	const FVector& CylinderCenter, const FVector& CylinderAxis, RealType CylinderRadius, RealType CylinderHeight,
	const FVector& RayOrigin, const FVector& RayDirection,
	bool& bIntersectsOut, RealType& OutRayParam)
{
	// adapted from GeometricTools GTEngine
	// https://www.geometrictools.com/GTE/Mathematics/IntrRay3Cylinder3.h and
	// https://www.geometrictools.com/GTE/Mathematics/IntrLine3Cylinder3.h 
	// (Engine/Plugins/Runtime/GeometryProcessing/Source/GeometryAlgorithms/Private/ThirdParty/GTEngine/Mathematics/GteIntrRay3Cylinder3.h and
	//  Engine/Plugins/Runtime/GeometryProcessing/Source/GeometryAlgorithms/Private/ThirdParty/GTEngine/Mathematics/GteIntrLine3Cylinder3.h)	

	// The cylinder axis is a line.  The origin of the cylinder is chosen to be
	// the line origin.  The cylinder wall is at a distance R units from the axis.
	// An infinite cylinder has infinite height.  A finite cylinder has center C
	// at the line origin and has a finite height H.  The segment for the finite
	// cylinder has endpoints C-(H/2)*D and C+(H/2)*D where D is a unit-length
	// direction of the line.

	// Initialize the result as if there is no intersection.  If we discover
	// an intersection, these values will be modified accordingly.
	bIntersectsOut  = false;
	int NumIntersections = 0;
	RealType RayParam[2];

	// Create a coordinate system for the cylinder.  In this system, the
	// cylinder segment center C is the origin and the cylinder axis direction
	// W is the z-axis.  U and V are the other coordinate axis directions.
	// If P = x*U+y*V+z*W, the cylinder is x^2 + y^2 = r^2, where r is the
	// cylinder radius.  The end caps are |z| = h/2, where h is the cylinder
	// height.
	FVector Basis[3];  // {W, U, V}
	Basis[0] = CylinderAxis.GetSafeNormal();
	// @todo: replace with MakePerpVectors() once it has been moved out of plugins
	Basis[1] = GetOrthogonalVector(Basis[0]).GetSafeNormal();
	Basis[2] = (Basis[0] ^ Basis[1]).GetSafeNormal();

	RealType HalfHeight = RealType(0.5) * CylinderHeight;
	RealType RadiusSquared = CylinderRadius * CylinderRadius;

	// Convert incoming line origin to capsule coordinates.
	FVector Diff = RayOrigin - CylinderCenter;
	FVector P(Basis[1] | Diff, Basis[2] | Diff, Basis[0] | Diff);

	// Get the z-value, in cylinder coordinates, of the incoming line's
	// unit-length direction.
	RealType Dz = static_cast<RealType>( Basis[0] | RayDirection );
	if (FMath::IsNearlyEqual(Dz, 1.0, SMALL_NUMBER)) 
	{
		// The line is parallel to the cylinder axis.  Determine whether the
		// line intersects the cylinder end disks.
		RealType RadialSquaredDist = static_cast<RealType>( RadiusSquared - P[0] * P[0] - P[1] * P[1]);
		if (RadialSquaredDist >= 0.0)
		{
			// The line intersects the cylinder end disks.
			NumIntersections = 2;
			if (Dz > 0.0)
			{
				RayParam[0] = static_cast<RealType>( -P[2] - HalfHeight );
				RayParam[1] = static_cast<RealType>( -P[2] + HalfHeight );
			}
			else
			{
				RayParam[0] = static_cast<RealType>( P[2] - HalfHeight );
				RayParam[1] = static_cast<RealType>( P[2] + HalfHeight );
			}
		}
		// else:  The line is outside the cylinder, no intersection.
	}
	else
	{
		// Convert the incoming line unit-length direction to cylinder
		// coordinates.
		FVector D((Basis[1] | RayDirection), (Basis[2] | RayDirection), Dz);

		RealType A0, A1, A2, Discr, Root, Inv, TValue;

		if (FMath::IsNearlyZero(D[2]))
		{
			// The line is perpendicular to the cylinder axis.
			if (FMath::Abs(P[2]) <= HalfHeight)
			{
				// Test intersection of line P+t*D with infinite cylinder
				// x^2+y^2 = r^2.  This reduces to computing the Roots of a
				// quadratic equation.  If P = (px,py,pz) and D = (dx,dy,dz),
				// then the quadratic equation is
				//   (dx^2+dy^2)*t^2 + 2*(px*dx+py*dy)*t + (px^2+py^2-r^2) = 0
				A0 = static_cast<RealType>( P[0] * P[0] + P[1] * P[1] - RadiusSquared );
				A1 = static_cast<RealType>( P[0] * D[0] + P[1] * D[1] );
				A2 = static_cast<RealType>( D[0] * D[0] + D[1] * D[1] );
				Discr = A1 * A1 - A0 * A2;
				if (FMath::IsNearlyZero(Discr)) 
				{
					// The line is tangent to the cylinder.
					NumIntersections = 1;
					RayParam[0] = -A1 / A2;
					RayParam[1] = RayParam[0];
				}
				else if (Discr > 0.0)
				{
					// The line intersects the cylinder in two places.
					NumIntersections = 2;
					Root = FMath::Sqrt(Discr);
					Inv = 1.0f / A2;
					RayParam[0] = (-A1 - Root) * Inv;
					RayParam[1] = (-A1 + Root) * Inv;
				}				
				// else: The line does not intersect the cylinder.
			}
			// else: The line is outside the planes of the cylinder end disks.
		}
		else
		{
			// Test for intersections with the planes of the end disks.
			Inv = static_cast<RealType>( 1.0 / D[2] );

			RealType T0 = static_cast<RealType>( (-HalfHeight - P[2]) * Inv );
			RealType TmpX = static_cast<RealType>( P[0] + T0 * D[0] );
			RealType TmpY = static_cast<RealType>( P[1] + T0 * D[1] );
			if (TmpX * TmpX + TmpY * TmpY <= RadiusSquared)
			{
				// Plane intersection inside the top cylinder end disk.
				RayParam[NumIntersections++] = T0;
			}

			RealType T1 = static_cast<RealType>( (+HalfHeight - P[2]) * Inv );
			TmpX = static_cast<RealType>( P[0] + T1 * D[0] );
			TmpY = static_cast<RealType>( P[1] + T1 * D[1] );
			if (TmpX * TmpX + TmpY * TmpY <= RadiusSquared)
			{
				// Plane intersection inside the bottom cylinder end disk.
				RayParam[NumIntersections++] = T1;
			}

			if (NumIntersections < 2)
			{
				// Test for intersection with the cylinder wall.
				A0 = static_cast<RealType>( P[0] * P[0] + P[1] * P[1] - RadiusSquared );
				A1 = static_cast<RealType>( P[0] * D[0] + P[1] * D[1] );
				A2 = static_cast<RealType>( D[0] * D[0] + D[1] * D[1] );
				Discr = A1 * A1 - A0 * A2;
				if (FMath::IsNearlyZero(Discr))
				{
					TValue = -A1 / A2;
					if (T0 <= T1)
					{
						if (T0 <= TValue && TValue <= T1)
						{
							RayParam[NumIntersections++] = TValue;
						}
					}
					else
					{
						if (T1 <= TValue && TValue <= T0)
						{
							RayParam[NumIntersections++] = TValue;
						}
					}
				}
				else if (Discr > 0.0)
				{
					Root = FMath::Sqrt(Discr);
					Inv = (1.0f) / A2;
					TValue = (-A1 - Root) * Inv;
					if (T0 <= T1)
					{
						if (T0 <= TValue && TValue <= T1)
						{
							RayParam[NumIntersections++] = TValue;
						}
					}
					else
					{
						if (T1 <= TValue && TValue <= T0)
						{
							RayParam[NumIntersections++] = TValue;
						}
					}

					if (NumIntersections < 2)
					{
						TValue = (-A1 + Root) * Inv;
						if (T0 <= T1)
						{
							if (T0 <= TValue && TValue <= T1)
							{
								RayParam[NumIntersections++] = TValue;
							}
						}
						else
						{
							if (T1 <= TValue && TValue <= T0)
							{
								RayParam[NumIntersections++] = TValue;
							}
						}
					}
					// else: Line intersects end disk and cylinder wall.
				}
				// else: Line does not intersect cylinder wall.
			}
			// else: Line intersects both top and bottom cylinder end disks.

			if (NumIntersections == 2)
			{
				if (RayParam[0] > RayParam[1])
				{
					RealType TmpT = RayParam[0];
					RayParam[0] = RayParam[1];
					RayParam[1] = TmpT;
				}
			}
			else if (NumIntersections == 1)
			{
				RayParam[1] = RayParam[0];
			}
		}
	}

	// Get rid of hits before ray origin
	if (NumIntersections > 0 && RayParam[0] >= RealType(0))
	{
		bIntersectsOut = true;
		OutRayParam = RayParam[0];
	}
	else if (NumIntersections == 2 && RayParam[1] >= RealType(0))
	{
		bIntersectsOut = true;
		OutRayParam = RayParam[1];
	}
	else
	{
		bIntersectsOut = false;
	}
}

template<typename RealType>
void GizmoMath::RayConeIntersection(
	const FVector& ConeCenter, const FVector& ConeDirection, RealType ConeCosAngle, RealType ConeHeight,
	const FVector& RayOrigin, const FVector& RayDirection,
	bool& bIntersectsOut, RealType& OutRayParam)
{
	// adapted from GeometricTools GTEngine
	// https://www.geometrictools.com/GTE/Mathematics/IntrRay3Cone3.h and
	// https://www.geometrictools.com/GTE/Mathematics/IntrLine3Cone3.h 
	// (Engine/Plugins/Runtime/GeometryProcessing/Source/GeometryAlgorithms/Private/ThirdParty/GTEngine/Mathematics/GteIntrRay3Cone3.h and
	//  Engine/Plugins/Runtime/GeometryProcessing/Source/GeometryAlgorithms/Private/ThirdParty/GTEngine/Mathematics/GteIntrLine3Cone3.h)	

	// The cone has vertex V, unit-length axis direction D, angle theta in
	// (0,pi/2), and height h in (0,+infinity).  The line is P + t*U, where U
	// is a unit-length direction vector.  Define g = cos(theta).  The cone
	// is represented by
	//   (X-V)^T * (D*D^T - g^2*I) * (X-V) = 0,  0 <= Dot(D,X-V) <= h
	// The first equation defines a double-sided cone.  The first inequality
	// in the second equation limits this to a single-sided cone containing
	// the ray V + s*D with s >= 0.  We will call this the 'positive cone'.
	// The single-sided cone containing ray V + s * t with s <= 0 is called
	// the 'negative cone'.  The double-sided cone is the union of the
	// positive cone and negative cone.  The second inequality in the second
	// equation limits the single-sided cone to the region bounded by the
	// height.  Setting X(t) = P + t*U, the equations are
	//   C2*t^2 + 2*C1*t + C0 = 0,  0 <= Dot(D,U)*t + Dot(D,P-V) <= h
	// where
	//   C2 = Dot(D,U)^2 - g^2
	//   C1 = Dot(D,U)*Dot(D,P-V) - g^2*Dot(U,P-V)
	//   C0 = Dot(D,P-V)^2 - g^2*Dot(P-V,P-V)
	// The following code computes the t-interval that satisfies the quadratic
	// equation subject to the linear inequality constraints.

	FVector PmV = RayOrigin - ConeCenter;
	RealType DdU = static_cast<RealType>( FVector::DotProduct(ConeDirection, RayDirection) );
	RealType DdPmV = static_cast<RealType>( FVector::DotProduct(ConeDirection, PmV) );
	RealType UdPmV = static_cast<RealType>( FVector::DotProduct(RayDirection, PmV) );
	RealType PmVdPmV = static_cast<RealType>( FVector::DotProduct(PmV, PmV) );
	RealType CosAngleSqr = ConeCosAngle * ConeCosAngle;
	RealType C2 = DdU * DdU - CosAngleSqr;
	RealType C1 = DdU * DdPmV - CosAngleSqr * UdPmV;
	RealType C0 = DdPmV * DdPmV - CosAngleSqr * PmVdPmV;
	RealType T;
	RealType RayParam[2];

	if (!FMath::IsNearlyZero(C2))
	{
		RealType Discr = C1 * C1 - C0 * C2;
		if (FMath::IsNearlyZero(Discr))
		{
			// One repeated real Root; the line is tangent to the double-sided
			// cone at a single point.  Report only the point if it is on the
			// positive cone.
			T = -C1 / C2;
			if (DdU * T + DdPmV >= 0.0)
			{
				RayParam[0] = T;
				RayParam[1] = T;
			}
			else
			{
				bIntersectsOut = false;
				return;
			}
		}
		else if (Discr < 0.0)
		{
			// The quadratic has no real-valued Roots.  The line does not
			// intersect the double-sided cone.
			bIntersectsOut = false;
			return;
		}
		else // (Discr > 0.0)
		{
			// The quadratic has two distinct real-valued Roots.  However, one
			// or both of them might intersect the negative cone.  We are
			// interested only in those intersections with the positive cone.
			RealType Root = FMath::Sqrt(Discr);
			RealType InvC2 = RealType(1) / C2;
			int NumIntersections = 0;

			T = (-C1 - Root) * InvC2;
			if (DdU * T + DdPmV >= 0.0)
			{
				RayParam[NumIntersections++] = T;
			}

			T = (-C1 + Root) * InvC2;
			if (DdU * T + DdPmV >= 0.0)
			{
				RayParam[NumIntersections++] = T;
			}

			if (NumIntersections == 2)
			{
				// The line intersects the positive cone in two distinct
				// points.
				if (RayParam[0] > RayParam[1])
				{
					RealType TmpT = RayParam[0];
					RayParam[0] = RayParam[1];
					RayParam[1] = TmpT;
				}
			}
			else if (NumIntersections == 1)
			{
				// The line intersects the positive cone in a single point and
				// the negative cone in a single point.  We report only the
				// intersection with the positive cone.
				if (DdU > 0.0)
				{
					RayParam[1] = TNumericLimits<RealType>::Max();
				}
				else
				{
					bIntersectsOut = false;
					return;
				}
			}
			else
			{
				bIntersectsOut = false;
				return;
			}
		}
	}
	else
	{
		bIntersectsOut = false;
		return;
	}

	if (!FMath::IsNearlyZero(DdU))
	{
		// Clamp the intersection to the height of the cone.
		RealType InvDdU = (1.0f) / DdU;
		RealType hInterval[2];
		if (DdU > 0.0)
		{
			hInterval[0] = -DdPmV * InvDdU;
			hInterval[1] = (ConeHeight - DdPmV) * InvDdU;
		}
		else // (DdU < 0.0)
		{
			hInterval[0] = (ConeHeight - DdPmV) * InvDdU;
			hInterval[1] = -DdPmV * InvDdU;
		}

		RealType Result0, Result1;
		int NumIntersections;
		GizmoMath::IntervalIntervalIntersection<RealType>(RayParam, hInterval, NumIntersections, Result0, Result1);

		if (NumIntersections > 0)
		{
			RayParam[0] = Result0;
			RayParam[1] = NumIntersections == 2 ? Result1 : Result0;
		}
		else
		{
			bIntersectsOut = false;
			return;
		}
	}
	else if (DdPmV > ConeHeight)
	{		
		bIntersectsOut = false;
		return;
	}

	// Get rid of hits before ray origin
	if (RayParam[0] >= RealType(0))
	{
		bIntersectsOut = true;
		OutRayParam = RayParam[0];
	}
	else if (RayParam[1] >= RealType(0))
	{
		bIntersectsOut = true;
		OutRayParam = RayParam[1];
	}
	else
	{
		bIntersectsOut = false;
	}
	
}

template <typename RealType>
void GizmoMath::IntervalIntervalIntersection(
	const RealType Interval0[2], const RealType Interval1[2], 
	int& OutNumIntersections, RealType& OutResult0, RealType& OutResult1)
{
	// adapted from GeometricTools GTEngine
	// https://www.geometrictools.com/GTE/Mathematics/IntrIntervals.h 
	// (Engine/Plugins/Runtime/GeometryProcessing/Source/GeometryAlgorithms/Private/ThirdParty/GTEngine/Mathematics/GteIntrIntervals.h)
	// Determines the intersection between two floating point intervals in which
	// the input intervals' values are sorted in increasing order.
	// Used by RayConeIntersection.

	if (Interval0[1] < Interval1[0] || Interval0[0] > Interval1[1])
	{
		OutNumIntersections = 0;
		OutResult0 = TNumericLimits<RealType>::Max();
		OutResult1 = -TNumericLimits<RealType>::Max();
	}
	else if (Interval0[1] > Interval1[0])
	{
		if (Interval0[0] < Interval1[1])
		{
			OutNumIntersections = 2;
			OutResult0 =
				(Interval0[0] < Interval1[0] ? Interval1[0] : Interval0[0]);
			OutResult1 =
				(Interval0[1] > Interval1[1] ? Interval1[1] : Interval0[1]);
			if (OutResult0 == OutResult1)
			{
				OutNumIntersections = 1;
			}
		}
		else  // Interval0[0] == Interval1[1]
		{
			OutNumIntersections = 1;
			OutResult0 = Interval0[0];
			OutResult1 = OutResult0;
		}
	}
	else  // Interval0[1] == Interval1[0]
	{
		OutNumIntersections = 1;
		OutResult0 = Interval0[1];
		OutResult1 = OutResult0;
	}
}




void GizmoMath::ClosetPointOnCircle(
	const FVector& QueryPoint,
	const FVector& CircleOrigin, const FVector& CircleNormal, float CircleRadius,
	FVector& ClosestPointOut)
{
	FVector PointDelta = QueryPoint - CircleOrigin;
	FVector DeltaInPlane = PointDelta - FVector::DotProduct(CircleNormal,PointDelta)*CircleNormal;
	double OriginDist = DeltaInPlane.Size();
	if (OriginDist > 0.0f)
	{
		ClosestPointOut =  CircleOrigin + ((double)CircleRadius / OriginDist) * DeltaInPlane;
	}
	else    // all points equidistant, use any one
	{
		FVector PlaneX, PlaneY;
		MakeNormalPlaneBasis(CircleNormal, PlaneX, PlaneY);
		ClosestPointOut = CircleOrigin + (double)CircleRadius * PlaneX;
	}
}



void GizmoMath::MakeNormalPlaneBasis(
	const FVector& PlaneNormal,
	FVector& BasisAxis1Out, FVector& BasisAxis2Out)
{
	// Duff et al method, from https://graphics.pixar.com/library/OrthonormalB/paper.pdf
	if (PlaneNormal.Z < 0)
	{
		double A = 1.0f / (1.0f - PlaneNormal.Z);
		double B = PlaneNormal.X * PlaneNormal.Y * A;
		BasisAxis1Out.X = 1.0f - PlaneNormal.X * PlaneNormal.X * A;
		BasisAxis1Out.Y = -B;
		BasisAxis1Out.Z = PlaneNormal.X;
		BasisAxis2Out.X = B;
		BasisAxis2Out.Y = PlaneNormal.Y * PlaneNormal.Y * A - 1.0f;
		BasisAxis2Out.Z = -PlaneNormal.Y;
	}
	else
	{
		double A = 1.0f / (1.0f + PlaneNormal.Z);
		double B = -PlaneNormal.X * PlaneNormal.Y * A;
		BasisAxis1Out.X = 1.0f - PlaneNormal.X * PlaneNormal.X * A;
		BasisAxis1Out.Y = B;
		BasisAxis1Out.Z = -PlaneNormal.X;
		BasisAxis2Out.X = B;
		BasisAxis2Out.Y = 1.0f - PlaneNormal.Y * PlaneNormal.Y * A;
		BasisAxis2Out.Z = -PlaneNormal.Y;
	}
}



float GizmoMath::ComputeAngleInPlane(
	const FVector& Point,
	const FVector& PlaneOrigin, const FVector& PlaneNormal,
	const FVector& PlaneAxis1, const FVector& PlaneAxis2)
{
	// project point into plane
	FVector LocalPoint = Point - PlaneOrigin;

	double X = FVector::DotProduct(LocalPoint, PlaneAxis1);
	double Y = FVector::DotProduct(LocalPoint, PlaneAxis2);

	float SignedAngle = (float)atan2(Y, X);
	return SignedAngle;
}




FVector2D GizmoMath::ComputeCoordinatesInPlane(
	const FVector& Point,
	const FVector& PlaneOrigin, const FVector& PlaneNormal,
	const FVector& PlaneAxis1, const FVector& PlaneAxis2)
{
	FVector LocalPoint = Point - PlaneOrigin;
	double X = FVector::DotProduct(LocalPoint, PlaneAxis1);
	double Y = FVector::DotProduct(LocalPoint, PlaneAxis2);
	return FVector2D(X, Y);
}


FVector GizmoMath::ProjectPointOntoPlane(
	const FVector& Point,
	const FVector& PlaneOrigin, const FVector& PlaneNormal)
{
	FVector LocalPoint = Point - PlaneOrigin;
	double NormalDot = FVector::DotProduct(LocalPoint, PlaneNormal);
	return Point - NormalDot * PlaneNormal;
}


template <typename RealType>
RealType GizmoMath::SnapToIncrement(RealType Value, RealType Increment)
{
	if (!FMath::IsFinite(Value))
	{
		return 0;
	}
	RealType Sign = FMath::Sign(Value);
	Value = FMath::Abs(Value);
	int IntIncrement = (int)(Value / Increment);
	RealType Remainder = (RealType)fmod(Value, Increment);
	if (Remainder > Increment / 2)
	{
		++IntIncrement;
	}
	return Sign * (RealType)IntIncrement * Increment;
}

// @todo: Remove this and replace calls to it with MakePerpVectors() once
// Engine\Plugins\Experimental\GeometryProcessing\Source\GeometricObjects\Public\VectorUtil.h has been moved out of Plugins dir.
FVector GizmoMath::GetOrthogonalVector(const FVector& V)
{
	FVector AbsVector(FMath::Abs(V.X), FMath::Abs(V.Y), FMath::Abs(V.Z));
	if ((AbsVector.X <= AbsVector.Y) && (AbsVector.X <= AbsVector.Z))
	{
		// X is the smallest component
		return FVector(0, V.Z, -V.Y);
	}
	if ((AbsVector.Z <= AbsVector.X) && (AbsVector.Z <= AbsVector.Y))
	{
		// Z is the smallest component
		return FVector(V.Y, -V.X, 0);
	}
	// Y is the smallest component
	return FVector(-V.Z, 0, V.X);
}

namespace GizmoMath
{
	template
	void INTERACTIVETOOLSFRAMEWORK_API GizmoMath::RayCylinderIntersection<float>(
		const FVector& CylinderCenter, const FVector& CylinderAxis, float CylinderRadius, float CylinderHeight,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, float& OutRayParam);

	template
	void INTERACTIVETOOLSFRAMEWORK_API GizmoMath::RayCylinderIntersection<double>(
		const FVector& CylinderCenter, const FVector& CylinderAxis, double CylinderRadius, double CylinderHeight,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, double& OutRayParam);

	template
	void INTERACTIVETOOLSFRAMEWORK_API GizmoMath::RayConeIntersection<float>(
		const FVector& ConeCenter, const FVector& ConeDirection, float ConeAngle, float ConeHeight,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, float& OutHitDepth);

	template
	void INTERACTIVETOOLSFRAMEWORK_API GizmoMath::RayConeIntersection<double>(
		const FVector& ConeCenter, const FVector& ConeDirection, double ConeAngle, double ConeHeight,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, double& OutHitDepth);
	
	template
	double INTERACTIVETOOLSFRAMEWORK_API GizmoMath::SnapToIncrement(double Value, double Increment);

	template
	float INTERACTIVETOOLSFRAMEWORK_API GizmoMath::SnapToIncrement(float Value, float Increment);

	void GetScreenSpaceLineStartEnd(
		const FSceneView* InSceneView,
		const FVector& InLineOrigin,
		const FVector& InLineDirection,
		const double& InLineLength,
		FVector& OutScreenStartPoint,
		FVector& OutScreenEndPoint,
		double& OutClampedWorldFraction,
		double& OutClampedWorldStartFraction
	)
	{
		// Facilitator method to quickly return the result of InSceneView->ScreenToPixel
		auto ScreenToPixel = [&](const FVector4& InScreenPoint)
		{
			FVector2D OutPoint = FVector2D::ZeroVector;
			InSceneView->ScreenToPixel(InScreenPoint, OutPoint);
			return OutPoint;
		};

		const FIntRect& ViewRect = InSceneView->UnscaledViewRect;

		const FVector& LineStart = InLineOrigin;
		const FVector LineEnd = InLineOrigin + InLineDirection * InLineLength;

		const FVector4 ScreenStartPoint = InSceneView->WorldToScreen(LineStart);
		const FVector4 ScreenEndPoint = InSceneView->WorldToScreen(LineEnd);

		// Pixel coordinates of line Start and End. These could be outside the viewport
		const FVector2D PixelStartPoint = ScreenToPixel(ScreenStartPoint);
		const FVector2D PixelEndPoint = ScreenToPixel(ScreenEndPoint);

		// Checks if the provided point is within the viewport or not
		auto IsOutsideViewRect = [&](const FVector2D& InPoint)
		{
			return InPoint.X < ViewRect.Min.X || InPoint.X > ViewRect.Max.X || InPoint.Y < ViewRect.Min.Y
				|| InPoint.Y > ViewRect.Max.Y;
		};

		const bool bNeedsClamping = IsOutsideViewRect(PixelStartPoint) || IsOutsideViewRect(PixelEndPoint);

		// If clamping is needed, we need to find their position, and re-compute screen and pixel coordinates
		if (bNeedsClamping)
		{
			FVector ClampedLineStart = LineStart;
			FVector ClampedLineEnd = LineEnd;
			// @note: the frustum clipping code below could be cached, with start and end intersection world points
			// which are then used if the corresponding point is out of bounds
			// @see: LevelEditorViewport.cpp TrimLineToFrustum
			FVector IntersectionPoint;
			for (const FPlane& FrustumPlane : InSceneView->ViewFrustum.Planes)
			{
				if (FMath::SegmentPlaneIntersection(LineStart, LineEnd, FrustumPlane, IntersectionPoint))
				{
					if ((FrustumPlane.Dot(IntersectionPoint - LineEnd) > UE_DOUBLE_SMALL_NUMBER))
					{
						ClampedLineStart = IntersectionPoint;
					}
					else
					{
						ClampedLineEnd = IntersectionPoint;
					}
				}
			}

			OutClampedWorldFraction = FVector::Distance(ClampedLineStart, ClampedLineEnd) / InLineLength;
			OutClampedWorldStartFraction = FVector::Distance(LineStart, ClampedLineStart) / InLineLength;

			// We have new Clamped Start and End of line: screen and pixel space points have to be re-computed
			const FVector4 ClampedScreenStartPoint = InSceneView->WorldToScreen(ClampedLineStart);
			const FVector4 ClampedScreenEndPoint = InSceneView->WorldToScreen(ClampedLineEnd);

			const FVector2D ClampedPixelStartPoint = ScreenToPixel(ClampedScreenStartPoint);
			const FVector2D ClampedPixelEndPoint = ScreenToPixel(ClampedScreenEndPoint);

			OutScreenStartPoint = FVector(ClampedPixelStartPoint.X, ClampedPixelStartPoint.Y, ClampedScreenStartPoint.W);
			OutScreenEndPoint = FVector(ClampedPixelEndPoint.X, ClampedPixelEndPoint.Y, ClampedScreenEndPoint.W);
		}
		else
		{
			OutScreenStartPoint = FVector(PixelStartPoint.X, PixelStartPoint.Y, ScreenStartPoint.W);
			OutScreenEndPoint = FVector(PixelEndPoint.X, PixelEndPoint.Y, ScreenEndPoint.W);
			OutClampedWorldStartFraction = 0.0;
		}

		// Clip to near plane if needed
		OutScreenStartPoint.Z = FMath::Max(OutScreenStartPoint.Z, UE_DOUBLE_SMALL_NUMBER);
		OutScreenEndPoint.Z = FMath::Max(OutScreenEndPoint.Z, UE_DOUBLE_SMALL_NUMBER);
	}

	double GetPixelToWorldScaleAtScreenFraction(const double& InStartDepthInView, const double& InEndDepthInView, double InScreenFraction)
	{
		// Clamp screen fraction to normalized range
		InScreenFraction = FMath::Clamp(InScreenFraction, 0.0, 1.0);

		// Fraction from the end depth, scaled to the (start - end) range
		const double ScreenFractionFromEnd = FMath::Max(
			(1.0 - InScreenFraction) * InEndDepthInView + (InScreenFraction * InStartDepthInView), UE_DOUBLE_SMALL_NUMBER
		);

		// As a ratio, for parameterization square the result
		return InEndDepthInView / ScreenFractionFromEnd;
	}

	void BuildDashedLine(FPrimitiveDrawInterface* InPDI, const FDashLineSettings& InDashSettings)
	{
		// Make sure we have at least 3 vertices
		const int32 NumSides = FMath::Max(3, InDashSettings.NumSides);

		// Determine the number of dashes based on world or screen-space metrics
		const float DashAndGapLength = (InDashSettings.DashLength + InDashSettings.DashGapLength);

		if (DashAndGapLength == 0)
		{
			return;
		}

		const int32 NumDashes = FMath::Max(1, FMath::CeilToInt32(InDashSettings.TotalLength / DashAndGapLength));
		const int32 AdjustedNumDashes = FMath::Min(InDashSettings.MaxNumDashes, NumDashes);

		const bool bAddFillerDash = NumDashes - AdjustedNumDashes > 0;

		// Whether to add the filler dash before or after the dashes (used so that the closest geometry has dashes vs. the furthest.
		const bool bAddFillerBeforeDashes = bAddFillerDash
										 && InDashSettings.ScreenStartPoint.Z > InDashSettings.ScreenEndPoint.Z;

		// Num Sides * 2 (top and bottom)
		const int32 NumVerticesPerDash = NumSides * 2;

		// Side Quads NumSides - 1 * 2 triangles per quad, Cap Tris (fan) = NumSides per cap
		const int32 NumTrianglesPerDash = ((NumSides - 1) * 2) + (NumSides * 2);

		const int32 NumCylinders = (AdjustedNumDashes + (bAddFillerDash ? 1 : 0));

		FDynamicMeshBuilder MeshBuilder(InPDI->View->GetFeatureLevel());
		MeshBuilder.ReserveVertices(NumVerticesPerDash * NumCylinders);
		MeshBuilder.ReserveTriangles(NumTrianglesPerDash * NumCylinders);

		TArray<FDynamicMeshVertex> Vertices;
		Vertices.Reserve(NumVerticesPerDash * NumCylinders);

		TArray<uint32> TriangleIndices;
		TriangleIndices.Reserve(NumTrianglesPerDash * NumCylinders);

		auto AppendCylinder = [&](const double& InWorldStart,
								  const double& InWorldEnd,
								  const double& InRadiusScaleStart,
								  const double& InRadiusScaleEnd,
								  TArray<FDynamicMeshVertex>& OutVertices,
								  TArray<uint32>& OutTriangleIndices)
		{
			const float HalfLength = InDashSettings.ScaledLength * 0.5f;

			const float HeightInView = InDashSettings.ScaledLength * InDashSettings.LineFractionInView;

			const double WorldDashMid = 0.5 * (InWorldStart + InWorldEnd);
			const double HalfDashLength = 0.5 * (InWorldEnd - InWorldStart) * HeightInView;

			// WorldVisibleStart "offsets" the base location according to the cut portion of the dash
			// This happens when the line origin sits outside the viewport (e.g. Gizmo Shift + Drag translate which moves the camera during translation)
			const float WorldVisibleStart = InDashSettings.ScaledLength * InDashSettings.WorldStartFraction;

			const FVector Base = (HeightInView * WorldDashMid + WorldVisibleStart - HalfLength) * FVector::UpVector;

			BuildTaperedCylinderVerts(
				Base,
				FVector::XAxisVector,
				FVector::YAxisVector,
				FVector::ZAxisVector,
				InDashSettings.Radius * InRadiusScaleStart,
				InDashSettings.Radius * InRadiusScaleEnd,
				HalfDashLength,
				NumSides,
				OutVertices,
				OutTriangleIndices
			);
		};

		auto AppendDash = [&](const double& InStart, const double& InEnd)
		{
			const double WorldStart = InDashSettings.ParamToWorldFunction(InStart);
			const double WorldEnd = InDashSettings.ParamToWorldFunction(InEnd);

			const double RadiusScaleStart = InDashSettings.ParamToRadiusScaleFunction(InStart);
			const double RadiusScaleEnd = InDashSettings.ParamToRadiusScaleFunction(InEnd);

			AppendCylinder(WorldStart, WorldEnd, RadiusScaleStart, RadiusScaleEnd, Vertices, TriangleIndices);
		};

		const int32 NumDashesForSpaceAllocation = FMath::Max(0, AdjustedNumDashes - 1 + (bAddFillerDash ? 1 : 0));
		const double SpaceForDashes = DashAndGapLength * NumDashesForSpaceAllocation;

		double UsedLength = 0.0;

		// Filler BEFORE dashes
		if (bAddFillerDash && bAddFillerBeforeDashes)
		{
			constexpr double FillerStart = 0.0;
			const double FillerEnd = (InDashSettings.TotalLength - SpaceForDashes) / InDashSettings.TotalLength;

			AppendDash(FillerStart, FillerEnd);

			UsedLength = InDashSettings.TotalLength - SpaceForDashes;
		}

		int32 DashIndex = 0;
		while (UsedLength < InDashSettings.TotalLength - 0.5 && DashIndex < AdjustedNumDashes)
		{
			// Clamp the dash length to avoid overshoot
			const double AdjustedDashLength =
				FMath::Min(InDashSettings.DashLength, InDashSettings.TotalLength - UsedLength);

			const double DashStart = UsedLength / InDashSettings.TotalLength;
			const double DashEnd = (UsedLength + AdjustedDashLength) / InDashSettings.TotalLength;

			AppendDash(DashStart, DashEnd);

			UsedLength += AdjustedDashLength + InDashSettings.DashGapLength;
			++DashIndex;
		}

		// Filler AFTER dashes
		if (bAddFillerDash && !bAddFillerBeforeDashes)
		{
			const double RemainingLength = InDashSettings.TotalLength - UsedLength;
			if (RemainingLength > UE_DOUBLE_SMALL_NUMBER)
			{
				const double FillerStart = UsedLength / InDashSettings.TotalLength;
				constexpr double FillerEnd = 1.0;

				AppendDash(FillerStart, FillerEnd);
			}
		}

		// Colorize
		Algo::ForEach(
			Vertices,
			[Color = InDashSettings.Color](FDynamicMeshVertex& InVertex)
			{
				InVertex.Color = Color;
			}
		);

		MeshBuilder.AddVertices(Vertices);
		MeshBuilder.AddTriangles(TriangleIndices);

		MeshBuilder.Draw(
			InPDI, InDashSettings.RenderLocalToWorldMatrix, InDashSettings.MaterialInterface->GetRenderProxy(), SDPG_Foreground
		);
	}
} // end namespace GizmoMath
