// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoPrivateUtil.h"

#include "BaseGizmos/AxisSources.h"
#include "BaseGizmos/AxisAngleGizmo.h"
#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/FreePositionSubGizmo.h"
#include "BaseGizmos/FreeRotationSubGizmo.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/HitTargets.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/StateTargets.h"
#include "BaseGizmos/TransformSources.h"
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams
#include "ContextObjectStore.h"
#include "GameFramework/Actor.h"
#include "InteractiveGizmoManager.h"
#include "Internationalization/Internationalization.h" // LOCTEXT
#include "UObject/UObjectGlobals.h" // GetTransientPackage

EAxis::Type UE::GizmoUtil::ToAxis(ETransformGizmoSubElements Element)
{
	switch (Element)
	{
	case ETransformGizmoSubElements::TranslateAxisX:
	case ETransformGizmoSubElements::RotateAxisX:
	case ETransformGizmoSubElements::ScaleAxisX:
	case ETransformGizmoSubElements::TranslatePlaneYZ:
	case ETransformGizmoSubElements::ScalePlaneYZ:
		return EAxis::X;
	case ETransformGizmoSubElements::TranslateAxisY:
	case ETransformGizmoSubElements::RotateAxisY:
	case ETransformGizmoSubElements::ScaleAxisY:
	case ETransformGizmoSubElements::TranslatePlaneXZ:
	case ETransformGizmoSubElements::ScalePlaneXZ:
		return EAxis::Y;
	case ETransformGizmoSubElements::TranslateAxisZ:
	case ETransformGizmoSubElements::RotateAxisZ:
	case ETransformGizmoSubElements::ScaleAxisZ:
	case ETransformGizmoSubElements::TranslatePlaneXY:
	case ETransformGizmoSubElements::ScalePlaneXY:
		return EAxis::Z;
	default:
		// We don't ensure here because it is sometimes convenient to write code that ends up
		//  passing in things that we don't end up using the axis for, like a uniform scale element.
		return EAxis::None;
	}
}

UGizmoViewContext* UE::GizmoUtil::GetGizmoViewContext(UInteractiveGizmoManager* GizmoManager)
{
	if (!ensure(GizmoManager))
	{
		return nullptr;
	}

	UContextObjectStore* ContextObjectStore = GizmoManager->GetContextObjectStore();
	if (!ensure(ContextObjectStore))
	{
		return nullptr;
	}

	UGizmoViewContext* GizmoViewContext = ContextObjectStore->FindContext<UGizmoViewContext>();
	ensure(GizmoViewContext);

	return GizmoViewContext;
}

bool UE::GizmoUtil::UpdateCameraAxisSource(UGizmoConstantFrameAxisSource& CameraAxisSource, 
	UInteractiveGizmoManager* GizmoManager, const FVector3d& AxisOrigin)
{
	if (!GizmoManager || !GizmoManager->GetContextQueriesAPI())
	{
		return false;
	}

	FViewCameraState CameraState;
	GizmoManager->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	CameraAxisSource.Origin = AxisOrigin;
	CameraAxisSource.Direction = -CameraState.Forward();
	CameraAxisSource.TangentX = CameraState.Right();
	CameraAxisSource.TangentY = CameraState.Up();
	return true;
}

template <typename SubGizmoType>
bool UE::GizmoUtil::SetCommonSubGizmoProperties(
	SubGizmoType* Gizmo,
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState,
	UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut)
{
	if (!Gizmo
		|| !Params.Component
		|| !Params.TransformProxy)
	{
		return false;
	}

	AActor* ComponentOwnerActor = Params.Component->GetOwner();
	UObject* OuterForSubobjects = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();

	// Set up the axis source if we got an axis
	if (Params.Axis != EAxis::None)
	{
		int AxisIndex = Params.GetClampedAxisIndex();
		if (!Params.bAxisIsBasedOnRootComponent)
		{
			// Axis will be based on the passed-in component transform
			Gizmo->AxisSource = UGizmoComponentAxisSource::Construct(Params.Component, AxisIndex,
				// bUseLocalAxes, not important because we're going to be updating this value if needed
				true,
				OuterForSubobjects);
		}
		else if (SharedState && SharedState->CardinalAxisSources[AxisIndex])
		{
			// Axis is based off root component, and we already have one in our shared state
			Gizmo->AxisSource = SharedState->CardinalAxisSources[AxisIndex];
		}
		else if (ensure(ComponentOwnerActor && ComponentOwnerActor->GetRootComponent()))
		{
			// Axis is based off root component, but we haven't created one yet
			UGizmoComponentAxisSource* CastAxisSource = UGizmoComponentAxisSource::Construct(ComponentOwnerActor->GetRootComponent(), AxisIndex,
				// bUseLocalAxes, not important because we're going to be updating this value if needed
				true,
				OuterForSubobjects);
			Gizmo->AxisSource = CastAxisSource;
			if (SharedState)
			{
				SharedState->CardinalAxisSources[AxisIndex] = CastAxisSource;
			}
		}
	}

	USceneComponent* ComponentToMove = Params.Component;
	if (Params.bManipulatesRootComponent && ensure(ComponentOwnerActor && ComponentOwnerActor->GetRootComponent()))
	{
		ComponentToMove = ComponentOwnerActor->GetRootComponent();
	}

	// The transform source is also transform destination. It forwards the resulting transform to our proxy and
	// to the component.
	if (SharedState && SharedState->TransformSource && Params.bManipulatesRootComponent)
	{
		// We already have one in our shared state and we can use it (because the sub gizmo doesn't just move
		//  its own component).
		TransformSourceOut = SharedState->TransformSource;
	}
	else
	{
		// Need to create a new transform source
		TransformSourceOut = UGizmoScaledAndUnscaledTransformSources::Construct(
			UGizmoTransformProxyTransformSource::Construct(Params.TransformProxy, OuterForSubobjects),
			ComponentToMove, OuterForSubobjects);
		if (SharedState && Params.bManipulatesRootComponent)
		{
			SharedState->TransformSource = TransformSourceOut;
		}
	}

	// Hit target is how we detect whether the gizmo has been hit, and what we update hover on. We don't
	//  use shared state here because this is always unique to each sub gizmo.
	UGizmoComponentHitTarget* CastHitTarget = UGizmoComponentHitTarget::Construct(Params.Component, OuterForSubobjects);
	Gizmo->HitTarget = CastHitTarget;
	// The default hover/interaction behavior is to forward that information to the component for rendering.
	CastHitTarget->UpdateHoverFunction = 
		// TODO: See comment in FTransformSubGizmoCommonParams- seems not worth giving the user the ability to
		//  customize this function.
		//Params.CustomUpdateHoverFunction ? Params.CustomUpdateHoverFunction : 
		[CastHitTarget](bool bHovering)
	{
		if (!ensure(IsValid(CastHitTarget))) return;
		if (IGizmoBaseComponentInterface* Hoverable = Cast<IGizmoBaseComponentInterface>(CastHitTarget->Component))
 		{
			Hoverable->UpdateHoverState(bHovering);
		}
	};
	CastHitTarget->UpdateInteractingFunction = [CastHitTarget](bool bInteracting)
	{
		if (!ensure(IsValid(CastHitTarget))) return;
		if (IGizmoBaseComponentInterface* Hoverable = Cast<IGizmoBaseComponentInterface>(CastHitTarget->Component))
		{
			Hoverable->UpdateInteractingState(bInteracting);
		}
	};

	// Set up shared state target, which handles undo/redo of the component and proxy transform. As
	//  with transform source, if the sub gizmo moves its own component (rather than the whole gizmo),
	//  we need our own target, otherwise we'll try to use shared state.
	if (SharedState && SharedState->StateTarget && Params.bManipulatesRootComponent)
	{
		Gizmo->StateTarget = SharedState->StateTarget;
	}
	else
	{
		const FText DefaultTransactionName(NSLOCTEXT("UCombinedTransformGizmo", "UCombinedTransformGizmoTransaction", "Transform"));

		FText TransactionName = Params.TransactionName.IsSet() ? Params.TransactionName.GetValue()
			: DefaultTransactionName;

		IToolContextTransactionProvider* TransactionProvider = Params.TransactionProvider ?
			Params.TransactionProvider : Gizmo->GetGizmoManager();

		UGizmoTransformChangeStateTarget* CastStateTarget = UGizmoTransformChangeStateTarget::Construct(ComponentToMove,
			TransactionName, TransactionProvider, OuterForSubobjects);
		CastStateTarget->DependentChangeSources.Add(MakeUnique<FTransformProxyChangeSource>(Params.TransformProxy));
		Gizmo->StateTarget = CastStateTarget;

		if (SharedState && Params.bManipulatesRootComponent)
		{
			SharedState->StateTarget = CastStateTarget;
		}
	}

	return true;
}

double UE::GizmoUtil::ClampRayOrigin(const UGizmoViewContext* InViewContext, FVector& InOutRayOrigin, const FVector& InRayDirection, const double InMaxDepth)
{
	// due to numerical imprecision, the ray origin needs to be clamped in ortho views
	// (cf. UEditorInteractiveToolsContext::GetRayFromMousePos)
	
	if (!InViewContext->IsPerspectiveProjection())
	{
		const double dot = FVector::DotProduct(InOutRayOrigin, InRayDirection);
		if (FMath::Abs(dot) > InMaxDepth)
		{
			InOutRayOrigin += -dot * InRayDirection;
			InOutRayOrigin -= InMaxDepth * InRayDirection;
			
			const double DepthBias = (-dot) - InMaxDepth;
			return DepthBias;
		}
	}
	
	return 0.0;
}


// Explicit instantiations to avoid having the definition in the header
template bool UE::GizmoUtil::SetCommonSubGizmoProperties<UAxisPositionGizmo>(
	UAxisPositionGizmo* Gizmo,
	const FTransformSubGizmoCommonParams& Params,
	FTransformSubGizmoSharedState* SharedState,
	UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut);
template bool UE::GizmoUtil::SetCommonSubGizmoProperties<UAxisAngleGizmo>(
	UAxisAngleGizmo* Gizmo,
	const FTransformSubGizmoCommonParams& Params,
	FTransformSubGizmoSharedState* SharedState,
	UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut);
template bool UE::GizmoUtil::SetCommonSubGizmoProperties<UPlanePositionGizmo>(
	UPlanePositionGizmo* Gizmo,
	const FTransformSubGizmoCommonParams& Params,
	FTransformSubGizmoSharedState* SharedState,
	UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut);
template bool UE::GizmoUtil::SetCommonSubGizmoProperties<UFreePositionSubGizmo>(
	UFreePositionSubGizmo* Gizmo,
	const FTransformSubGizmoCommonParams& Params,
	FTransformSubGizmoSharedState* SharedState,
	UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut);
template bool UE::GizmoUtil::SetCommonSubGizmoProperties<UFreeRotationSubGizmo>(
	UFreeRotationSubGizmo* Gizmo,
	const FTransformSubGizmoCommonParams& Params,
	FTransformSubGizmoSharedState* SharedState,
	UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut);