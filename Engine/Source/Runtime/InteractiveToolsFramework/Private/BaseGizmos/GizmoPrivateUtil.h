// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveGizmo.h" // ETransformGizmoSubElements
#include "Math/Axis.h"
#include "VectorTypes.h"

class UGizmoConstantFrameAxisSource;
class UGizmoScaledAndUnscaledTransformSources;
class UGizmoViewContext;
class UInteractiveGizmoManager;
namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}

/**
 * This file holds implementation helpers that don't necessarily need exposing. If eventually needed,
 *  we can move some of these into GizmoUtil or TransformSubGizmoUtil
 */

namespace UE::GizmoUtil 
{
	/**
	 * Given a single element, gives the axis that defines that element (e.g., x for TranslateAxisX or TranslatePlaneYZ, etc).
	 *  Gives EAxis::None if the element is not a single element.
	 */
	EAxis::Type ToAxis(ETransformGizmoSubElements Element);

	/**
	 * Simple helper that gets the gizmo view context out of the context object store associated with a gizmo manager.
	 * Fires ensures if it does not find the expected objects along the way.
	 */
	UGizmoViewContext* GetGizmoViewContext(UInteractiveGizmoManager* GizmoManager);

	/**
	 * Helper to update an axis source that is based on the camera location, called from tick.
	 */
	bool UpdateCameraAxisSource(UGizmoConstantFrameAxisSource& CameraAxisSource,
		UInteractiveGizmoManager* GizmoManager, const FVector3d& AxisOrigin);

	/**
	 * Helper that sets some common properties on sub gizmos, namely AxisSource, HitTarget, 
	 *  and StateTarget. Also returns a TransformSource, which typically gets wrapped up in
	 *  a parameter source.
	 * 
	 * Template because the properties it manipulates aren't part of a base class. Perhaps they
	 *  should be, but we have not needed that yet, other than this.
	 */
	template <typename SubGizmoType>
	bool SetCommonSubGizmoProperties(
		SubGizmoType* Gizmo,
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState,
		UGizmoScaledAndUnscaledTransformSources*& TransformSourceOut);

	/**
	 * Helper to ensure that the ray origin is not too far in orthographic views to avoid numerical imprecision later
	 * @return the depth bias between the current ray origin and the updated one if the ray origin is greater than InMaxDepth
	 */
	double ClampRayOrigin(const UGizmoViewContext* InViewContext, FVector& InOutRayOrigin, const FVector& InRayDirection, const double InMaxDepth = UE_LARGE_HALF_WORLD_MAX1);
	
}// end UE::GizmoUtil