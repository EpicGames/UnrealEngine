// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoRectangleComponent.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoRectangleComponent)

namespace GizmoRectangleComponentLocals
{
	const float RECTANGLE_RENDERVISIBILITY_DOT_THRESHOLD = 0.25;

	template <typename SceneViewOrGizmoViewContext>
	bool GetWorldCorners(bool bIsViewDependent, const SceneViewOrGizmoViewContext* View, 
		const FVector& WorldOrigin, const FVector& DirectionX, const FVector& DirectionY, 
		float OffsetX, float OffsetY, float LengthX, float LengthY,
		bool bUseWorldAxes, bool bOrientYAccordingToCamera, 
		TFunctionRef<FVector(const FVector&)> VectorTransform,
		TArray<FVector>& CornersOut,
		float& PixelToWorldScaleOut)
	{
		FVector UseDirectionX = (bUseWorldAxes) ? DirectionX : VectorTransform(DirectionX);
		FVector UseDirectionY = (bUseWorldAxes) ? DirectionY : VectorTransform(DirectionY);
		float LengthScale = 1;

		if (bIsViewDependent)
		{
			bool bIsOrtho = !View->IsPerspectiveProjection();

			// direction to origin of gizmo
			FVector ViewDirection =
				View->IsPerspectiveProjection() ? WorldOrigin - View->ViewLocation : View->GetViewDirection();
			ViewDirection.Normalize();

			bool bFlippedX = (FVector::DotProduct(ViewDirection, UseDirectionX) > 0);
			UseDirectionX = (bFlippedX) ? -UseDirectionX : UseDirectionX;

			if (bOrientYAccordingToCamera)
			{
				// See if by rotating the y axis around the x axis 90 degrees, we end up with y that is less
				// colinear to our view ray.
				FVector RotatedY = UseDirectionY.RotateAngleAxis(90, UseDirectionX);
				if (FMath::Abs(RotatedY.Dot(ViewDirection)) < FMath::Abs(UseDirectionY.Dot(ViewDirection)))
				{
					UseDirectionY = RotatedY;
				}
			}
			bool bFlippedY = (FVector::DotProduct(ViewDirection, UseDirectionY) > 0);
			UseDirectionY = (bFlippedY) ? -UseDirectionY : UseDirectionY;

			FVector PlaneNormal = FVector::CrossProduct(UseDirectionX, UseDirectionY);
			bool bRenderVisibility =
				FMath::Abs(FVector::DotProduct(PlaneNormal, ViewDirection)) > RECTANGLE_RENDERVISIBILITY_DOT_THRESHOLD;

			if (!bRenderVisibility)
			{
				return false;
			}

			LengthScale = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, WorldOrigin);
		}

		double UseOffsetX = LengthScale * OffsetX;
		double UseOffsetLengthX = LengthScale * (OffsetX + LengthX);
		double UseOffsetY = LengthScale * OffsetY;
		double UseOffsetLengthY = LengthScale * (OffsetY + LengthY);

		CornersOut.SetNum(4);
		CornersOut[0] = WorldOrigin + UseOffsetX * UseDirectionX + UseOffsetY * UseDirectionY;
		CornersOut[1] = WorldOrigin + UseOffsetLengthX * UseDirectionX + UseOffsetY * UseDirectionY;
		CornersOut[2] = WorldOrigin + UseOffsetLengthX * UseDirectionX + UseOffsetLengthY * UseDirectionY;
		CornersOut[3] = WorldOrigin + UseOffsetX * UseDirectionX + UseOffsetLengthY * UseDirectionY;

		PixelToWorldScaleOut = LengthScale;

		return true;
	}
}

class FGizmoRectangleComponentSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FGizmoRectangleComponentSceneProxy(const UGizmoRectangleComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent),
		Color(InComponent->Color),
		DirectionX(InComponent->DirectionX),
		DirectionY(InComponent->DirectionY),
		bOrientYAccordingToCamera(InComponent->bOrientYAccordingToCamera),
		OffsetX(InComponent->OffsetX),
		OffsetY(InComponent->OffsetY),
		LengthX(InComponent->LengthX),
		LengthY(InComponent->LengthY),
		Thickness(InComponent->Thickness),
		HoverThicknessMultiplier(InComponent->HoverSizeMultiplier),
		SegmentFlags(InComponent->SegmentFlags)
	{
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		using namespace GizmoRectangleComponentLocals;

		const FMatrix& LocalToWorldMatrix = GetLocalToWorld();
		FVector Origin = LocalToWorldMatrix.TransformPosition(FVector::ZeroVector);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

				bool bWorldAxis = (bExternalWorldLocalState) ? (*bExternalWorldLocalState) : false;
				
				TArray<FVector> Corners;
				bool bIsViewDependent = (bExternalIsViewDependent) ? (*bExternalIsViewDependent) : false;
				float PixelToWorldScale = 1.0;
				bool bRenderVisibility = GetWorldCorners(bIsViewDependent, View, Origin, DirectionX, DirectionY,
					OffsetX, OffsetY, LengthX, LengthY, bWorldAxis, bOrientYAccordingToCamera,
					[&LocalToWorldMatrix](const FVector& VectorIn) { return FVector{ LocalToWorldMatrix.TransformVector(VectorIn) }; },
					Corners, PixelToWorldScale);

				if (!bRenderVisibility)
				{
					continue;
				}
				
				float UseThickness = (bExternalHoverState != nullptr && *bExternalHoverState == true) ?
					(HoverThicknessMultiplier * Thickness) : (Thickness);
				if (View->IsPerspectiveProjection())
				{
					UseThickness *= (View->FOV / 90.0f);		// compensate for FOV scaling in Gizmos...
				}

				if (SegmentFlags & 0x1)
				{
					PDI->DrawLine(Corners[0], Corners[1], Color, SDPG_Foreground, UseThickness, 0.0f, true);
				}
				if (SegmentFlags & 0x2)
				{
					PDI->DrawLine(Corners[1], Corners[2], Color, SDPG_Foreground, UseThickness, 0.0f, true);
				}
				if (SegmentFlags & 0x4)
				{
					PDI->DrawLine(Corners[2], Corners[3], Color, SDPG_Foreground, UseThickness, 0.0f, true);
				}
				if (SegmentFlags & 0x8)
				{
					PDI->DrawLine(Corners[3], Corners[0], Color, SDPG_Foreground, UseThickness, 0.0f, true);
				}

			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();

		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return false;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return IntCastChecked<uint32>( sizeof *this + GetAllocatedSize() ); }
	SIZE_T GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	void SetExternalHoverState(bool* HoverState)
	{
		bExternalHoverState = HoverState;
	}

	void SetExternalWorldLocalState(bool* bWorldLocalState)
	{
		bExternalWorldLocalState = bWorldLocalState;
	}

	void SetExternalIsViewDependent(bool* bExternalIsViewDependentIn)
	{
		bExternalIsViewDependent = bExternalIsViewDependentIn;
	}

private:
	FLinearColor Color;
	FVector DirectionX, DirectionY;
	bool bOrientYAccordingToCamera;
	float OffsetX, OffsetY;
	float LengthX, LengthY;
	float Thickness;
	float HoverThicknessMultiplier;
	uint8 SegmentFlags;

	// set on Component for use in ::GetDynamicMeshElements()
	bool* bExternalHoverState = nullptr;
	bool* bExternalWorldLocalState = nullptr;
	bool* bExternalIsViewDependent = nullptr;
};



FPrimitiveSceneProxy* UGizmoRectangleComponent::CreateSceneProxy()
{
	FGizmoRectangleComponentSceneProxy* NewProxy = new FGizmoRectangleComponentSceneProxy(this);
	NewProxy->SetExternalHoverState(&bHovering);
	NewProxy->SetExternalWorldLocalState(&bWorld);
	NewProxy->SetExternalIsViewDependent(&bIsViewDependent);
	return NewProxy;
}

FBoxSphereBounds UGizmoRectangleComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	float MaxOffset = FMath::Max(OffsetX, OffsetY);
	float MaxLength = FMath::Max(LengthX, LengthY);
	return FBoxSphereBounds(FSphere(FVector::ZeroVector, 100*MaxOffset+MaxLength).TransformBy(LocalToWorld));
}

bool UGizmoRectangleComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	using namespace GizmoRectangleComponentLocals;

	const FTransform& Transform = GetComponentToWorld();

	FVector UseOrigin = Transform.TransformPosition(FVector::ZeroVector);

	TArray<FVector> Corners;
	float PixelToWorldScale = 1.0;
	bool bRenderVisibility = GetWorldCorners(bIsViewDependent, ToRawPtr(GizmoViewContext), UseOrigin, DirectionX, DirectionY, 
		OffsetX, OffsetY, LengthX, LengthY,	bWorld, bOrientYAccordingToCamera,
		[&Transform](const FVector& VectorIn) { return Transform.TransformVector(VectorIn); },
		Corners, PixelToWorldScale);

	if (bRenderVisibility == false)
	{
		return false;
	}

	static const int Triangles[2][3] = { {0,1,2}, {0,2,3} };

	// If ray is within pixel distance tolerance of boundary edges of rectangle,
	// consider this a hit. Currently rectangle is rendered as a thick outline, 
	// so if this is not done, and only the ray/try tests below are done, then
	// the cursor can be directly over the line without a hit being registered.
	double MinHitDistance = PixelHitDistanceThreshold * PixelToWorldScale;
	bool bFoundLineHit = false;
	for (int32 k = 0; k < 4; ++k)
	{
		FVector NearestArrow, NearestLine;
		FMath::SegmentDistToSegmentSafe(Corners[k], Corners[(k+1)%4], Start, End, NearestArrow, NearestLine);
		double Distance = FVector::Distance(NearestArrow, NearestLine);
		if (Distance < MinHitDistance)
		{
			bFoundLineHit = true;
			OutHit.Component = this;
			OutHit.Distance = static_cast<float>(FVector::Distance(Start, NearestLine));
			OutHit.ImpactPoint = NearestLine;
			MinHitDistance = Distance;
		}
	}
	if (bFoundLineHit)
	{
		FVector Edge1(Corners[Triangles[0][1]] - Corners[Triangles[0][0]]);
		Edge1.Normalize();
		FVector Edge2(Corners[Triangles[0][2]] - Corners[Triangles[0][0]]);
		Edge2.Normalize();
		OutHit.ImpactNormal = Edge2.Cross(Edge1);	// same code here as SegmentTriangleIntersection()
		OutHit.ImpactNormal.Normalize();
		return true;
	}

	// if line was not hit, try hitting rectangle triangles
	for (int j = 0; j < 2; ++j)
	{
		const int* Triangle = Triangles[j];
		FVector HitPoint, HitNormal;
		if (FMath::SegmentTriangleIntersection(Start, End,
			Corners[Triangle[0]], Corners[Triangle[1]], Corners[Triangle[2]],
			HitPoint, HitNormal) )
		{
			OutHit.Component = this;
			OutHit.Distance = static_cast<float>( FVector::Distance(Start, HitPoint) );
			OutHit.ImpactPoint = HitPoint;
			OutHit.ImpactNormal = HitNormal;
			return true;
		}
	}

	return false;
}

void UGizmoRectangleComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	//OutMaterials.Add(GEngine->VertexColorMaterial);
}
