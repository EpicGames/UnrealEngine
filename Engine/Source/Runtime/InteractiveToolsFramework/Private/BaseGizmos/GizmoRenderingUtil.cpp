// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoRenderingUtil.h"

#include "BaseGizmos/GizmoPrivateUtil.h" // GetGizmoViewContext
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/ViewAdjustedStaticMeshGizmoComponent.h"
#include "RHI.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Math/UnrealMathUtility.h"
#include "Math/NumericLimits.h"
#include "Misc/AxisDisplayInfo.h"

// yuck global value set by Editor
static const FSceneView* GlobalCurrentSceneView = nullptr;

static FCriticalSection GlobalCurrentSceneViewLock;

#if WITH_EDITOR
static bool bGlobalUseCurrentSceneViewTracking = true;
#else
static bool bGlobalUseCurrentSceneViewTracking = false;
#endif

namespace GizmoRenderingUtilLocals
{
	static double VectorDifferenceSqr(const FVector2D& A, const FVector2D& B)
	{
		double ax = A.X, ay = A.Y;
		double bx = B.X, by = B.Y;
		ax -= bx;
		ay -= by;
		return ax * ax + ay * ay;
	}

	static double VectorDifferenceSqr(const FVector& A, const FVector& B)
	{
		double ax = A.X, ay = A.Y, az = A.Z;
		double bx = B.X, by = B.Y, bz = B.Z;
		ax -= bx;
		ay -= by;
		az -= bz;
		return ax * ax + ay * ay + az * az;
	}

	// duplicates FSceneView::WorldToPixel but in double where possible (unfortunately WorldToScreen still in float)
	static FVector2D WorldToPixelDouble(const UE::GizmoRenderingUtil::ISceneViewInterface* View, const FVector& Location)
	{
		FVector4 ScreenPoint = View->WorldToScreen(Location);

		double InvW = (ScreenPoint.W > 0.0 ? 1.0 : -1.0) / (double)ScreenPoint.W;
		double Y = (GProjectionSignY > 0.0) ? (double)ScreenPoint.Y : 1.0 - (double)ScreenPoint.Y;

		const FIntRect& UnscaledViewRect = View->GetUnscaledViewRect();
		double PosX = (double)UnscaledViewRect.Min.X + (0.5 + (double)ScreenPoint.X * 0.5 * InvW) * (double)UnscaledViewRect.Width();
		double PosY = (double)UnscaledViewRect.Min.Y + (0.5 - Y * 0.5 * InvW) * (double)UnscaledViewRect.Height();

		return FVector2D(PosX, PosY);
	}

	// This matches "CurrentColor" in FWidget::Widget()
	FColor HoverColor = FColor::Yellow;
}

UViewAdjustedStaticMeshGizmoComponent* UE::GizmoRenderingUtil::CreateDefaultMaterialGizmoMeshComponent(
	UStaticMesh* Mesh, UGizmoViewContext* GizmoViewContext, UObject* OwnerComponentOrActor, 
	const FLinearColor& Color, bool bAddHoverMaterial)
{
	using namespace GizmoRenderingUtilLocals;

	if (!ensureMsgf(OwnerComponentOrActor, TEXT("CreateDefaultMaterialGizmoMeshComponent: Need owner component or actor to create component.")))
	{
		return nullptr;
	}
	UViewAdjustedStaticMeshGizmoComponent* Component = NewObject<UViewAdjustedStaticMeshGizmoComponent>(OwnerComponentOrActor);
	Component->SetStaticMesh(Mesh);

	Component->SetGizmoViewContext(GizmoViewContext);
	Component->TranslucencySortPriority = UE::GizmoRenderingUtil::GIZMO_TRANSLUCENCY_SORT_PRIORITY;
	// Used by the default material to be able to be occluded by other gizmo elements
	Component->bRenderCustomDepth = true;
	// Not sure that this actually gets respected in any way for non-PDI calls, but just in case
	Component->DepthPriorityGroup = SDPG_Foreground;

	Component->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
	// Important: make it so that typical world line traces don't hit this gizmo component. 
	Component->SetCollisionResponseToAllChannels(ECR_Ignore);
	// The LineTraceComponent calls that UGizmoComponentHitTarget uses work even without us being bound
	//  to a particular collision channel (so the following two lines are not strictly speaking necessary),
	//  but let's go ahead and associate ourselves with the COLLISION_GIZMO channel (modeled on UGizmoHandleGroup::CreateMeshHandle).
	Component->SetCollisionResponseToChannel(COLLISION_GIZMO, ECollisionResponse::ECR_Block);
	Component->SetCollisionObjectType(COLLISION_GIZMO);

	Component->SetAllMaterials(UE::GizmoRenderingUtil::GetDefaultGizmoComponentMaterial(Color, Component));
	if (bAddHoverMaterial)
	{
		Component->SetHoverOverrideMaterial(UE::GizmoRenderingUtil::GetDefaultGizmoComponentMaterial(HoverColor, Component));
	}

	return Component;
}

UViewAdjustedStaticMeshGizmoComponent* UE::GizmoRenderingUtil::CreateDefaultMaterialGizmoMeshComponent(
	UStaticMesh* Mesh, UInteractiveGizmoManager* GizmoManager, UObject* OwnerComponentOrActor, 
	const FLinearColor& Color, bool bAddHoverMaterial)
{
	return CreateDefaultMaterialGizmoMeshComponent(Mesh, UE::GizmoUtil::GetGizmoViewContext(GizmoManager), 
		OwnerComponentOrActor, Color, bAddHoverMaterial);
}


float UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(
	const FSceneView* View,
	const FVector& Location,
	const double InDPIScale)
{
	using namespace GizmoRenderingUtilLocals;

	if (!ensure(View))
	{
		return 1.0f;
	}

	UE::GizmoRenderingUtil::FSceneViewWrapper Wrapper(*View);
	return CalculateLocalPixelToWorldScale(&Wrapper, Location, true, InDPIScale);
}

template <typename RealType>
RealType UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(
	const UE::GizmoRenderingUtil::ISceneViewInterface* View, 
	const FVector& Location,
	const bool bInWithDPIScale,
	const TOptional<double>& InDPIScale)
{
	using namespace GizmoRenderingUtilLocals;

	// To calculate this scale at Location, we project Location to screen and also project a second
	// point at a small distance from Location in a camera-perpendicular plane, then measure 2D/3D distance ratio. 
	// However, because some of the computations are done in float, there will be enormous numerical error 
	// when the camera is very far from the location if the distance is relatively small.
	//
	// For perspective projection, the "W" value from WorldToScreen gives us a sense of distance, so we 
	// make the offset relative to that.
	//
	// For orthographic projection, W is constant (~1.0) regardless of zoom level, so instead we derive
	// the offset from the inverse of the projection matrix scale (M[0][0]), which encodes the ortho width.
	// This provides a zoom-proportional offset that maintains numerical stability at extreme zoom levels.
	
	constexpr double Offset = 0.01;
	
	double OffsetDelta;
	if (View->IsPerspectiveProjection())
	{
		// Perspective: offset proportional to distance (W component)
		FVector4 LocationScreenPoint = View->WorldToScreen(Location);
		OffsetDelta = LocationScreenPoint.W * Offset;
	}
	else
	{
		// Orthographic: offset proportional to ortho scale (inverse of M[0][0])
		// M[0][0] = 2 / OrthoWidth, so 1/M[0][0] = OrthoWidth/2 gives us a world-space measure
		const FMatrix& ProjMatrix = View->GetProjectionMatrix();
		OffsetDelta = FMath::Abs(1.0 / ProjMatrix.M[0][0]) * Offset;
	}


	FVector2D PixelA = WorldToPixelDouble(View, Location);
	FVector OffsetPointWorld = Location + OffsetDelta * View->GetViewRight() + OffsetDelta * View->GetViewUp();
	FVector2D PixelB = WorldToPixelDouble(View, OffsetPointWorld);

	constexpr double MinPixelToWorldScale = (1.e-32); // Tiny, but non-zero. Built-in "SMALL_NUMBER" variants aren't small enough for micro-zoom levels.
	constexpr double MaxPixelToWorldScale = UE_DOUBLE_BIG_NUMBER; // Arbitrary large value to prevent division by zero or very small scale

	double PixelDeltaSqr = FMath::Max(MinPixelToWorldScale, VectorDifferenceSqr(PixelA, PixelB));
	double WorldDeltaSqr = VectorDifferenceSqr(Location, OffsetPointWorld);

	double Result = FMath::Sqrt(WorldDeltaSqr / PixelDeltaSqr);
	Result = FMath::Clamp(Result, MinPixelToWorldScale, MaxPixelToWorldScale);
	if (bInWithDPIScale)
	{
		Result *= InDPIScale.Get(View->GetDPIScale());
	}

	return static_cast<RealType>(Result);
}

template INTERACTIVETOOLSFRAMEWORK_API float UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale<float>(
	const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, bool, const TOptional<double>&);

template INTERACTIVETOOLSFRAMEWORK_API double UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale<double>(
	const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, bool, const TOptional<double>&);

float UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten(
	const FSceneView* View,
	const FVector& Location,
	const float InScale,
	FVector& OutFlattenScale)
{
	if (!ensure(View))
	{
		return 1.0f;
	}
	UE::GizmoRenderingUtil::FSceneViewWrapper Wrapper(*View);
	return CalculateViewDependentScaleAndFlatten(&Wrapper, Location, InScale, OutFlattenScale);
}

template <typename RealType>
RealType UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten(
	const UE::GizmoRenderingUtil::ISceneViewInterface* View, 
	const FVector& Location, 
	const RealType InScale, 
	FVector& OutFlattenScale)
{
	const FMatrix& ViewMatrix = View->GetViewMatrix();
	bool bIsPerspective = View->GetProjectionMatrix().M[3][3] < 1.0;
	bool bIsOrthoXY = !bIsPerspective && FMath::Abs(ViewMatrix.M[2][2]) > 0.0;
	bool bIsOrthoXZ = !bIsPerspective && FMath::Abs(ViewMatrix.M[1][2]) > 0.0;
	bool bIsOrthoYZ = !bIsPerspective && FMath::Abs(ViewMatrix.M[0][2]) > 0.0;
	RealType UniformScale = static_cast<RealType> (InScale * View->WorldToScreen(Location).W * (4.0 / View->GetUnscaledViewRect().Width() / View->GetProjectionMatrix().M[0][0]));

	// Clamp to tolerance to prevent division by zero.
	// @todo change to use MathUtil<RealType>::ZeroTolerance and TMathUtil<RealType>::SignNonZero(Value)
	RealType MinimumScale = static_cast<RealType>(UE_DOUBLE_SMALL_NUMBER);
	if (FMath::Abs(UniformScale) < MinimumScale)
	{
		UniformScale = MinimumScale * (UniformScale < 0.0 ? -1.0 : 1.0);
	}

	if (bIsPerspective)
	{
		OutFlattenScale = FVector::OneVector;
	}
	else
	{
		// Flatten scale prevents scaling in the direction of the camera and thus intersecting the near plane.
		// Based on legacy FWidget render code but is flatten actually necessary?? that axis wasn't scaled anyways!
		if (bIsOrthoXY)
		{
			OutFlattenScale = FVector(1.0, 1.0, 1.0 / UniformScale);
		}
		else if (bIsOrthoXZ)
		{
			OutFlattenScale = FVector(1.0, 1.0 / UniformScale, 1.0);
		}
		else if (bIsOrthoYZ)
		{
			OutFlattenScale = FVector(1.0 / UniformScale, 1.0, 1.0);
		}
	}
	return UniformScale;
}

template INTERACTIVETOOLSFRAMEWORK_API float UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten<float>(
	const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, float, FVector&);

template INTERACTIVETOOLSFRAMEWORK_API double UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten<double>(
	const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, double, FVector&);

UMaterialInterface* UE::GizmoRenderingUtil::GetDefaultGizmoComponentMaterial(const FLinearColor& Color, UObject* Outer)
{
	return GetDefaultGizmoComponentMaterial(Color, FDefaultGizmoMaterialExtraParams(), Outer);
}

UMaterialInterface* UE::GizmoRenderingUtil::GetDefaultGizmoComponentMaterial(const FLinearColor & Color, const FDefaultGizmoMaterialExtraParams & Params, UObject * Outer)
{
	UMaterialInterface* Material = Params.bDimOccluded ? LoadObject<UMaterial>(nullptr, TEXT("/Engine/InteractiveToolsFramework/Materials/GizmoComponentMaterial"))
		: LoadObject<UMaterial>(nullptr, TEXT("/Engine/InteractiveToolsFramework/Materials/GizmoComponentMaterial_NotDimmed"));
	if (Material == nullptr)
	{
		return nullptr;
	}

	UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, Outer);

	MatInstance->SetVectorParameterValue(TEXT("GizmoColor"), Color);
	return MatInstance;
}

FLinearColor UE::GizmoRenderingUtil::GetDefaultAxisColor(EAxis::Type Axis)
{
	// The below colors come from FWidget::FWidget() and elsewhere
	switch (Axis)
	{
	case EAxis::X:
		return AxisDisplayInfo::GetAxisColor(EAxisList::X);
	case EAxis::Y:
		return AxisDisplayInfo::GetAxisColor(EAxisList::Y);
	case EAxis::Z:
		return AxisDisplayInfo::GetAxisColor(EAxisList::Z);
	default:
		return FLinearColor::White;
	}
}

// Forward the deprecated methods (that are in the wrong namespace) to the proper namespace methods
float GizmoRenderingUtil::CalculateLocalPixelToWorldScale(
	const FSceneView* View,
	const FVector& Location)
{
	return UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, Location);
}
float GizmoRenderingUtil::CalculateLocalPixelToWorldScale(
	const UGizmoViewContext* ViewContext,
	const FVector& Location)
{
	return UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(ViewContext, Location);
}
float GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten(
	const FSceneView* View,
	const FVector& Location,
	const float InScale,
	FVector& OutFlattenScale)
{
	return UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten(View, Location, InScale, OutFlattenScale);
}