// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/GizmoUtil.h"

#include "BaseGizmos/GizmoElementCircleBase.h"
#include "BaseGizmos/GizmoElementGroup.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveGizmo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GizmoUtil)

UInteractiveGizmo* UE::GizmoUtil::CreateGizmoViaSimpleBuilder(UInteractiveGizmoManager* GizmoManager, 
	TSubclassOf<UInteractiveGizmo> GizmoClass, const FString& InstanceIdentifier, void* Owner)
{
	if (!ensure(GizmoManager))
	{
		return nullptr;
	}

	const FString BuilderIdentifier = "__CreateGizmoViaSimpleBuilder_TemporaryBuilder";

	USimpleLambdaInteractiveGizmoBuilder* Builder = NewObject<USimpleLambdaInteractiveGizmoBuilder>();
	Builder->BuilderFunc = [GizmoManager, GizmoClass](const FToolBuilderState& SceneState) { return NewObject<UInteractiveGizmo>(GizmoManager, GizmoClass.Get()); };

	GizmoManager->RegisterGizmoType(BuilderIdentifier, Builder);
	UInteractiveGizmo* Gizmo = GizmoManager->CreateGizmo(BuilderIdentifier, InstanceIdentifier, Owner);
	GizmoManager->DeregisterGizmoType(BuilderIdentifier);

	return Gizmo;
}

TConstArrayView<TObjectPtr<UGizmoElementBase>> FGizmoElementAccessor::GetSubElements(const UGizmoElementBase& InGizmoElement)
{
	return InGizmoElement.GetSubElements();
}

void FGizmoElementAccessor::GetSubElementsRecursive(const UGizmoElementBase& InGizmoElement, TArray<TObjectPtr<UGizmoElementBase>>& OutElements)
{
	InGizmoElement.GetSubElementsRecursive(OutElements);
}

bool FGizmoElementAccessor::GetSubElementsRecursive(const UGizmoElementBase& InGizmoElement, TArray<TObjectPtr<UGizmoElementBase>>& OutElements, const uint32 InPartId)
{
	return InGizmoElement.GetSubElementsRecursive(OutElements, InPartId);
}

bool FGizmoElementAccessor::UpdateRenderState(UGizmoElementBase& InGizmoElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const
{
	FQuat AlignRot;
	bool bHasAlignRot;
	return UpdateRenderState(InGizmoElement, InRenderAPI, InLocalCenter, InOutRenderState, bHasAlignRot, AlignRot);
}

bool FGizmoElementAccessor::UpdateRenderState(UGizmoElementBase& InGizmoElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState, bool& bOutHasAlignRot, FQuat& OutAlignRot) const
{
	if (!ensure(InRenderAPI))
	{
		return false;
	}

	return InGizmoElement.UpdateRenderState(InRenderAPI, InLocalCenter, InOutRenderState, bOutHasAlignRot, OutAlignRot);
}

bool FGizmoElementAccessor::IsEnabledForInteractionState(const UGizmoElementBase& InGizmoElement, const TOptional<EGizmoElementInteractionState>& InInteractionState)
{
	return InGizmoElement.GetEnabledForInteractionState(InInteractionState.Get(InGizmoElement.GetElementInteractionState()));
}

bool FGizmoElementAccessor::IsPartial(
	UGizmoElementCircleBase& InGizmoElement,
	const FVector& InWorldCenter,
	const FVector& InWorldNormal,
	const FVector& InViewLocation,
	const FVector& InViewDirection,
	const bool bIsPerspectiveProjection)
{
	return InGizmoElement.IsPartial(InWorldCenter, InWorldNormal, InViewLocation, InViewDirection, bIsPerspectiveProjection);
}
