// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/HitTargets.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/HitResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HitTargets)

FInputRayHit UGizmoLambdaHitTarget::IsHit(const FInputDeviceRay& ClickPos) const
{
	if (IsHitFunction)
	{
		return IsHitFunction(ClickPos);
	}
	return FInputRayHit();
}

void UGizmoLambdaHitTarget::UpdateHoverState(bool bHovering)
{
	if (UpdateHoverFunction)
	{
		UpdateHoverFunction(bHovering);
	}
}

void UGizmoLambdaHitTarget::UpdateInteractingState(bool bInteracting)
{
	if (UpdateInteractingFunction)
	{
		UpdateInteractingFunction(bInteracting);
	}
}

void UGizmoLambdaHitTarget::UpdateSelectedState(bool bSelected)
{
	if (UpdateSelectedFunction)
	{
		UpdateSelectedFunction(bSelected);
	}
}

void UGizmoLambdaHitTarget::UpdateSubdueState(bool bSubdued)
{
	if (UpdateSubdueFunction)
	{
		UpdateSubdueFunction(bSubdued);
	}
}

FInputRayHit UGizmoComponentHitTarget::IsHit(const FInputDeviceRay& ClickPos) const
{
	FInputRayHit Hit;
	if (Component && (!Condition || Condition(ClickPos)))
	{
		// if a gizmo is not visible it cannot be hit
		bool bVisible = Component->IsVisible() && ( Component->GetOwner() && Component->GetOwner()->IsHidden() == false );
#if WITH_EDITOR		
		bVisible = bVisible && Component->IsVisibleInEditor() && (Component->GetOwner() && Component->GetOwner()->IsHiddenEd() == false);
#endif

		if (bVisible)
		{
			FVector End = ClickPos.WorldRay.PointAt(HALF_WORLD_MAX);
			FHitResult OutHit;
			if (Component->LineTraceComponent(OutHit, ClickPos.WorldRay.Origin, End, FCollisionQueryParams(SCENE_QUERY_STAT(HitTest), true)))
			{
				return FInputRayHit(OutHit.Distance);
			}
		}
	}
	return Hit;
}

void UGizmoComponentHitTarget::UpdateHoverState(bool bHovering)
{
	if (UpdateHoverFunction)
	{
		UpdateHoverFunction(bHovering);
	}
}

void UGizmoComponentHitTarget::UpdateInteractingState(bool bHovering)
{
	if (UpdateInteractingFunction)
	{
		UpdateInteractingFunction(bHovering);
	}
}

void UGizmoComponentHitTarget::UpdateSelectedState(bool bSelected)
{
	if (UpdateSelectedFunction)
	{
		UpdateSelectedFunction(bSelected);
	}
}

void UGizmoComponentHitTarget::UpdateSubdueState(bool bSubdued)
{
	if (UpdateSubduedFunction)
	{
		UpdateSubduedFunction(bSubdued);
	}
}

UGizmoComponentHitTarget* UGizmoComponentHitTarget::Construct(UPrimitiveComponent* Component, UObject* Outer)
{
	UGizmoComponentHitTarget* NewHitTarget = NewObject<UGizmoComponentHitTarget>(Outer);
	NewHitTarget->Component = Component;
	return NewHitTarget;
}

