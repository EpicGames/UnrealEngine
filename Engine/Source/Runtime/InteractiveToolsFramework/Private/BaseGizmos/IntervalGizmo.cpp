// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/IntervalGizmo.h"
#include "InteractiveGizmoManager.h"
#include "BaseGizmos/AxisPositionGizmo.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/GizmoLineHandleComponent.h"
#include "BaseGizmos/GizmoViewContext.h"

// need this to implement hover
#include "BaseGizmos/GizmoBaseComponent.h"

#include "Components/SphereComponent.h"
#include "Components/PrimitiveComponent.h"
#include "ContextObjectStore.h"
#include "Engine/World.h"
#include "Engine/CollisionProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntervalGizmo)


#define LOCTEXT_NAMESPACE "UIntervalGizmo"

/**
* FFloatParameterProxyChange tracks a change to the base transform for a FloatParameter
*/
class FFloatParameterProxyChange : public FToolCommandChange
{
public:
	FGizmoFloatParameterChange To;
	FGizmoFloatParameterChange From;


	virtual void Apply(UObject* Object) override 
	{
		UGizmoLocalFloatParameterSource* ParameterSource = CastChecked<UGizmoLocalFloatParameterSource>(Object);
		ParameterSource->SetParameter(To.CurrentValue);
	}
	virtual void Revert(UObject* Object) override
	{
		UGizmoLocalFloatParameterSource* ParameterSource = CastChecked<UGizmoLocalFloatParameterSource>(Object);
		ParameterSource->SetParameter(From.CurrentValue);
	}

	virtual FString ToString() const override { return TEXT("FFloatParameterProxyChange"); }
};

/**
 * FGizmoFloatParameterChangeSource generates FFloatParameterProxyChange instances on Begin/End.
 * Instances of this class can (for example) be attached to a UGizmoTransformChangeStateTarget for use TransformGizmo change tracking.
 */
class FGizmoFloatParameterChangeSource : public IToolCommandChangeSource
{
public:
	FGizmoFloatParameterChangeSource(UGizmoLocalFloatParameterSource* ProxyIn)
	{
		Proxy = ProxyIn;
	}

	virtual ~FGizmoFloatParameterChangeSource() {}

	TWeakObjectPtr<UGizmoLocalFloatParameterSource> Proxy;
	TUniquePtr<FFloatParameterProxyChange> ActiveChange;

	virtual void BeginChange() override
	{
		if (Proxy.IsValid())
		{
			ActiveChange = MakeUnique<FFloatParameterProxyChange>();
			ActiveChange->From = Proxy->LastChange;
		}
	}
	virtual TUniquePtr<FToolCommandChange> EndChange() override
	{
		if (Proxy.IsValid())
		{
			ActiveChange->To = Proxy->LastChange;
			return MoveTemp(ActiveChange);
		}
		return TUniquePtr<FToolCommandChange>();
	}
	virtual void CancelChange() override
	{
		
	}
	virtual UObject* GetChangeTarget() override
	{
		return Proxy.Get();
	}
	virtual FText GetChangeDescription() override
	{
		return LOCTEXT("FFGizmoFloatParameterChangeDescription", "GizmoFloatParameterChange");
	}
};

/**
 * This change source doesn't actually issue any valid transactions. Instead, it is a helper class 
 * that can get attached to the interval gizmo's state target to fire off BeginEditSequence and 
 * EndEditSequence on the start/end of a drag.
 */
class FIntervalGizmoChangeBroadcaster : public IToolCommandChangeSource
{
public:
	FIntervalGizmoChangeBroadcaster(UIntervalGizmo* IntervalGizmoIn) : IntervalGizmo(IntervalGizmoIn) {}

	virtual ~FIntervalGizmoChangeBroadcaster() {}

	TWeakObjectPtr<UIntervalGizmo> IntervalGizmo;

	virtual void BeginChange() override
	{
		if (IntervalGizmo.IsValid())
		{
			IntervalGizmo->BeginEditSequence();
		}
	}
	virtual TUniquePtr<FToolCommandChange> EndChange() override
	{
		if (IntervalGizmo.IsValid())
		{
			IntervalGizmo->EndEditSequence();
		}
		return TUniquePtr<FToolCommandChange>();
	}
	virtual void CancelChange() override
	{
		if (IntervalGizmo.IsValid())
		{
			IntervalGizmo->EndEditSequence();
		}
	}
	virtual UObject* GetChangeTarget() override
	{
		return IntervalGizmo.Get();
	}
	virtual FText GetChangeDescription() override
	{
		return LOCTEXT("FIntervalGizmoChangeBroadcaster", "IntervalGizmoEdit");
	}
};


AIntervalGizmoActor::AIntervalGizmoActor()
{
	// root component is a hidden sphere
	USphereComponent* SphereComponent = CreateDefaultSubobject<USphereComponent>(TEXT("GizmoCenter"));
	RootComponent = SphereComponent;
	SphereComponent->InitSphereRadius(1.0f);
	SphereComponent->SetVisibility(false);
	SphereComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

AIntervalGizmoActor* AIntervalGizmoActor::ConstructDefaultIntervalGizmo(UWorld* World, UGizmoViewContext* GizmoViewContext)
{
	FActorSpawnParameters SpawnInfo;
	AIntervalGizmoActor* NewActor = World->SpawnActor<AIntervalGizmoActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);

	const FLinearColor MintGreen(152 / 255.f, 1.f, 152 / 255.f);
	
	// add all possible interval components (note: some may be hidden / unused)
	NewActor->UpIntervalComponent      = AddDefaultLineHandleComponent(World, NewActor, GizmoViewContext, MintGreen, FVector(0, 1, 0), FVector(0, 0, 1));
	NewActor->DownIntervalComponent    = AddDefaultLineHandleComponent(World, NewActor, GizmoViewContext, MintGreen, FVector(0, 1, 0), FVector(0, 0, 1));
	NewActor->ForwardIntervalComponent = AddDefaultLineHandleComponent(World, NewActor, GizmoViewContext, MintGreen, FVector(1, 0, 0), FVector(0, 1, 0));
	NewActor->BackwardIntervalComponent = AddDefaultLineHandleComponent(World, NewActor, GizmoViewContext, MintGreen, FVector(1, 0, 0), FVector(0, 1, 0));
	NewActor->RightIntervalComponent = AddDefaultLineHandleComponent(World, NewActor, GizmoViewContext, MintGreen, FVector(0, 0, 1), FVector(1, 0, 0));
	NewActor->LeftIntervalComponent = AddDefaultLineHandleComponent(World, NewActor, GizmoViewContext, MintGreen, FVector(0, 0, 1), FVector(1, 0, 0));

	return NewActor;
}



UInteractiveGizmo* UIntervalGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UIntervalGizmo* NewGizmo = NewObject<UIntervalGizmo>(SceneState.GizmoManager);
	NewGizmo->SetWorld(SceneState.World);

	UGizmoViewContext* GizmoViewContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	check(GizmoViewContext && GizmoViewContext->IsValidLowLevel());

	// use default gizmo actor if client has not given us a new builder
	NewGizmo->SetGizmoActorBuilder(GizmoActorBuilder ? GizmoActorBuilder : MakeShared<FIntervalGizmoActorFactory>(GizmoViewContext));

	// override default hover function if proposed
	if (UpdateHoverFunction)
	{
		NewGizmo->SetUpdateHoverFunction(UpdateHoverFunction);
	}

	if (UpdateCoordSystemFunction)
	{
		NewGizmo->SetUpdateCoordSystemFunction(UpdateCoordSystemFunction);
	}

	return NewGizmo;
}

// Init static FName
FString UIntervalGizmo::GizmoName = TEXT("IntervalGizmo");

void UIntervalGizmo::SetWorld(UWorld* WorldIn)
{
	this->World = WorldIn;
}

void UIntervalGizmo::SetGizmoActorBuilder(TSharedPtr<FIntervalGizmoActorFactory> Builder)
{
	GizmoActorBuilder = Builder;
}

void UIntervalGizmo::SetUpdateHoverFunction(TFunction<void(UPrimitiveComponent*, bool)> HoverFunction)
{
	UpdateHoverFunction = HoverFunction;
}

void UIntervalGizmo::SetUpdateCoordSystemFunction(TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> CoordSysFunction)
{
	UpdateCoordSystemFunction = CoordSysFunction;
}

void UIntervalGizmo::SetWorldAlignmentFunctions(TUniqueFunction<bool()>&& ShouldAlignDestinationIn, TUniqueFunction<bool(const FRay&, FVector&)>&& DestinationAlignmentRayCasterIn)
{
	// Save these so that any later gizmo resets (using SetActiveTarget) keep the settings.
	ShouldAlignDestination = MoveTemp(ShouldAlignDestinationIn);
	DestinationAlignmentRayCaster = MoveTemp(DestinationAlignmentRayCasterIn);

	for (UInteractiveGizmo* SubGizmo : this->ActiveGizmos)
	{
		if (UAxisPositionGizmo* CastGizmo = Cast<UAxisPositionGizmo>(SubGizmo))
		{
			CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
			CastGizmo->CustomDestinationFunc =
				[this](const UAxisPositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
				return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
			};
			CastGizmo->bCustomDestinationAlignsAxisOrigin = false; // We're aligning the endpoints of the intervals
		}
	}
}

void UIntervalGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	UpdateHoverFunction = [](UPrimitiveComponent* Component, bool bHovering)
	{
		if (Cast<UGizmoBaseComponent>(Component) != nullptr)
		{
			Cast<UGizmoBaseComponent>(Component)->UpdateHoverState(bHovering);
		}
	};

	UpdateCoordSystemFunction = [](UPrimitiveComponent* Component, EToolContextCoordinateSystem CoordSystem)
	{
		if (Cast<UGizmoBaseComponent>(Component) != nullptr)
		{
			Cast<UGizmoBaseComponent>(Component)->UpdateWorldLocalState(CoordSystem == EToolContextCoordinateSystem::World);
		}
	};

	GizmoActor = GizmoActorBuilder->CreateNewGizmoActor(World);
}

void UIntervalGizmo::Shutdown()
{
	ClearActiveTarget();

	if (GizmoActor)
	{
		GizmoActor->Destroy();
		GizmoActor = nullptr;
	}

	ClearSources();

}

void UIntervalGizmo::Tick(float DeltaTime)
{
	EToolContextCoordinateSystem CoordSystem = GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem();
	check(CoordSystem == EToolContextCoordinateSystem::World || CoordSystem == EToolContextCoordinateSystem::Local)
		bool bUseLocalAxes =
		(GetGizmoManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::Local);

	// Update gizmo location.
	{
		USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();
		// move gizmo to target location
		FTransform TargetTransform = TransformProxy->GetTransform();
		FVector SaveScale = TargetTransform.GetScale3D();
		TargetTransform.SetScale3D(FVector(1, 1, 1));
		GizmoComponent->SetWorldTransform(TargetTransform);
	}
	// Update the lengths
	EnumerateValidIntervals([this](UGizmoLocalFloatParameterSource* Source, UGizmoLineHandleComponent* Component, UGizmoComponentAxisSource* Axis, UE::Geometry::FInterval1f& IntervalRange, FVector3d Direction, float DirectionAxisSign)
	{
		if (Component)
		{
			Component->Length = Source->GetParameter();
		}
	});
	
	if (UpdateCoordSystemFunction)
	{
		for (UPrimitiveComponent* Component : ActiveComponents)
		{
			UpdateCoordSystemFunction(Component, CoordSystem);
		}
	}
}
void UIntervalGizmo::SetActiveTarget(UTransformProxy* TransformTargetIn, UGizmoLocalFloatParameterSource* UpInterval, UGizmoLocalFloatParameterSource* DownInterval, UGizmoLocalFloatParameterSource* ForwardInterval, IToolContextTransactionProvider* TransactionProvider)
{
	FParameterSources Sources;
	Sources.UpInterval = UpInterval;
	Sources.DownInterval = DownInterval;
	Sources.ForwardInterval = ForwardInterval;
	SetActiveTarget(TransformTargetIn, Sources, TransactionProvider);
}

void UIntervalGizmo::SetActiveTarget(UTransformProxy* TransformTargetIn, const FParameterSources& ParameterSources, IToolContextTransactionProvider* TransactionProvider)
{
	if (TransformProxy != nullptr)
	{
		ClearActiveTarget();
		ClearSources();
	}

	// This state target emits an explicit FChange that moves the GizmoActor root component during undo/redo.
	// It also opens/closes the Transaction that saves/restores the target object locations.
	if (TransactionProvider == nullptr)
	{
		TransactionProvider = GetGizmoManager();
	}

	TransformProxy = TransformTargetIn;

	// parameters and init lengths for each interval
	UpIntervalSource      = ParameterSources.UpInterval;
	DownIntervalSource    = ParameterSources.DownInterval;
	ForwardIntervalSource = ParameterSources.ForwardInterval;
	BackwardIntervalSource = ParameterSources.BackwardInterval;
	RightIntervalSource = ParameterSources.RightInterval;
	LeftIntervalSource = ParameterSources.LeftInterval;

	if (ParameterSources.InitParameterRanges == EDefaultParameterRanges::HalfRange)
	{
		EnumerateAllIntervals([](UGizmoLocalFloatParameterSource* Source, UGizmoLocalFloatParameterSource* OppositeSource, UGizmoLineHandleComponent* Component, UE::Geometry::FInterval1f& IntervalRange, float DirectionAxisSign)
			{
				IntervalRange = DirectionAxisSign < 0 ? UE::Geometry::FInterval1f(-FLT_MAX, 0.f) : UE::Geometry::FInterval1f(0.f, FLT_MAX);
			}
		);
	}
	else if (ParameterSources.InitParameterRanges == EDefaultParameterRanges::FullRange)
	{
		EnumerateAllIntervals([](UGizmoLocalFloatParameterSource* Source, UGizmoLocalFloatParameterSource* OppositeSource, UGizmoLineHandleComponent* Component, UE::Geometry::FInterval1f& IntervalRange, float DirectionAxisSign)
			{
				IntervalRange = UE::Geometry::FInterval1f(-FLT_MAX, FLT_MAX);
			}
		);
	}
	else // EDefaultParameterRanges::HalfIfMatched
	{
		EnumerateAllIntervals([](UGizmoLocalFloatParameterSource* Source, UGizmoLocalFloatParameterSource* OppositeSource, UGizmoLineHandleComponent* Component, UE::Geometry::FInterval1f& IntervalRange, float DirectionAxisSign)
			{
				IntervalRange = UE::Geometry::FInterval1f(-FLT_MAX, FLT_MAX);
				if (OppositeSource) // if opposite interval exists, cut range in half
				{
					if (DirectionAxisSign < 0)
					{
						IntervalRange.Max = 0;
					}
					else
					{
						IntervalRange.Min = 0;
					}
				}
			}
		);
	}

	// Get the parameter source to notify our delegate of any changes
	EnumerateValidIntervals([this](UGizmoLocalFloatParameterSource* Source, UGizmoLineHandleComponent* Component, UGizmoComponentAxisSource* Axis, UE::Geometry::FInterval1f& IntervalRange, FVector3d Direction, float DirectionAxisSign)
	{
		Source->OnParameterChanged.AddWeakLambda(this, [this, Direction, DirectionAxisSign](IGizmoFloatParameterSource*, FGizmoFloatParameterChange Change) {
			OnIntervalChanged.Broadcast(this, Direction, DirectionAxisSign * Change.CurrentValue);
		});
	});

	USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();

	// move gizmo to target location
	FTransform TargetTransform = TransformTargetIn->GetTransform();
	FVector SaveScale = TargetTransform.GetScale3D();
	TargetTransform.SetScale3D(FVector(1, 1, 1));
	GizmoComponent->SetWorldTransform(TargetTransform);

	
	// TargetTransform tracks location of GizmoComponent. Note that TransformUpdated is not called during undo/redo transactions!
	// We currently rely on the transaction system to undo/redo target object locations. This will not work during runtime...
	GizmoComponent->TransformUpdated.AddLambda(
		[this, SaveScale](USceneComponent* Component, EUpdateTransformFlags /*UpdateTransformFlags*/, ETeleportType /*Teleport*/) {
		//this->GetGizmoManager()->DisplayMessage(TEXT("TRANSFORM UPDATED"), EToolMessageLevel::Internal);
		FTransform NewXForm = Component->GetComponentToWorld();
		NewXForm.SetScale3D(SaveScale);
		this->TransformProxy->SetTransform(NewXForm);
	});

	


	StateTarget = UGizmoTransformChangeStateTarget::Construct(GizmoComponent,
		LOCTEXT("UIntervalGizmoTransaction", "Interval"), TransactionProvider, this);
	StateTarget->DependentChangeSources.Add(MakeUnique<FTransformProxyChangeSource>(TransformProxy));
	EnumerateValidIntervals([this](UGizmoLocalFloatParameterSource* Source, UGizmoLineHandleComponent* Component, UGizmoComponentAxisSource* Axis, UE::Geometry::FInterval1f& IntervalRange, FVector3d Direction, float DirectionAxisSign)
	{
		StateTarget->DependentChangeSources.Add(MakeUnique<FGizmoFloatParameterChangeSource>(Source));
	});

	// Have the state target notify us of the start/end of drags
	StateTarget->DependentChangeSources.Add(MakeUnique<FIntervalGizmoChangeBroadcaster>(this));

	// root component provides local X/Y/Z axis, identified by AxisIndex
	AxisXSource = UGizmoComponentAxisSource::Construct(GizmoComponent, 0, true, this);
	AxisYSource = UGizmoComponentAxisSource::Construct(GizmoComponent, 1, true, this);
	AxisZSource = UGizmoComponentAxisSource::Construct(GizmoComponent, 2, true, this);
	
	EnumerateAllIntervals([](UGizmoLocalFloatParameterSource* Source, UGizmoLocalFloatParameterSource* OppositeSource, UGizmoLineHandleComponent* Component, UE::Geometry::FInterval1f& IntervalRange, float DirectionAxisSign)
	{
		Component->SetVisibility(false);
	});
	EnumerateValidIntervals([this, GizmoComponent](UGizmoLocalFloatParameterSource* Source, UGizmoLineHandleComponent* Component, UGizmoComponentAxisSource* Axis, UE::Geometry::FInterval1f& IntervalRange, FVector3d Direction, float DirectionAxisSign)
	{
		AddIntervalHandleGizmo(GizmoComponent, Component, Axis, Source, IntervalRange.Min, IntervalRange.Max, StateTarget);
		ActiveComponents.Add(Component);
		Component->SetVisibility(true);
	});
}

void UIntervalGizmo::SetVisibility(bool bVisible)
{
	GizmoActor->SetActorHiddenInGame(bVisible == false);
#if WITH_EDITOR
	GizmoActor->SetIsTemporarilyHiddenInEditor(bVisible == false);
#endif
}

void UIntervalGizmo::ClearSources()
{
	UpIntervalSource = nullptr;
	DownIntervalSource = nullptr;
	ForwardIntervalSource = nullptr;
	BackwardIntervalSource = nullptr;
	RightIntervalSource = nullptr;
	LeftIntervalSource = nullptr;
}

void UIntervalGizmo::ClearActiveTarget()
{
	for (UInteractiveGizmo* Gizmo : ActiveGizmos)
	{
		GetGizmoManager()->DestroyGizmo(Gizmo);
	}
	ActiveGizmos.Empty();
	ActiveComponents.Empty();
	
	ClearSources();

	TransformProxy = nullptr;
}

FTransform UIntervalGizmo::GetGizmoTransform() const
{
	return TransformProxy->GetTransform();
}

UInteractiveGizmo* UIntervalGizmo::AddIntervalHandleGizmo(
	USceneComponent* RootComponent,
	UPrimitiveComponent* HandleComponent,
	IGizmoAxisSource* AxisSource,
	IGizmoFloatParameterSource* FloatParameterSource,
	float MinParameter,
	float MaxParameter,
	IGizmoStateTarget* StateTargetIn)
{
	// create axis-position gizmo, axis-position parameter will drive translation
	UAxisPositionGizmo* IntervalGizmo = Cast<UAxisPositionGizmo>(GetGizmoManager()->CreateGizmo(
		UInteractiveGizmoManager::DefaultAxisPositionBuilderIdentifier));
	check(IntervalGizmo);

	// axis source provides the scale axis
	IntervalGizmo->AxisSource = Cast<UObject>(AxisSource);


	// parameter source maps axis-parameter-change to change in interval length
	IntervalGizmo->ParameterSource = UGizmoAxisIntervalParameterSource::Construct(FloatParameterSource, MinParameter, MaxParameter, this);

	// sub-component provides hit target
	UGizmoComponentHitTarget* HitTarget = UGizmoComponentHitTarget::Construct(HandleComponent, this);
	if (this->UpdateHoverFunction)
	{
		HitTarget->UpdateHoverFunction = [HandleComponent, this](bool bHovering) { this->UpdateHoverFunction(HandleComponent, bHovering); };
	}
	IntervalGizmo->HitTarget = HitTarget;

	IntervalGizmo->StateTarget = Cast<UObject>(StateTargetIn);

	IntervalGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignDestination(); };
	IntervalGizmo->CustomDestinationFunc =
		[this](const UAxisPositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return DestinationAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	ActiveGizmos.Add(IntervalGizmo);

	return IntervalGizmo;
}

// Call IterFn on each Source/Component combination where the Source is not null. Note IterFn will still be called if the component and/or axis are null.
void UIntervalGizmo::EnumerateValidIntervals(
	TFunctionRef<void(UGizmoLocalFloatParameterSource* Source, UGizmoLineHandleComponent* Component, UGizmoComponentAxisSource* Axis, UE::Geometry::FInterval1f& IntervalRange, FVector3d Direction, float DirectionAxisSign)> IterFn
)
{
	constexpr float IntervalMax = FLT_MAX;
	if (RightIntervalSource)
	{
		IterFn(RightIntervalSource, GizmoActor ? GizmoActor->RightIntervalComponent : nullptr, AxisXSource, RightIntervalRange, FVector3d(1, 0, 0), 1.f);
	}
	if (LeftIntervalSource)
	{
		IterFn(LeftIntervalSource, GizmoActor ? GizmoActor->LeftIntervalComponent : nullptr, AxisXSource, LeftIntervalRange, FVector3d(-1, 0, 0), -1.f);
	}

	if (ForwardIntervalSource)
	{
		IterFn(ForwardIntervalSource, GizmoActor ? GizmoActor->ForwardIntervalComponent : nullptr, AxisYSource, ForwardIntervalRange, FVector3d(0, 1, 0), 1.f);
	}
	if (BackwardIntervalSource)
	{
		IterFn(BackwardIntervalSource, GizmoActor ? GizmoActor->BackwardIntervalComponent : nullptr, AxisYSource, BackwardIntervalRange, FVector3d(0, -1, 0), -1.f);
	}

	if (UpIntervalSource)
	{
		IterFn(UpIntervalSource, GizmoActor ? GizmoActor->UpIntervalComponent : nullptr, AxisZSource, UpIntervalRange, FVector3d(0, 0, 1), 1.f);
	}
	if (DownIntervalSource)
	{
		IterFn(DownIntervalSource, GizmoActor ? GizmoActor->DownIntervalComponent : nullptr, AxisZSource, DownIntervalRange, FVector3d(0, 0, -1), -1.f);
	}
}

// Call IterFn on each Source/Component combination, including those where the Source is null
void UIntervalGizmo::EnumerateAllIntervals(TFunctionRef<void(UGizmoLocalFloatParameterSource* Source, UGizmoLocalFloatParameterSource* OppositeSource, UGizmoLineHandleComponent* Component, UE::Geometry::FInterval1f& IntervalRange, float DirectionAxisSign)> IterFn)
{
	if (GizmoActor)
	{
		IterFn(RightIntervalSource, LeftIntervalSource, GizmoActor->RightIntervalComponent, RightIntervalRange, 1.f);
		IterFn(LeftIntervalSource, RightIntervalSource, GizmoActor->LeftIntervalComponent, LeftIntervalRange, -1.f);
		IterFn(ForwardIntervalSource, BackwardIntervalSource, GizmoActor->ForwardIntervalComponent, ForwardIntervalRange, 1.f);
		IterFn(BackwardIntervalSource, ForwardIntervalSource, GizmoActor->BackwardIntervalComponent, BackwardIntervalRange, -1.f);
		IterFn(UpIntervalSource, DownIntervalSource, GizmoActor->UpIntervalComponent, UpIntervalRange, 1.f);
		IterFn(DownIntervalSource, UpIntervalSource, GizmoActor->DownIntervalComponent, DownIntervalRange, -1.f);
	}
}

float UGizmoAxisIntervalParameterSource::GetParameter() const
{
	return FloatParameterSource->GetParameter();
}

void UGizmoAxisIntervalParameterSource::SetParameter(float NewValue) 
{

	NewValue = FMath::Clamp(NewValue, MinParameter, MaxParameter);

	FloatParameterSource->SetParameter(NewValue);

}

void UGizmoAxisIntervalParameterSource::BeginModify()
{
	FloatParameterSource->BeginModify();
}

void UGizmoAxisIntervalParameterSource::EndModify()
{
	FloatParameterSource->EndModify();
}

UGizmoAxisIntervalParameterSource* UGizmoAxisIntervalParameterSource::Construct(
	IGizmoFloatParameterSource* FloatSourceIn,
	float ParameterMin,
	float ParameterMax,
	UObject* Outer)
{
	UGizmoAxisIntervalParameterSource* NewSource = NewObject<UGizmoAxisIntervalParameterSource>(Outer);

	NewSource->FloatParameterSource = Cast<UObject>(FloatSourceIn);

	// Clamp the initial value
	float DefaultValue = NewSource->FloatParameterSource->GetParameter();
	DefaultValue = FMath::Clamp(DefaultValue, ParameterMin, ParameterMax);
	NewSource->FloatParameterSource->SetParameter(DefaultValue);

	// record the min / max allowed
	NewSource->MinParameter = ParameterMin;
	NewSource->MaxParameter = ParameterMax;

	return NewSource;
}

#undef LOCTEXT_NAMESPACE
