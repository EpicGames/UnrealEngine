// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "BaseGizmos/ParameterToTransformAdapters.h"
#include "BaseGizmos/GizmoMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParameterToTransformAdapters)


void UGizmoAxisTranslationParameterSource::SetParameter(float NewValue)
{
	Parameter = NewValue;
	LastChange.CurrentValue = NewValue;

	double UseDelta = LastChange.GetChangeDelta();

	// check for any constraints on the delta value
	double SnappedDelta = 0;
	if (AxisDeltaConstraintFunction(UseDelta, SnappedDelta))
	{
		UseDelta = SnappedDelta;
	}

	// construct translation as delta from initial position
	FVector Translation = UseDelta * CurTranslationAxis;

	// translate the initial transform
	FTransform NewTransform = InitialTransform;
	NewTransform.AddToTranslation(Translation);

	// apply position constraint
	FVector SnappedPos;
	if (PositionConstraintFunction(NewTransform.GetTranslation(), SnappedPos))
	{
		FVector SnappedLinePos = GizmoMath::ProjectPointOntoLine(SnappedPos, CurTranslationOrigin, CurTranslationAxis);
		NewTransform.SetTranslation(SnappedLinePos);
	}

	TransformSource->SetTransform(NewTransform);

	OnParameterChanged.Broadcast(this, LastChange);
}

void UGizmoAxisTranslationParameterSource::BeginModify()
{
	check(AxisSource);

	LastChange = FGizmoFloatParameterChange(Parameter);

	InitialTransform = TransformSource->GetTransform();
	CurTranslationAxis = AxisSource->GetDirection();
	CurTranslationOrigin = AxisSource->GetOrigin();
}

void UGizmoAxisTranslationParameterSource::EndModify()
{
}







void UGizmoPlaneTranslationParameterSource::SetParameter(const FVector2D& NewValue)
{
	Parameter = NewValue;
	LastChange.CurrentValue = NewValue;

	// construct translation as delta from initial position
	FVector2D Delta = LastChange.GetChangeDelta();
	double UseDeltaX = Delta.X;
	double UseDeltaY = Delta.Y;

	// check for any constraints on the delta value
	double SnappedDeltaX = 0, SnappedDeltaY = 0;
	if (AxisXDeltaConstraintFunction(UseDeltaX, SnappedDeltaX))
	{
		UseDeltaX = SnappedDeltaX;
	}
	if (AxisYDeltaConstraintFunction(UseDeltaY, SnappedDeltaY))
	{
		UseDeltaY = SnappedDeltaY;
	}

	FVector Translation = UseDeltaX*CurTranslationAxisX + UseDeltaY*CurTranslationAxisY;

	// apply translation to initial transform
	FTransform NewTransform = InitialTransform;
	NewTransform.AddToTranslation(Translation);

	// apply position constraint
	FVector SnappedPos;
	if (PositionConstraintFunction(NewTransform.GetTranslation(), SnappedPos))
	{
		if (bProjectConstrainedPosition)
		{
			SnappedPos = GizmoMath::ProjectPointOntoPlane(SnappedPos, CurTranslationOrigin, CurTranslationNormal);
		}
		NewTransform.SetTranslation(SnappedPos);
	}

	TransformSource->SetTransform(NewTransform);

	OnParameterChanged.Broadcast(this, LastChange);
}

void UGizmoPlaneTranslationParameterSource::BeginModify()
{
	check(AxisSource);

	LastChange = FGizmoVec2ParameterChange(Parameter);

	// save initial transformation and axis information
	InitialTransform = TransformSource->GetTransform();
	CurTranslationOrigin = AxisSource->GetOrigin();
	AxisSource->GetAxisFrame(CurTranslationNormal, CurTranslationAxisX, CurTranslationAxisY);
}

void UGizmoPlaneTranslationParameterSource::EndModify()
{
}



void UGizmoAxisRotationParameterSource::SetParameter(float NewValue)
{
	Angle = NewValue;
	LastChange.CurrentValue = NewValue;

	double AngleDelta = LastChange.GetChangeDelta();
	double SnappedDelta;
	if (AngleDeltaConstraintFunction(AngleDelta, SnappedDelta))
	{
		AngleDelta = SnappedDelta;
	}

	// construct rotation as delta from initial position
	FQuat DeltaRotation(CurRotationAxis, AngleDelta);
	DeltaRotation = RotationConstraintFunction(DeltaRotation);

	// rotate the vector from the rotation origin to the transform origin, 
	// to get the translation of the origin produced by the rotation
	FVector DeltaPosition = InitialTransform.GetLocation() - CurRotationOrigin;
	DeltaPosition = DeltaRotation * DeltaPosition;
	FVector NewLocation = CurRotationOrigin + DeltaPosition;

	// rotate the initial transform by the rotation
	FQuat NewRotation = DeltaRotation * InitialTransform.GetRotation();

	// construct new transform
	FTransform NewTransform = InitialTransform;
	NewTransform.SetLocation(NewLocation);
	NewTransform.SetRotation(NewRotation);
	TransformSource->SetTransform(NewTransform);

	OnParameterChanged.Broadcast(this, LastChange);
}


void UGizmoAxisRotationParameterSource::BeginModify()
{
	check(AxisSource != nullptr);

	LastChange = FGizmoFloatParameterChange(Angle);

	// save initial transformation and axis information
	InitialTransform = TransformSource->GetTransform();
	CurRotationAxis = AxisSource->GetDirection();
	CurRotationOrigin = AxisSource->GetOrigin();
}

void UGizmoAxisRotationParameterSource::EndModify()
{
}




void UGizmoUniformScaleParameterSource::SetParameter(const FVector2D& NewValue)
{
	Parameter = NewValue;
	LastChange.CurrentValue = NewValue;

	// Convert 2D parameter delta to a 1D uniform scale change
	// This possibly be exposed as a TFunction to allow customization?
	double SignedDelta = LastChange.GetChangeDelta().X + LastChange.GetChangeDelta().Y;
	SignedDelta *= ScaleMultiplier;

	FTransform NewTransform = InitialTransform;
	const FVector StartScale = InitialTransform.GetScale3D();
	
	double SnappedDelta;
	bool bIsSnapped = false;

	// if using snapping while scaling
	if (ScaleAxisDeltaConstraintFunction(SignedDelta, SnappedDelta))
	{
		SignedDelta = SnappedDelta;
		bIsSnapped = true;
	}

	FVector NewScale;
	// if the initial scale is uniform and snapping is on, we can use an additive method to scale up or down
	if (StartScale.IsUniform() && bIsSnapped)
	{
		NewScale = FVector(SignedDelta) + StartScale;
	}
	// otherwise, we need to use multiplication to scale
	// ex: initial scale is (1,2,4) and scale delta is .5 -> next incremented scale should be (1.5, 3, 6)
	//     to preserve proportions. Addition would result in (1.5, 2.5, 4.5) which does not keep original proportions.
	//     Additionally, using multiplication when scale is uniform would result in an ex where InitScale=(2,2,2) and
	//     ScaleDelta = .5 where next scale would be (3,3,3), where the intermediate scale of (2.5,2.5,2.5) is unreachable
	else
	{
		NewScale = SignedDelta * StartScale + StartScale;
	}

	// currently calling ScaleConstraintFunction has no effect (no changes made to SignedDelta) because this constraint function
	// is intended to relate to WorldGridSnapping. Currently WorldGridSnapping has no effect on scaling, with or without normal snapping
	// because the viewport scale mode fixes the transform space to local
	SignedDelta = ScaleConstraintFunction(SignedDelta);

	NewTransform.SetScale3D(NewScale);

	TransformSource->SetTransform(NewTransform);

	OnParameterChanged.Broadcast(this, LastChange);
}

void UGizmoUniformScaleParameterSource::BeginModify()
{
	check(AxisSource);

	LastChange = FGizmoVec2ParameterChange(Parameter);

	// save initial transformation and axis information
	InitialTransform = TransformSource->GetTransform();
	CurScaleOrigin = AxisSource->GetOrigin();
	// note: currently not used!
	AxisSource->GetAxisFrame(CurScaleNormal, CurScaleAxisX, CurScaleAxisY);
}

void UGizmoUniformScaleParameterSource::EndModify()
{
}





void UGizmoAxisScaleParameterSource::SetParameter(float NewValue)
{
	Parameter = NewValue;
	LastChange.CurrentValue = NewValue;
	
	double ScaleDelta = LastChange.GetChangeDelta();
	ScaleDelta *= ScaleMultiplier;

	FTransform NewTransform = InitialTransform;
	const FVector StartScale = InitialTransform.GetScale3D();

	FVector NewScale;
	double SnappedDelta;
	
	// check for any constraints on the delta value
	if (ScaleAxisDeltaConstraintFunction(ScaleDelta, SnappedDelta))
	{
		ScaleDelta = SnappedDelta;
		// use additive scaling when snap is on
		NewScale = StartScale + ScaleDelta * CurScaleAxis;
	}
	else
	{
		// use multiplicative scaling when snap is off
		NewScale = StartScale * (FVector3d{1} + (ScaleDelta * CurScaleAxis));
	}

	// currently calling ScaleConstraintFunction has no effect (no changes made to ScaleDelta) because this constraint function
	// is intended to relate to WorldGridSnapping. Currently WorldGridSnapping has no effect on scaling, with or without normal snapping
	// because the viewport scale mode fixes the transform space to local
	ScaleDelta = ScaleConstraintFunction(ScaleDelta);

	if (bClampToZero)
	{
		NewScale = FVector::Max(FVector::ZeroVector, NewScale);
	}

	NewTransform.SetScale3D(NewScale);

	TransformSource->SetTransform(NewTransform);

	OnParameterChanged.Broadcast(this, LastChange);
}

void UGizmoAxisScaleParameterSource::BeginModify()
{
	check(AxisSource);

	LastChange = FGizmoFloatParameterChange(Parameter);

	InitialTransform = TransformSource->GetTransform();
		
	CurScaleAxis = AxisSource->GetDirection();
	CurScaleOrigin = AxisSource->GetOrigin();
}

void UGizmoAxisScaleParameterSource::EndModify()
{
}





void UGizmoPlaneScaleParameterSource::SetParameter(const FVector2D& NewValue)
{
	Parameter = NewValue;
	LastChange.CurrentValue = NewValue;

	// construct Scale as delta from initial position
	FVector2D ScaleDelta = LastChange.GetChangeDelta() * ScaleMultiplier;

	if (bUseEqualScaling)
	{
		ScaleDelta = FVector2D(ScaleDelta.X + ScaleDelta.Y);
	}

	FTransform NewTransform = InitialTransform;
	const FVector StartScale = InitialTransform.GetScale3D();

	double UseScaleDeltaX = ScaleDelta.X;
	double UseScaleDeltaY = ScaleDelta.Y;
	
	FVector NewScale;

	if (bUseEqualScaling)
	{
		double SnappedDeltaX = 0.0, SnappedDeltaY = 0.0;
		bool bIsSnapped = true;
		
		// if using snapping while scaling on X and Y axis
		if (ScaleAxisXDeltaConstraintFunction(UseScaleDeltaX, SnappedDeltaX))
		{
			UseScaleDeltaX = SnappedDeltaX;
		}
		else
		{
			bIsSnapped = false;
		}
		if (ScaleAxisYDeltaConstraintFunction(UseScaleDeltaY, SnappedDeltaY))
		{
			UseScaleDeltaY = SnappedDeltaY;
		}
		else
		{
			bIsSnapped = false;
		}

		// determines if the initial scales of the 2 affected axes are equivalent, and if we can therefore use uniform scaling (additive function)
		const FVector AffectedValuesVector  = StartScale*CurScaleAxisX + StartScale*CurScaleAxisY;
		const bool bIsUniformAcrossScaleAxes = (AffectedValuesVector.X == AffectedValuesVector.Y) || (AffectedValuesVector.X == AffectedValuesVector.Z) || (AffectedValuesVector.Y == AffectedValuesVector.Z);

		// will use additive if scale is uniform across 2 axes of the plane AND snapping is on
		if (bIsUniformAcrossScaleAxes && bIsSnapped)
		{
			// uses an additive function to scale when both initial values are equal
			// ex: allows for a case where InitScale= (2,2,1) scaling by 1 across Z axis, next increment will be (3,3,1) instead of (4,4,1)
			NewScale = StartScale + UseScaleDeltaX*CurScaleAxisX + UseScaleDeltaY*CurScaleAxisY;
		}
		else
		{
			NewScale = StartScale + (UseScaleDeltaX * StartScale * CurScaleAxisX) + (UseScaleDeltaY * StartScale * CurScaleAxisY);
		}
	}
	else
	{
		NewScale = StartScale + (UseScaleDeltaX * StartScale * CurScaleAxisX) + (UseScaleDeltaY * StartScale * CurScaleAxisY);
	}
	
	// currently calling ScaleConstraintFunction has no effect (no changes made to SignedDelta) because this constraint function
	// is intended to relate to WorldGridSnapping. Currently WorldGridSnapping has no effect on scaling, with or without normal snapping
	// because the viewport scale mode fixes the transform space to local
	ScaleDelta = ScaleConstraintFunction(ScaleDelta);

	if (bClampToZero)
	{
		NewScale = FVector::Max(NewScale, FVector::ZeroVector);
	}

	NewTransform.SetScale3D(NewScale);

	TransformSource->SetTransform(NewTransform);

	OnParameterChanged.Broadcast(this, LastChange);
}

void UGizmoPlaneScaleParameterSource::BeginModify()
{
	check(AxisSource);

	LastChange = FGizmoVec2ParameterChange(Parameter);

	// save initial transformation and axis information
	InitialTransform = TransformSource->GetTransform();
	CurScaleOrigin = AxisSource->GetOrigin();
	AxisSource->GetAxisFrame(CurScaleNormal, CurScaleAxisX, CurScaleAxisY);
}

void UGizmoPlaneScaleParameterSource::EndModify()
{
}
