// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/PlanePositionGizmo.h"
#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // SetCommonSubGizmoProperties, UpdateCameraSource
#include "BaseGizmos/TransformSubGizmoUtil.h" // FTransformSubGizmoCommonParams
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlanePositionGizmo)

namespace PlanePositionGizmoLocals
{

}


UInteractiveGizmo* UPlanePositionGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UPlanePositionGizmo* NewGizmo = NewObject<UPlanePositionGizmo>(SceneState.GizmoManager);
	return NewGizmo;
}




void UPlanePositionGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	// Add default mouse input behavior
	MouseBehavior = NewObject<UClickDragInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(MouseBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(HoverBehavior);

	AxisSource = NewObject<UGizmoConstantAxisSource>(this);
	ParameterSource = NewObject<UGizmoLocalVec2ParameterSource>(this);
	HitTarget = NewObject<UGizmoComponentHitTarget>(this);
	StateTarget = NewObject<UGizmoNilStateTarget>(this);

	bInInteraction = false;
}

bool UPlanePositionGizmo::InitializeAsTranslateGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy
		|| Params.Axis == EAxis::None)
	{
		return false;
	}

	UGizmoScaledAndUnscaledTransformSources* TransformSource;
	if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();

	// Parameter source maps axis-parameter-change to translation of TransformSource's transform
	ParameterSource = UGizmoPlaneTranslationParameterSource::Construct(
		AxisSource.GetInterface(), TransformSource, Owner);

	return true;
}

bool UPlanePositionGizmo::InitializeAsScaleGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params, bool bDisallowNegativeScaling,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	if (!Params.Component
		|| !Params.TransformProxy
		|| Params.Axis == EAxis::None)
	{
		return false;
	}

	int AxisIndex = Params.GetClampedAxisIndex();

	UGizmoScaledAndUnscaledTransformSources* TransformSource;
	if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();
	bEnableSignedAxis = true;

	// Although the normal axis source gets used for detecting interactions, the parameter application has
	//  to happen along unrotated axes because the scaling gets applied before rotation. In other words if we
	//  tried to apply scaling measured along a rotated vector, we would end up incorrectly scaling along
	//  multiple axes.
	UGizmoComponentAxisSource* UnitCardinalAxisSource = nullptr;
	// See if we already have it in our shared state
	if (SharedState && SharedState->UnitCardinalAxisSources[AxisIndex])
	{
		UnitCardinalAxisSource = SharedState->UnitCardinalAxisSources[AxisIndex];
	}
	else
	{
		// Create new and add to shared state.
		USceneComponent* RootComponent = Params.Component->GetOwner()->GetRootComponent();
		UGizmoComponentAxisSource* CastAxisSource = UGizmoComponentAxisSource::Construct(RootComponent, AxisIndex,
			// bUseLocalAxes, not important because we're going to be updating this value every tick
			true,
			Owner);
		UnitCardinalAxisSource = CastAxisSource;
		if (SharedState)
		{
			SharedState->UnitCardinalAxisSources[AxisIndex] = UnitCardinalAxisSource;
		}
	}

	// Parameter source maps axis-parameter-change to scale of TransformSource's transform
	UGizmoPlaneScaleParameterSource* CastParameterSource = UGizmoPlaneScaleParameterSource::Construct(
		UnitCardinalAxisSource, TransformSource, Owner);
	ParameterSource = CastParameterSource;
	CastParameterSource->bClampToZero = bDisallowNegativeScaling;
	CastParameterSource->bUseEqualScaling = true;

	return true;
}

bool UPlanePositionGizmo::InitializeAsUniformScaleGizmo(
	const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
	bool bDisallowNegativeScaling,
	UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState)
{
	using namespace PlanePositionGizmoLocals;

	if (!Params.Component
		|| !Params.TransformProxy)
	{
		return false;
	}

	UGizmoScaledAndUnscaledTransformSources* TransformSource;

	// Make sure axis index is invalid so that the SetCommonSubGizmoProperties call below doesn't create
	// an axis source for us.
	if (!ensureMsgf(Params.Axis == EAxis::None, TEXT("InitializeAsUniformScaleGizmo uses a camera axis source.")))
	{
		UE::GizmoUtil::FTransformSubGizmoCommonParams ParamsCopy = Params;
		ParamsCopy.Axis = EAxis::None;
		if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, ParamsCopy, SharedState, TransformSource))
		{
			return false;
		}
	}
	else if (!UE::GizmoUtil::SetCommonSubGizmoProperties(this, Params, SharedState, TransformSource))
	{
		return false;
	}

	UObject* Owner = Params.OuterForSubobjects ? Params.OuterForSubobjects : GetTransientPackage();

	UGizmoConstantFrameAxisSource* CameraAxisSource = nullptr;

	// See if we already have it in our shared state
	if (SharedState && SharedState->CameraAxisSource)
	{
		CameraAxisSource = SharedState->CameraAxisSource;
	}
	else
	{
		// Create new and add to shared state.
		CameraAxisSource = NewObject<UGizmoConstantFrameAxisSource>(Owner);
		TWeakObjectPtr<AActor> OwnerActor = Params.Component->GetOwner();
		CustomTickFunction = [this, OwnerActor](float DeltaTime)
		{
			if (UGizmoConstantFrameAxisSource* FrameAxisSource = Cast<UGizmoConstantFrameAxisSource>(AxisSource.GetObject()))
			{
				UE::GizmoUtil::UpdateCameraAxisSource(
					*FrameAxisSource,
					GetGizmoManager(),
					OwnerActor.IsValid() ? OwnerActor->GetTransform().GetLocation() : FVector3d::ZeroVector);
			}
		};

		if (SharedState)
		{
			SharedState->CameraAxisSource = CameraAxisSource;
		}
	}
	AxisSource = CameraAxisSource;

	// Parameter source maps axis-parameter-change to scale of TransformSource's transform
	UGizmoUniformScaleParameterSource* CastParameterSource = UGizmoUniformScaleParameterSource::Construct(
		CameraAxisSource, TransformSource, Owner);
	ParameterSource = CastParameterSource;

	return true;
}

FInputRayHit UPlanePositionGizmo::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget && AxisSource && ParameterSource)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
		if (GizmoHit.bHit)
		{
			LastHitPosition = PressPos.WorldRay.PointAt(GizmoHit.HitDepth);
		}
	}
	return GizmoHit;
}

void UPlanePositionGizmo::OnClickPress(const FInputDeviceRay& PressPos)
{
	InteractionOrigin = LastHitPosition;
	InteractionNormal = AxisSource->GetDirection();
	if (AxisSource->HasTangentVectors())
	{
		AxisSource->GetTangentVectors(InteractionAxisX, InteractionAxisY);
	}
	else
	{
		GizmoMath::MakeNormalPlaneBasis(InteractionNormal, InteractionAxisX, InteractionAxisY);
	}

	bool bIntersects; FVector IntersectionPoint;
	GizmoMath::RayPlaneIntersectionPoint(
		InteractionOrigin, InteractionNormal,
		PressPos.WorldRay.Origin, PressPos.WorldRay.Direction,
		bIntersects, IntersectionPoint);
	if (!bIntersects)
	{
		// Generally should not happen since the user clicked the plane to start the interaction, but could happen in a floating point error edge case
		bInInteraction = false;
		return;
	}

	InteractionStartPoint = InteractionCurPoint = IntersectionPoint;

	FVector AxisOrigin = AxisSource->GetOrigin();

	double DirectionSignX = FVector::DotProduct(InteractionStartPoint - AxisOrigin, InteractionAxisX);
	ParameterSigns.X = (bEnableSignedAxis && DirectionSignX < 0) ? -1.0 : 1.0;
	ParameterSigns.X *= (bFlipX) ? -1.0 : 1.0;
	double DirectionSignY = FVector::DotProduct(InteractionStartPoint - AxisOrigin, InteractionAxisY);
	ParameterSigns.Y = (bEnableSignedAxis && DirectionSignY < 0) ? -1.0 : 1.0;
	ParameterSigns.Y *= (bFlipY) ? -1.0 : 1.0;

	InteractionStartParameter = GizmoMath::ComputeCoordinatesInPlane(IntersectionPoint,
			InteractionOrigin, InteractionNormal, InteractionAxisX, InteractionAxisY);

	// Figure out how the parameters would need to be adjusted to bring the axis origin to the
	// interaction start point. This is used when aligning the axis origin to a custom destination.
	FVector2D OriginParamValue = GizmoMath::ComputeCoordinatesInPlane(AxisOrigin,
		InteractionOrigin, InteractionNormal, InteractionAxisX, InteractionAxisY);
	InteractionStartOriginParameterOffset = InteractionStartParameter - OriginParamValue;

	InteractionStartParameter.X *= ParameterSigns.X;
	InteractionStartParameter.Y *= ParameterSigns.Y;
	InteractionCurParameter = InteractionStartParameter;

	InitialTargetParameter = ParameterSource->GetParameter();
	ParameterSource->BeginModify();

	bInInteraction = true;

	if (StateTarget)
	{
		StateTarget->BeginUpdate();
	}
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}

void UPlanePositionGizmo::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!bInInteraction)
	{
		return;
	}

	FVector HitPoint;
	FVector2D NewParamValue;

	// See if we should use the custom destination function.
	FCustomDestinationParams Params;
	Params.WorldRay = &DragPos.WorldRay;
	if (ShouldUseCustomDestinationFunc() && CustomDestinationFunc(Params, HitPoint))
	{
		InteractionCurPoint = GizmoMath::ProjectPointOntoPlane(HitPoint, InteractionOrigin, InteractionNormal);
		InteractionCurParameter = GizmoMath::ComputeCoordinatesInPlane(InteractionCurPoint,
			InteractionOrigin, InteractionNormal, InteractionAxisX, InteractionAxisY);

		InteractionCurParameter += InteractionStartOriginParameterOffset;
	}
	else
	{
		bool bIntersects; 
		GizmoMath::RayPlaneIntersectionPoint(
			InteractionOrigin, InteractionNormal,
			DragPos.WorldRay.Origin, DragPos.WorldRay.Direction,
			bIntersects, HitPoint);

		if (bIntersects == false)
		{
			return;
		}
		InteractionCurPoint = HitPoint;

		InteractionCurParameter = GizmoMath::ComputeCoordinatesInPlane(InteractionCurPoint,
			InteractionOrigin, InteractionNormal, InteractionAxisX, InteractionAxisY);
		InteractionCurParameter.X *= ParameterSigns.X;
		InteractionCurParameter.Y *= ParameterSigns.Y;
	}

	FVector2D DeltaParam = InteractionCurParameter - InteractionStartParameter;
	NewParamValue = InitialTargetParameter + DeltaParam;

	ParameterSource->SetParameter(NewParamValue);
}

void UPlanePositionGizmo::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	if (!bInInteraction)
	{
		return;
	}

	ParameterSource->EndModify();
	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}


void UPlanePositionGizmo::OnTerminateDragSequence()
{
	if (!bInInteraction)
	{
		return;
	}

	ParameterSource->EndModify();
	if (StateTarget)
	{
		StateTarget->EndUpdate();
	}
	bInInteraction = false;
	if (ensure(HitTarget))
	{
		HitTarget->UpdateInteractingState(bInInteraction);
	}
}





FInputRayHit UPlanePositionGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
	}
	return GizmoHit;
}

void UPlanePositionGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	HitTarget->UpdateHoverState(true);
}

bool UPlanePositionGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// not necessary...
	HitTarget->UpdateHoverState(true);
	return true;
}

void UPlanePositionGizmo::OnEndHover()
{
	HitTarget->UpdateHoverState(false);
}


void UPlanePositionGizmo::Tick(float DeltaTime)
{
	if (CustomTickFunction)
	{
		CustomTickFunction(DeltaTime);
	}
}
