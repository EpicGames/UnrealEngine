// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/RepositionableTransformGizmo.h"

#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseGizmos/AxisAngleGizmo.h"
#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/FreePositionSubGizmo.h"
#include "BaseGizmos/FreeRotationSubGizmo.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/HitTargets.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/TransformProxy.h"
#include "BaseGizmos/TransformSources.h"
#include "BaseGizmos/TransformSubGizmoUtil.h"
#include "ContextObjectStore.h"

#include "InteractiveGizmo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RepositionableTransformGizmo)

#define LOCTEXT_NAMESPACE "URepositionableTransformGizmo"


UInteractiveGizmo* URepositionableTransformGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UCombinedTransformGizmo* NewGizmo = NewObject<URepositionableTransformGizmo>(SceneState.GizmoManager);
	NewGizmo->SetWorld(SceneState.World);

	UGizmoViewContext* GizmoViewContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	check(GizmoViewContext && GizmoViewContext->IsValidLowLevel());

	// use default gizmo actor if client has not given us a new builder
	NewGizmo->SetGizmoActorBuilder(GizmoActorBuilder ? GizmoActorBuilder : MakeShared<FCombinedTransformGizmoActorFactory>(GizmoViewContext));

	NewGizmo->SetSubGizmoBuilderIdentifiers(AxisPositionBuilderIdentifier, PlanePositionBuilderIdentifier, AxisAngleBuilderIdentifier);


	// override default hover function if proposed
	if (UpdateHoverFunction)
	{
		NewGizmo->SetUpdateHoverFunction(UpdateHoverFunction);
	}

	if (UpdateCoordSystemFunction)
	{
		NewGizmo->SetUpdateCoordSystemFunction(UpdateCoordSystemFunction);
	}

	return NewGizmo;
}


void URepositionableTransformGizmo::SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider)
{
	if (!ensure(IsValid(GizmoActor)))
	{
		return;
	}

	// We're going to add on to the original transform gizmo, adding extra gizmos centered around
	// existing gizmo components. These will respond to middle clicks and take "set pivot" code
	// paths in their transform source and state target.
	UCombinedTransformGizmo::SetActiveTarget(Target, TransactionProvider);

	// Create the alternate transform source.
	UGizmoTransformProxyTransformSource* ProxyTransformSource = UGizmoTransformProxyTransformSource::Construct(ActiveTarget, this);
	ProxyTransformSource->bOverrideSetPivotMode = true;
	USceneComponent* GizmoComponent = GizmoActor->GetRootComponent();
	UGizmoScaledAndUnscaledTransformSources* TransformSource = UGizmoScaledAndUnscaledTransformSources::Construct(
		ProxyTransformSource, GizmoComponent, this);

	// The state target gives a different name to the transactions and issues different begin/end calls.
	RepositionStateTarget = UGizmoTransformChangeStateTarget::Construct(GizmoComponent,
		LOCTEXT("TransformGizmoTransaction", "Reposition Pivot"), 
		Cast<IToolContextTransactionProvider>(StateTarget->TransactionManager.GetObject()), 
		this);
	TUniquePtr<FTransformProxyChangeSource> ProxyChangeSource = MakeUnique<FTransformProxyChangeSource>(Target);
	ProxyChangeSource->bOverrideSetPivotMode = true;
	RepositionStateTarget->DependentChangeSources.Add(MoveTemp(ProxyChangeSource));

	// Pack the above up for use in the initialization functions
	FTransformSubGizmoSharedState SharedStateToUse;
	SharedStateToUse.TransformSource = TransformSource;
	SharedStateToUse.StateTarget = RepositionStateTarget;
	// The shared data struct should have been created in the base class SetActiveTarget
	if (ensure(SubGizmoSharedState.IsValid()))
	{
		for (int i = 0; i < 3; ++i)
		{
			SharedStateToUse.CardinalAxisSources[i] = SubGizmoSharedState->CardinalAxisSources[i];
		}
	}

	// Add on the extra gizmos to existing components. We use the base class addition functions and
	// then go back to tweak the parts that need changing using our own "Modify..." functions.
	// We also store the repositioning gizmos in an additional separate list so that we can
	// modify their alignment functions separately if we need to.

	UE::GizmoUtil::FTransformSubGizmoCommonParams Params;
	Params.TransformProxy = ActiveTarget;
	Params.TransactionProvider = TransactionProvider;
	Params.bManipulatesRootComponent = true;

	EAxis::Type Axes[3] = { EAxis::X, EAxis::Y, EAxis::Z };
	UPrimitiveComponent* TranslateAxisComponents[3]{ GizmoActor->TranslateX, GizmoActor->TranslateY, GizmoActor->TranslateZ };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (TranslateAxisComponents[AxisIndex])
		{
			Params.Component = TranslateAxisComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			UInteractiveGizmo* SubGizmo = AddAxisTranslationGizmo(Params, SharedStateToUse);
			ModifyPivotAxisGizmo(SubGizmo);
			PivotAlignmentGizmos.Add(SubGizmo);
		}
	}
	UPrimitiveComponent* TranslatePlaneComponents[3]{ GizmoActor->TranslateYZ, GizmoActor->TranslateXZ, GizmoActor->TranslateXY };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (TranslatePlaneComponents[AxisIndex])
		{
			Params.Component = TranslatePlaneComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			UInteractiveGizmo* SubGizmo = AddPlaneTranslationGizmo(Params, SharedStateToUse);
			ModifyPivotPlaneGizmo(SubGizmo);
			PivotAlignmentGizmos.Add(SubGizmo);
		}
	}
	UPrimitiveComponent* RotationAxisComponents[3]{ GizmoActor->RotateX, GizmoActor->RotateY, GizmoActor->RotateZ };
	for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (RotationAxisComponents[AxisIndex])
		{
			Params.Component = RotationAxisComponents[AxisIndex];
			Params.Axis = Axes[AxisIndex];
			UInteractiveGizmo* SubGizmo = AddAxisRotationGizmo(Params, SharedStateToUse);
			ModifyPivotRotateGizmo(SubGizmo);
			PivotAlignmentGizmos.Add(SubGizmo);
		}
	}

	Params.Axis = EAxis::None;
	if (GizmoActor->FreeTranslateHandle)
	{
		Params.Component = GizmoActor->FreeTranslateHandle;
		UInteractiveGizmo* SubGizmo = AddFreeTranslationGizmo(Params, SharedStateToUse);
		ModifyPivotFreeTranslationGizmo(SubGizmo);
		PivotAlignmentGizmos.Add(SubGizmo);
	}
	
	if (GizmoActor->FreeRotateHandle)
	{
		Params.Component = GizmoActor->FreeRotateHandle;
		UInteractiveGizmo* SubGizmo = AddFreeRotationGizmo(Params, SharedStateToUse);
		ModifyPivotFreeRotationGizmo(SubGizmo);
		PivotAlignmentGizmos.Add(SubGizmo);
	}
}

/** 
 * Modifies a UAxisPositionGizmo after its addition via AddAxisTranslationGizmo for use in 
 * gizmo repositioning.
 */
void URepositionableTransformGizmo::ModifyPivotAxisGizmo(UInteractiveGizmo* SubGizmoIn)
{
	UAxisPositionGizmo* SubGizmo = Cast<UAxisPositionGizmo>(SubGizmoIn);
	check(SubGizmo);
	SubGizmo->MouseBehavior->SetUseMiddleMouseButton();
	SubGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
	SubGizmo->CustomDestinationFunc =
		[this](const UAxisPositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(SubGizmo->HitTarget.GetObject());
	check(HitTarget);
	HitTarget->UpdateHoverFunction = nullptr; // Already taken care of by the normal gizmo
}

/**
 * Modifies a UPlanePositionGizmo after its addition via AddPlaneTranslationGizmo for use in
 * gizmo repositioning.
 */
void URepositionableTransformGizmo::ModifyPivotPlaneGizmo(UInteractiveGizmo* SubGizmoIn)
{
	UPlanePositionGizmo* SubGizmo = Cast<UPlanePositionGizmo>(SubGizmoIn);
	check(SubGizmo);
	SubGizmo->MouseBehavior->SetUseMiddleMouseButton();
	SubGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
	SubGizmo->CustomDestinationFunc =
		[this](const UPlanePositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(SubGizmo->HitTarget.GetObject());
	check(HitTarget);
	HitTarget->UpdateHoverFunction = nullptr; // Already taken care of by the normal gizmo
}

/**
 * Modifies a UAxisAngleGizmo after its addition via AddAxisRotationGizmo for use in
 * gizmo repositioning.
 */
void URepositionableTransformGizmo::ModifyPivotRotateGizmo(UInteractiveGizmo* SubGizmoIn)
{
	UAxisAngleGizmo* SubGizmo = Cast<UAxisAngleGizmo>(SubGizmoIn);
	check(SubGizmo);
	SubGizmo->MouseBehavior->SetUseMiddleMouseButton();
	SubGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
	SubGizmo->CustomDestinationFunc =
		[this](const UAxisAngleGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(SubGizmo->HitTarget.GetObject());
	check(HitTarget);
	HitTarget->UpdateHoverFunction = nullptr; // Already taken care of by the normal gizmo

	// We don't want to be able to reposition the rotation of the gizmo when using the world coordinate system,
	// since the axes must stay aligned with world axes. However we can't hide the component either because it
	// is used for regular rotation. Thus, we add a check that disables the reposition gizmo hit testing if
	// the coordinate system is not local.
	HitTarget->Condition = [this](const FInputDeviceRay&) {
		return (CurrentCoordinateSystem == EToolContextCoordinateSystem::Local);
	};
}

void URepositionableTransformGizmo::ModifyPivotFreeTranslationGizmo(UInteractiveGizmo* SubGizmoIn)
{
	UFreePositionSubGizmo* SubGizmo = Cast<UFreePositionSubGizmo>(SubGizmoIn);
	if (!ensure(SubGizmo)) { return; }

	SubGizmo->MouseBehavior->SetUseMiddleMouseButton();
	SubGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
	SubGizmo->CustomDestinationFunc =
		[this](const UPlanePositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
		return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
	};

	UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(SubGizmo->HitTarget.GetObject());
	check(HitTarget);
	HitTarget->UpdateHoverFunction = nullptr; // Already taken care of by the normal gizmo
}

void URepositionableTransformGizmo::ModifyPivotFreeRotationGizmo(UInteractiveGizmo* SubGizmoIn)
{
	UFreeRotationSubGizmo* SubGizmo = Cast<UFreeRotationSubGizmo>(SubGizmoIn);
	if (!ensure(SubGizmo)) { return; }

	SubGizmo->MouseBehavior->SetUseMiddleMouseButton();

	UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(SubGizmo->HitTarget.GetObject());
	check(HitTarget);
	HitTarget->UpdateHoverFunction = nullptr; // Already taken care of by the normal gizmo

	// We don't want to be able to reposition the rotation of the gizmo when using the world coordinate system,
	// since the axes must stay aligned with world axes. 
	HitTarget->Condition = [this](const FInputDeviceRay&) {
		return (CurrentCoordinateSystem == EToolContextCoordinateSystem::Local);
	};
}

void URepositionableTransformGizmo::SetWorldAlignmentFunctions(
	TUniqueFunction<bool()>&& ShouldAlignTranslationIn,
	TUniqueFunction<bool(const FRay&, FVector&)>&& TranslationAlignmentRayCasterIn)
{
	// The base class function will do most of what we want.
	UCombinedTransformGizmo::SetWorldAlignmentFunctions(MoveTemp(ShouldAlignTranslationIn), MoveTemp(TranslationAlignmentRayCasterIn));

	// However, we need to adjust our saved pivot alignment functions so changes of the gizmo target keep the settings.
	// Since the functions are TUniqueFunction, we can't actually copy them here, we need to create new functors that
	// forward to the other ones.
	ShouldAlignPivot = [this]() { return ShouldAlignDestination(); };
	PivotAlignmentRayCaster = [this](const FRay& WorldRay, FVector& OutputPoint) { 
		return DestinationAlignmentRayCaster(WorldRay, OutputPoint); };

}

void URepositionableTransformGizmo::SetPivotAlignmentFunctions(
	TUniqueFunction<bool()>&& ShouldAlignPivotIn,
	TUniqueFunction<bool(const FRay&, FVector&)>&& PivotAlignmentRayCasterIn)
{
	// Save these so that later changes of gizmo target keep the settings.
	ShouldAlignPivot = MoveTemp(ShouldAlignPivotIn);
	PivotAlignmentRayCaster = MoveTemp(PivotAlignmentRayCasterIn);

	// We allow this function to be called after Setup(), so modify any existing translation sub gizmos.
	for (UInteractiveGizmo* SubGizmo : PivotAlignmentGizmos)
	{
		if (UAxisPositionGizmo* CastGizmo = Cast<UAxisPositionGizmo>(SubGizmo))
		{
			CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
			CastGizmo->CustomDestinationFunc =
				[this](const UAxisPositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
				return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
			};
		}
		if (UPlanePositionGizmo* CastGizmo = Cast<UPlanePositionGizmo>(SubGizmo))
		{
			CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
			CastGizmo->CustomDestinationFunc =
				[this](const UPlanePositionGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
				return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
			};
		}
		if (UAxisAngleGizmo* CastGizmo = Cast<UAxisAngleGizmo>(SubGizmo))
		{
			CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
			CastGizmo->CustomDestinationFunc =
				[this](const UAxisAngleGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
				return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
			};
		}
		if (UFreePositionSubGizmo* CastGizmo = Cast<UFreePositionSubGizmo>(SubGizmo))
		{
			CastGizmo->ShouldUseCustomDestinationFunc = [this]() { return ShouldAlignPivot(); };
			CastGizmo->CustomDestinationFunc =
				[this](const UFreePositionSubGizmo::FCustomDestinationParams& Params, FVector& OutputPoint) {
				return PivotAlignmentRayCaster(*Params.WorldRay, OutputPoint);
			};
		}
	}
}

#undef LOCTEXT_NAMESPACE

