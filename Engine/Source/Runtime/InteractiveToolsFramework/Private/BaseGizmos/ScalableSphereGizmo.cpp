// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/ScalableSphereGizmo.h"
#include "InteractiveGizmoManager.h"
#include "PrimitiveDrawingUtils.h"
#include "BaseGizmos/GizmoBoxComponent.h"
#include "Components/SphereComponent.h"
#include "Engine/CollisionProfile.h"
#include "BaseGizmos/GizmoMath.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/GizmoRenderingUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScalableSphereGizmo)

#define LOCTEXT_NAMESPACE "UScalableSphereGizmo"
// UScalableSphereGizmoBuilder

UInteractiveGizmo* UScalableSphereGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UScalableSphereGizmo* NewGizmo = NewObject<UScalableSphereGizmo>(SceneState.GizmoManager);
	return NewGizmo;
}

// UScalableSphereGizmo

void UScalableSphereGizmo::Setup()
{
	UInteractiveGizmo::Setup();

	Radius = 1000.0f;

	TransactionDescription = LOCTEXT("ScalableSphereGizmo", "Scale Sphere Gizmo");

	UScalableSphereGizmoInputBehavior* ScalableSphereBehavior = NewObject<UScalableSphereGizmoInputBehavior>(this);
	ScalableSphereBehavior->Initialize(this);
	AddInputBehavior(ScalableSphereBehavior);

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);
}

void UScalableSphereGizmo::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (ActiveTarget)
	{
		FColor UseColor = FColor(200, 255, 255);

		// Draw the gizmo in yellow and draw lines showing drag direction if hovering or dragging
		if (bIsHovering || bIsDragging)
		{
			UseColor = FColor::Yellow;

			// Parameters for the line that shows the drag direction
			FVector LineStart = DragCurrentPositionProjected;
			float LineLength = 30.f;
			FVector LineEnd = LineStart + ActiveAxis * LineLength;

			// Get the Pixel to World scale of the line
			const FSceneView* View = RenderAPI->GetSceneView();
			float PixelToWorld = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(View, LineEnd);

			// Draw the lines in both directions
			RenderAPI->GetPrimitiveDrawInterface()->DrawLine(LineStart, LineStart + ActiveAxis * PixelToWorld * LineLength, FLinearColor::Red, SDPG_Foreground);
			RenderAPI->GetPrimitiveDrawInterface()->DrawLine(LineStart, LineStart - ActiveAxis * PixelToWorld * LineLength, FLinearColor::Red, SDPG_Foreground);
		}

		DrawWireSphereAutoSides(RenderAPI->GetPrimitiveDrawInterface(), ActiveTarget->GetTransform(), UseColor, Radius, SDPG_Foreground);
	}
}

FInputRayHit UScalableSphereGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FHitResult HitResult;
	FVector HitAxis;
	FTransform DragTransform;

	if (HitTest(PressPos.WorldRay, HitResult, HitAxis, DragTransform))
	{
		bIsHovering = true;
		DragStartWorldPosition = DragTransform.GetLocation();
		DragCurrentPositionProjected = DragStartWorldPosition;
		ActiveAxis = HitAxis;
		
		return FInputRayHit(HitResult.Distance);
	}
	
	bIsHovering = false;
	// Return invalid ray hit to say we don't want to listen to hover input
	return FInputRayHit();
}

bool UScalableSphereGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (!ActiveTarget)
	{
		bIsHovering = false;
		return false;
	}

	FVector Start = DevicePos.WorldRay.Origin;
	const float MaxRaycastDistance = 1e6f;
	FVector End = DevicePos.WorldRay.Origin + DevicePos.WorldRay.Direction * MaxRaycastDistance;

	FRay HitCheckRay(Start, End - Start);
	FHitResult HitResult;
	FVector HitAxis;
	FTransform DragTransform;

	if (HitTest(HitCheckRay, HitResult, HitAxis, DragTransform))
	{
		bIsHovering = true;
		DragStartWorldPosition = DragTransform.GetLocation();
		DragCurrentPositionProjected = DragStartWorldPosition;
		ActiveAxis = HitAxis;
		return true;
	}

	bIsHovering = false;
	return false;
}

void UScalableSphereGizmo::OnEndHover()
{
	bIsHovering = false;
}

void UScalableSphereGizmo::SetTarget(UTransformProxy* InTarget)
{
	ActiveTarget = InTarget;
}

bool UScalableSphereGizmo::CheckCircleIntersection(const FRay& Ray, FVector CircleNormal, FVector& OutHitLocation, FVector& OutHitAxis)
{
	// Find the intresection with the circle plane. Note that unlike the FMath version, GizmoMath::RayPlaneIntersectionPoint() 
	// checks that the ray isn't parallel to the plane.
	FVector WorldOrigin = ActiveTarget->GetTransform().GetLocation();

	FVector Start = Ray.Origin;
	const float MaxRaycastDistance = 1e6f;
	FVector End = Ray.Origin + Ray.Direction * MaxRaycastDistance;

	FVector HitPos;
	bool bIntersects = false;

	// Figure out if the ray intersects the circle plane
	GizmoMath::RayPlaneIntersectionPoint(WorldOrigin, CircleNormal, Ray.Origin, Ray.Direction, bIntersects, HitPos);

	if (!bIntersects || Ray.GetParameter(HitPos) > Ray.GetParameter(End))
	{
		return false;
	}

	// Find the point on the circle closest to the intersection
	FVector NearestCircle;
	GizmoMath::ClosetPointOnCircle(HitPos, WorldOrigin, CircleNormal, Radius, NearestCircle);

	FVector NearestRay = Ray.ClosestPoint(NearestCircle);

	// Make sure the point is within a certain distance of the ray
	double Distance = FVector::Distance(NearestCircle, NearestRay);

	if (Distance > HitErrorThreshold)
	{
		return false;
	}

	OutHitAxis = NearestCircle - WorldOrigin;
	OutHitAxis.Normalize();

	OutHitLocation = NearestCircle;
	return true;
}

bool UScalableSphereGizmo::HitTest(const FRay& Ray, FHitResult& OutHit, FVector& OutAxis, FTransform &OutTransform)
{
	if (!ActiveTarget)
	{
		return false;
	}

	FVector OutHitLocation;
	FVector OutHitAxis;

	if (CheckCircleIntersection(Ray, FVector::XAxisVector, OutHitLocation, OutHitAxis))
	{
		OutAxis = OutHitAxis;
		OutTransform.SetIdentity();
		OutTransform.SetTranslation(OutHitLocation);

		return true;
	}

	if (CheckCircleIntersection(Ray, FVector::YAxisVector, OutHitLocation, OutHitAxis))
	{
		OutAxis = OutHitAxis;
		OutTransform.SetIdentity();
		OutTransform.SetTranslation(OutHitLocation);

		return true;
	}

	if (CheckCircleIntersection(Ray, FVector::ZAxisVector, OutHitLocation, OutHitAxis))
	{
		OutAxis = OutHitAxis;
		OutTransform.SetIdentity();
		OutTransform.SetTranslation(OutHitLocation);

		return true;
	}

	return false;
}

void UScalableSphereGizmo::SetRadius(float InRadius)
{
	// Negative Radius not allowed
	if (InRadius < 0)
	{
		InRadius = 0;
	}

	Radius = InRadius;

	if (UpdateRadiusFunc)
	{
		UpdateRadiusFunc(Radius);
	}
}

void UScalableSphereGizmo::OnBeginDrag(const FInputDeviceRay& Ray)
{
	FVector Start = Ray.WorldRay.Origin;
	const float MaxRaycastDistance = 1e6f;
	FVector End = Ray.WorldRay.Origin + Ray.WorldRay.Direction * MaxRaycastDistance;

	FRay HitCheckRay(Start, End - Start);
	FHitResult HitResult;
	FVector HitAxis;
	FTransform DragTransform;

	 // Check if the Ray hit any of the handles
	if (HitTest(HitCheckRay, HitResult, HitAxis, DragTransform))
	{
		ActiveAxis = HitAxis;

		FVector RayNearestPt; 
		float RayNearestParam;
		FVector InteractionStartPoint;

		 // Find the initial parameters along the hit axis
		GizmoMath::NearestPointOnLineToRay(DragTransform.GetLocation(), ActiveAxis,
			Ray.WorldRay.Origin, Ray.WorldRay.Direction,
			InteractionStartPoint, InteractionStartParameter,
			RayNearestPt, RayNearestParam);

		DragStartWorldPosition = DragTransform.GetLocation();
		DragCurrentPositionProjected = DragStartWorldPosition;

		bIsDragging = true;

		GetGizmoManager()->BeginUndoTransaction(TransactionDescription);
	}
}

void UScalableSphereGizmo::OnUpdateDrag(const FInputDeviceRay& Ray)
{
	FVector AxisNearestPt; 
	float AxisNearestParam;
	FVector RayNearestPt; 
	float RayNearestParam;

	 // Find the parameters along the hit axis
	GizmoMath::NearestPointOnLineToRay(DragStartWorldPosition, ActiveAxis,
		Ray.WorldRay.Origin, Ray.WorldRay.Direction,
		AxisNearestPt, AxisNearestParam,
		RayNearestPt, RayNearestParam);

	float InteractionCurrentParameter = AxisNearestParam;

	float DeltaParam = InteractionCurrentParameter - InteractionStartParameter;

	InteractionStartParameter = InteractionCurrentParameter;

	DragCurrentPositionProjected = AxisNearestPt;

	 // Update the radius
	SetRadius(Radius + DeltaParam);
}

void UScalableSphereGizmo::OnEndDrag(const FInputDeviceRay& Ray)
{
	GetGizmoManager()->EndUndoTransaction();
	bIsDragging = false;
}

// UScalableSphereGizmoInputBehavior

void UScalableSphereGizmoInputBehavior::Initialize(UScalableSphereGizmo* InGizmo)
{
	Gizmo = InGizmo;
}

FInputCaptureRequest UScalableSphereGizmoInputBehavior::WantsCapture(const FInputDeviceState& input)
{
	if (IsPressed(input))
	{
		FHitResult HitResult;
		FVector HitAxis;
		FTransform DragTransform;
		if (Gizmo->HitTest(input.Mouse.WorldRay, HitResult, HitAxis, DragTransform))
		{
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.Distance);
		}
	}

	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate UScalableSphereGizmoInputBehavior::BeginCapture(const FInputDeviceState& input, EInputCaptureSide eSide)
{
	FInputDeviceRay DeviceRay(input.Mouse.WorldRay, input.Mouse.Position2D);
	LastWorldRay = DeviceRay.WorldRay;
	LastScreenPosition = DeviceRay.ScreenPosition;

	// Forward behavior to the Gizmo
	Gizmo->OnBeginDrag(DeviceRay);

	bInputDragCaptured = true;
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UScalableSphereGizmoInputBehavior::UpdateCapture(const FInputDeviceState& input, const FInputCaptureData& data)
{
	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid ray data.
	if (!input.IsFromDevice(GetSupportedDevices()))
	{
		return FInputCaptureUpdate::Continue();
	}

	FInputDeviceRay DeviceRay(input.Mouse.WorldRay, input.Mouse.Position2D);
	LastWorldRay = DeviceRay.WorldRay;
	LastScreenPosition = DeviceRay.ScreenPosition;

	if (IsReleased(input))
	{
		bInputDragCaptured = false;
		Gizmo->OnEndDrag(FInputDeviceRay(LastWorldRay));
		return FInputCaptureUpdate::End();
	}

	// Forward behavior to the Gizmo
	Gizmo->OnUpdateDrag(FInputDeviceRay(LastWorldRay));

	return FInputCaptureUpdate::Continue();
}

void UScalableSphereGizmoInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	if (bInputDragCaptured)
	{
		bInputDragCaptured = false;
		Gizmo->OnEndDrag(FInputDeviceRay(LastWorldRay));
	}
}

#undef LOCTEXT_NAMESPACE
