// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/SimpleSingleClickGizmo.h"

#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseGizmos/GizmoBaseComponent.h" // IGizmoBaseComponentInterface
#include "BaseGizmos/HitTargets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleSingleClickGizmo)

bool USimpleSingleClickGizmo::InitializeWithComponent(UPrimitiveComponent* ComponentIn)
{
	if (!ComponentIn)
	{
		return false;
	}

	UGizmoComponentHitTarget* CastHitTarget = UGizmoComponentHitTarget::Construct(ComponentIn, this);
	CastHitTarget->UpdateHoverFunction = [ComponentIn](bool bHovering)
	{
		if (IGizmoBaseComponentInterface* Hoverable = Cast<IGizmoBaseComponentInterface>(ComponentIn))
		{
			Hoverable->UpdateHoverState(bHovering);
		}
	};
	HitTarget = CastHitTarget;
	return true;
}

void USimpleSingleClickGizmo::Setup()
{
	ClickBehavior = NewObject<USingleClickInputBehavior>(this);
	ClickBehavior->Initialize(this);
	ClickBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(ClickBehavior);
	
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY));
	AddInputBehavior(HoverBehavior);
}

FInputRayHit USimpleSingleClickGizmo::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget)
	{
		GizmoHit = HitTarget->IsHit(ClickPos);
	}
	return GizmoHit;
}

void USimpleSingleClickGizmo::OnClicked(const FInputDeviceRay& ClickPos)
{
	OnClick.Broadcast(*this, ClickPos);
}

FInputRayHit USimpleSingleClickGizmo::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit GizmoHit;
	if (HitTarget)
	{
		GizmoHit = HitTarget->IsHit(PressPos);
	}
	return GizmoHit;
}

void USimpleSingleClickGizmo::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	HitTarget->UpdateHoverState(true);
}

bool USimpleSingleClickGizmo::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return true;
}

void USimpleSingleClickGizmo::OnEndHover()
{
	HitTarget->UpdateHoverState(false);
}
