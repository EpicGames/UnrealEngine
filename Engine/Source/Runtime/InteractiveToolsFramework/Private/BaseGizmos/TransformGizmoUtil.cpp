// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/TransformGizmoUtil.h"

#include "ContextObjectStore.h"
#include "InteractiveToolsContext.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"

#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/AxisAngleGizmo.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/RepositionableTransformGizmo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformGizmoUtil)

const FString UCombinedTransformGizmoContextObject::DefaultAxisPositionBuilderIdentifier = TEXT("Util_StandardXFormAxisTranslationGizmo");
const FString UCombinedTransformGizmoContextObject::DefaultPlanePositionBuilderIdentifier = TEXT("Util_StandardXFormPlaneTranslationGizmo");
const FString UCombinedTransformGizmoContextObject::DefaultAxisAngleBuilderIdentifier = TEXT("Util_StandardXFormAxisRotationGizmo");
const FString UCombinedTransformGizmoContextObject::DefaultThreeAxisTransformBuilderIdentifier = TEXT("Util_DefaultThreeAxisTransformBuilderIdentifier");
const FString UCombinedTransformGizmoContextObject::CustomThreeAxisTransformBuilderIdentifier = TEXT("Util_CustomThreeAxisTransformBuilderIdentifier");
const FString UCombinedTransformGizmoContextObject::CustomRepositionableThreeAxisTransformBuilderIdentifier = TEXT("Util_CustomRepositionableThreeAxisTransformBuilderIdentifier");


void UCombinedTransformGizmoContextObject::RegisterGizmosWithManager(UInteractiveToolManager* ToolManager)
{
	if (ensure(!bDefaultGizmosRegistered) == false)
	{
		return;
	}

	UInteractiveGizmoManager* GizmoManager = ToolManager->GetPairedGizmoManager();
	ToolManager->GetContextObjectStore()->AddContextObject(this);

	UAxisPositionGizmoBuilder* AxisTranslationBuilder = NewObject<UAxisPositionGizmoBuilder>();
	GizmoManager->RegisterGizmoType(DefaultAxisPositionBuilderIdentifier, AxisTranslationBuilder);

	UPlanePositionGizmoBuilder* PlaneTranslationBuilder = NewObject<UPlanePositionGizmoBuilder>();
	GizmoManager->RegisterGizmoType(DefaultPlanePositionBuilderIdentifier, PlaneTranslationBuilder);

	UAxisAngleGizmoBuilder* AxisRotationBuilder = NewObject<UAxisAngleGizmoBuilder>();
	GizmoManager->RegisterGizmoType(DefaultAxisAngleBuilderIdentifier, AxisRotationBuilder);

	UCombinedTransformGizmoBuilder* TransformBuilder = NewObject<UCombinedTransformGizmoBuilder>();
	TransformBuilder->AxisPositionBuilderIdentifier = DefaultAxisPositionBuilderIdentifier;
	TransformBuilder->PlanePositionBuilderIdentifier = DefaultPlanePositionBuilderIdentifier;
	TransformBuilder->AxisAngleBuilderIdentifier = DefaultAxisAngleBuilderIdentifier;
	GizmoManager->RegisterGizmoType(DefaultThreeAxisTransformBuilderIdentifier, TransformBuilder);

	UGizmoViewContext* GizmoViewContext = ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	if (!GizmoViewContext)
	{
		GizmoViewContext = NewObject<UGizmoViewContext>();
		ToolManager->GetContextObjectStore()->AddContextObject(GizmoViewContext);
	}

	GizmoActorBuilder = MakeShared<FCombinedTransformGizmoActorFactory>(GizmoViewContext);

	UCombinedTransformGizmoBuilder* CustomThreeAxisBuilder = NewObject<UCombinedTransformGizmoBuilder>();
	CustomThreeAxisBuilder->AxisPositionBuilderIdentifier = DefaultAxisPositionBuilderIdentifier;
	CustomThreeAxisBuilder->PlanePositionBuilderIdentifier = DefaultPlanePositionBuilderIdentifier;
	CustomThreeAxisBuilder->AxisAngleBuilderIdentifier = DefaultAxisAngleBuilderIdentifier;
	CustomThreeAxisBuilder->GizmoActorBuilder = GizmoActorBuilder;
	GizmoManager->RegisterGizmoType(CustomThreeAxisTransformBuilderIdentifier, CustomThreeAxisBuilder);

	URepositionableTransformGizmoBuilder* CustomRepositionableThreeAxisBuilder = NewObject<URepositionableTransformGizmoBuilder>();
	CustomRepositionableThreeAxisBuilder->AxisPositionBuilderIdentifier = DefaultAxisPositionBuilderIdentifier;
	CustomRepositionableThreeAxisBuilder->PlanePositionBuilderIdentifier = DefaultPlanePositionBuilderIdentifier;
	CustomRepositionableThreeAxisBuilder->AxisAngleBuilderIdentifier = DefaultAxisAngleBuilderIdentifier;
	CustomRepositionableThreeAxisBuilder->GizmoActorBuilder = GizmoActorBuilder;
	GizmoManager->RegisterGizmoType(CustomRepositionableThreeAxisTransformBuilderIdentifier, CustomRepositionableThreeAxisBuilder);

	bDefaultGizmosRegistered = true;


}

void UCombinedTransformGizmoContextObject::DeregisterGizmosWithManager(UInteractiveToolManager* ToolManager)
{
	UInteractiveGizmoManager* GizmoManager = ToolManager->GetPairedGizmoManager();
	ToolManager->GetContextObjectStore()->RemoveContextObject(this);

	ensure(bDefaultGizmosRegistered);
	GizmoManager->DeregisterGizmoType(DefaultAxisPositionBuilderIdentifier);
	GizmoManager->DeregisterGizmoType(DefaultPlanePositionBuilderIdentifier);
	GizmoManager->DeregisterGizmoType(DefaultAxisAngleBuilderIdentifier);
	GizmoManager->DeregisterGizmoType(DefaultThreeAxisTransformBuilderIdentifier);
	GizmoManager->DeregisterGizmoType(CustomThreeAxisTransformBuilderIdentifier);
	GizmoManager->DeregisterGizmoType(CustomRepositionableThreeAxisTransformBuilderIdentifier);
	bDefaultGizmosRegistered = false;
}




bool UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UCombinedTransformGizmoContextObject* Found = ToolsContext->ContextObjectStore->FindContext<UCombinedTransformGizmoContextObject>();
		if (Found == nullptr)
		{
			UCombinedTransformGizmoContextObject* GizmoHelper = NewObject<UCombinedTransformGizmoContextObject>(ToolsContext->ToolManager);
			if (ensure(GizmoHelper))
			{
				GizmoHelper->RegisterGizmosWithManager(ToolsContext->ToolManager);
				return true;
			}
			else
			{
				return false;
			}
		}
		return true;
	}
	return false;
}


bool UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UCombinedTransformGizmoContextObject* Found = ToolsContext->ContextObjectStore->FindContext<UCombinedTransformGizmoContextObject>();
		if (Found != nullptr)
		{
			Found->DeregisterGizmosWithManager(ToolsContext->ToolManager);
			ToolsContext->ContextObjectStore->RemoveContextObject(Found);
		}
		return true;
	}
	return false;
}



UCombinedTransformGizmo* UCombinedTransformGizmoContextObject::Create3AxisTransformGizmo(UInteractiveGizmoManager* GizmoManager, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(bDefaultGizmosRegistered))
	{
		UInteractiveGizmo* NewGizmo = GizmoManager->CreateGizmo(DefaultThreeAxisTransformBuilderIdentifier, InstanceIdentifier, Owner);
		UCombinedTransformGizmo* CastGizmo = Cast<UCombinedTransformGizmo>(NewGizmo);
		if (ensure(CastGizmo))
		{
			OnGizmoCreated.Broadcast(CastGizmo);
		}
		return CastGizmo;
	}
	return nullptr;
}
UCombinedTransformGizmo* UE::TransformGizmoUtil::Create3AxisTransformGizmo(UInteractiveGizmoManager* GizmoManager, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(GizmoManager))
	{
		UCombinedTransformGizmoContextObject* UseThis = GizmoManager->GetContextObjectStore()->FindContext<UCombinedTransformGizmoContextObject>();
		if (ensure(UseThis))
		{
			return UseThis->Create3AxisTransformGizmo(GizmoManager, Owner, InstanceIdentifier);
		}
	}
	return nullptr;
}
UCombinedTransformGizmo* UE::TransformGizmoUtil::Create3AxisTransformGizmo(UInteractiveToolManager* ToolManager, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(ToolManager))
	{
		return Create3AxisTransformGizmo(ToolManager->GetPairedGizmoManager(), Owner, InstanceIdentifier);
	}
	return nullptr;
}

UCombinedTransformGizmo* UCombinedTransformGizmoContextObject::CreateCustomTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(bDefaultGizmosRegistered))
	{
		GizmoActorBuilder->EnableElements = Elements;
		UInteractiveGizmo* NewGizmo = GizmoManager->CreateGizmo(CustomThreeAxisTransformBuilderIdentifier, InstanceIdentifier, Owner);
		UCombinedTransformGizmo* CastGizmo = Cast<UCombinedTransformGizmo>(NewGizmo);
		if (ensure(CastGizmo))
		{
			OnGizmoCreated.Broadcast(CastGizmo);
		}
		return CastGizmo;
	}
	return nullptr;
}
UCombinedTransformGizmo* UE::TransformGizmoUtil::CreateCustomTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(GizmoManager))
	{
		UCombinedTransformGizmoContextObject* UseThis = GizmoManager->GetContextObjectStore()->FindContext<UCombinedTransformGizmoContextObject>();
		if (ensure(UseThis))
		{
			return UseThis->CreateCustomTransformGizmo(GizmoManager, Elements, Owner, InstanceIdentifier);
		}
	}
	return nullptr;
}
UCombinedTransformGizmo* UE::TransformGizmoUtil::CreateCustomTransformGizmo(UInteractiveToolManager* ToolManager, ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(ToolManager))
	{
		return CreateCustomTransformGizmo(ToolManager->GetPairedGizmoManager(), Elements, Owner, InstanceIdentifier);
	}
	return nullptr;
}


UCombinedTransformGizmo* UCombinedTransformGizmoContextObject::CreateCustomRepositionableTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(bDefaultGizmosRegistered))
	{
		GizmoActorBuilder->EnableElements = Elements;
		UInteractiveGizmo* NewGizmo = GizmoManager->CreateGizmo(CustomRepositionableThreeAxisTransformBuilderIdentifier, InstanceIdentifier, Owner);
		ensure(NewGizmo);
		UCombinedTransformGizmo* CastGizmo = Cast<UCombinedTransformGizmo>(NewGizmo);
		if (ensure(CastGizmo))
		{
			OnGizmoCreated.Broadcast(CastGizmo);
		}
		return CastGizmo;
	}
	return nullptr;
}
UCombinedTransformGizmo* UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(GizmoManager))
	{
		UCombinedTransformGizmoContextObject* UseThis = GizmoManager->GetContextObjectStore()->FindContext<UCombinedTransformGizmoContextObject>();
		if (ensure(UseThis))
		{
			return UseThis->CreateCustomRepositionableTransformGizmo(GizmoManager, Elements, Owner, InstanceIdentifier);
		}
	}
	return nullptr;
}
UCombinedTransformGizmo* UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(UInteractiveToolManager* ToolManager, ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	if (ensure(ToolManager))
	{
		return CreateCustomRepositionableTransformGizmo(ToolManager->GetPairedGizmoManager(), Elements, Owner, InstanceIdentifier);
	}
	return nullptr;
}

