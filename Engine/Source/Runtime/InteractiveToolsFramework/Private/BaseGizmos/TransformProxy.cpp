// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/TransformProxy.h"
#include "Components/SceneComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformProxy)


#define LOCTEXT_NAMESPACE "UTransformProxy"

DEFINE_LOG_CATEGORY_STATIC(LogInteractiveToolsTransformProxy, Log, All);

void UTransformProxy::AddComponent(USceneComponent* Component, bool bModifyComponentOnTransform)
{
	check(Component);

	FRelativeObject& NewObj = Objects.Emplace_GetRef();
	NewObj.Component = Component;
	NewObj.bModifyComponentOnTransform = bModifyComponentOnTransform;
	NewObj.GetTransformFunc = [Component]() { return Component->GetComponentToWorld(); };
	NewObj.SetTransformFunc = [Component](FTransform NewTransform) 
										 { 
											Component->SetWorldTransform(NewTransform); 
											// mimic behavior in USceneComponent::PostEditChangeProperty that is triggered by updating the scale3 in the transform details panel.
											// otherwise changes in the sign of the scale will result rendering artifacts 
											Component->InvalidateLightingCacheDetailed(true, false); 
										 };
	NewObj.UserDefinedIndex = 0;
	NewObj.StartTransform = NewObj.GetTransformFunc();
	NewObj.RelativeTransform = FTransform::Identity;

	UpdateSharedTransform();
	OnPivotChanged.Broadcast(this, SharedTransform);
}

void UTransformProxy::AddComponentCustom(
	USceneComponent* Component,
	TUniqueFunction<FTransform(void)> GetTransformFunc,
	TUniqueFunction<void(const FTransform&)> SetTransformFunc,
	int64 UserDefinedIndex,
	bool bModifyComponentOnTransform)
{
	check(Component);

	FRelativeObject& NewObj = Objects.Emplace_GetRef();
	NewObj.Component = Component;
	NewObj.bModifyComponentOnTransform = bModifyComponentOnTransform;
	NewObj.GetTransformFunc = MoveTemp(GetTransformFunc);
	NewObj.SetTransformFunc = MoveTemp(SetTransformFunc);
	NewObj.UserDefinedIndex = UserDefinedIndex;
	NewObj.StartTransform = NewObj.GetTransformFunc();
	NewObj.RelativeTransform = FTransform::Identity;

	UpdateSharedTransform();
	OnPivotChanged.Broadcast(this, SharedTransform);
}



FTransform UTransformProxy::GetTransform() const
{
	return SharedTransform;
}

void UTransformProxy::SetTransform(const FTransform& TransformIn)
{
	SharedTransform = TransformIn;

	if (bSetPivotMode)
	{
		UpdateObjectTransforms();
		OnPivotChanged.Broadcast(this, SharedTransform);
	}
	else
	{
		UpdateObjects();
		OnTransformChanged.Broadcast(this, SharedTransform);
	}
}


void UTransformProxy::BeginTransformEditSequence()
{
	OnBeginTransformEdit.Broadcast(this);
}

void UTransformProxy::EndTransformEditSequence()
{
	OnEndTransformEdit.Broadcast(this);
}

void UTransformProxy::BeginPivotEditSequence()
{
	OnBeginPivotEdit.Broadcast(this);
}

void UTransformProxy::EndPivotEditSequence()
{
	OnEndPivotEdit.Broadcast(this);
}



void UTransformProxy::UpdateObjects()
{
	for (FRelativeObject& Obj : Objects)
	{
		FTransform CombinedTransform;
		if (bRotatePerObject && Objects.Num() > 1)
		{
			// We want to apply the compare the shared transform to the shared transform that existed
			// at the time the object's StartTransform was set, then apply the changes to the StartTransform.
			// It may seem that FTransform::RelativeTransform() might be of use here, but that gives the
			// transform from the point of view of the initial frame, which would give us incorrect
			// translation if the initial frame axes didn't line up with world axes.
			CombinedTransform = Obj.StartTransform;

			CombinedTransform.AddToTranslation(SharedTransform.GetTranslation() - InitialSharedTransform.GetTranslation());
			CombinedTransform.ConcatenateRotation(InitialSharedTransform.GetRotation().Inverse());
			CombinedTransform.ConcatenateRotation(SharedTransform.GetRotation());
			CombinedTransform.SetScale3D(CombinedTransform.GetScale3D() * SharedTransform.GetScale3D() / InitialSharedTransform.GetScale3D());
		}
		else
		{
			FTransform::Multiply(&CombinedTransform, &Obj.RelativeTransform, &SharedTransform);
		}
		
		if (Obj.Component.IsValid())
		{
			if (Obj.bModifyComponentOnTransform)
			{
				Obj.Component->Modify();
			}

			Obj.SetTransformFunc(CombinedTransform);
		}
	}
}




void UTransformProxy::UpdateSharedTransform()
{
	if (Objects.Num() == 0)
	{
		SharedTransform = FTransform::Identity;
	}
	else if (Objects.Num() == 1)
	{
		SharedTransform = Objects[0].StartTransform;

		Objects[0].RelativeTransform = FTransform::Identity;
	}
	else
	{
		FVector Origin = FVector::ZeroVector;
		for (const FRelativeObject& Obj : Objects)
		{
			Origin += Obj.StartTransform.GetLocation();
		}
		Origin /= (float)Objects.Num();

		SharedTransform = FTransform(Origin);

		for (FRelativeObject& Obj : Objects)
		{
			Obj.RelativeTransform = Obj.StartTransform;
			Obj.RelativeTransform.SetToRelativeTransform(SharedTransform);
		}
	}

	InitialSharedTransform = SharedTransform;
}



void UTransformProxy::UpdateObjectTransforms()
{
	for (FRelativeObject& Obj : Objects)
	{
		if (Obj.Component != nullptr)
		{
			Obj.StartTransform = Obj.GetTransformFunc();
		}
		Obj.RelativeTransform = Obj.StartTransform;
		Obj.RelativeTransform.SetToRelativeTransform(SharedTransform);
	}

	InitialSharedTransform = SharedTransform;
}





void FTransformProxyChange::Apply(UObject* Object)
{
	UTransformProxy* Proxy = CastChecked<UTransformProxy>(Object);

	bool bSavedSetPivotMode = Proxy->bSetPivotMode;
	Proxy->bSetPivotMode = bSetPivotMode;
	Proxy->SetTransform(To);
	Proxy->OnTransformChangedUndoRedo.Broadcast(Proxy, To);
	Proxy->bSetPivotMode = bSavedSetPivotMode;
}

void FTransformProxyChange::Revert(UObject* Object)
{
	UTransformProxy* Proxy = CastChecked<UTransformProxy>(Object);

	bool bSavedSetPivotMode = Proxy->bSetPivotMode;
	Proxy->bSetPivotMode = bSetPivotMode;
	Proxy->SetTransform(From);
	Proxy->OnTransformChangedUndoRedo.Broadcast(Proxy, From);
	Proxy->bSetPivotMode = bSavedSetPivotMode;
}


void FTransformProxyChangeSource::BeginChange()
{
	if (Proxy.IsValid())
	{
		ensureMsgf(!ActiveChange, TEXT("FTransformProxyChangeSource: BeginChange called without finishing previous change via EndChange."));

		ActiveChange = MakeUnique<FTransformProxyChange>();
		ActiveChange->From = Proxy->GetTransform();
		ActiveChange->bSetPivotMode = bOverrideSetPivotMode ? true : Proxy->bSetPivotMode;

		if (ActiveChange->bSetPivotMode)
		{
			Proxy->BeginPivotEditSequence();
		}
		else
		{
			Proxy->BeginTransformEditSequence();
		}
	}
}

TUniquePtr<FToolCommandChange> FTransformProxyChangeSource::EndChange()
{
	if (Proxy.IsValid())
	{
		if (!ensureMsgf(ActiveChange, TEXT("FTransformProxyChangeSource: EndChange called without a pending change.")))
		{
			// One way this error could occur is if there is (incorrectly) one Begin/EndChange pair nested inside another one,
			// so the previous EndChange call closes out the pending transaction.
			return TUniquePtr<FToolCommandChange>();
		}

		if (ActiveChange->bSetPivotMode)
		{
			Proxy->EndPivotEditSequence();
		}
		else
		{
			Proxy->EndTransformEditSequence();
		}

		if (!ActiveChange)
		{
			// One way this error could occur is if the Proxy call forced a re-entrant call to EndChange() by destroying
			// the gizmo which in turn will ForceTerminateSource() and end an ActiveChange.
			UE_LOGF(LogInteractiveToolsTransformProxy, Error, "FTransformProxyChangeSource: Re-entrant EndChange invalidated ActiveChange. Did you delete a gizmo on EndPivotEditSequence/EndTransformEditSequence?");
			return TUniquePtr<FToolCommandChange>();
		}
		ActiveChange->To = Proxy->GetTransform();
		return MoveTemp(ActiveChange);
	}
	return TUniquePtr<FToolCommandChange>();
}

void FTransformProxyChangeSource::CancelChange()
{
	if (Proxy.IsValid() && ensureMsgf(ActiveChange, TEXT("FTransformProxyChangeSource: CancelChange called without a pending change.")))
	{
		if (ActiveChange->bSetPivotMode)
		{
			Proxy->EndPivotEditSequence();
		}
		else
		{
			Proxy->EndTransformEditSequence();
		}
		
		ActiveChange.Reset();
	}
}

UObject* FTransformProxyChangeSource::GetChangeTarget()
{
	return Proxy.Get();
}

FText FTransformProxyChangeSource::GetChangeDescription()
{
	return LOCTEXT("FTransformProxyChangeDescription", "TransformProxyChange");
}


#undef LOCTEXT_NAMESPACE

