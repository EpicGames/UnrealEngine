// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/TransformSources.h"
#include "Components/SceneComponent.h" 

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformSources)



FTransform UGizmoComponentWorldTransformSource::GetTransform() const
{
	return Component->GetComponentToWorld();
}

void UGizmoComponentWorldTransformSource::SetTransform(const FTransform& NewTransform)
{
	if (bModifyComponentOnTransform)
	{
		Component->Modify();
	}
	Component->SetWorldTransform(NewTransform);
	OnTransformChanged.Broadcast(this);
}


FTransform UGizmoScaledTransformSource::GetTransform() const
{
	// get transform from child, and replace scale with external scale
	FTransform Transform = ChildTransformSource->GetTransform();
	FVector ExternalScale = ScaleProvider.GetScale();
	Transform.SetScale3D(ExternalScale);
	return Transform;
}

void UGizmoScaledTransformSource::SetTransform(const FTransform& NewTransform)
{
	// forward incoming scale to external provider
	FVector ExternalScale = NewTransform.GetScale3D();
	ScaleProvider.SetScale(ExternalScale);

	// remove scale from transform and forward to child source
	FTransform Unscaled(NewTransform);
	Unscaled.SetScale3D(FVector::OneVector);
	ChildTransformSource->SetTransform(Unscaled);

	OnTransformChanged.Broadcast(this);
}




FTransform UGizmoTransformProxyTransformSource::GetTransform() const
{
	return Proxy->GetTransform();
}

void UGizmoTransformProxyTransformSource::SetTransform(const FTransform& NewTransform)
{
	if (bOverrideSetPivotMode)
	{
		bool bProxySetPivotOriginal = Proxy->bSetPivotMode;
		Proxy->bSetPivotMode = true;
		Proxy->SetTransform(NewTransform);
		Proxy->bSetPivotMode = bProxySetPivotOriginal;

		OnPivotChanged.Broadcast(this);
	}
	else
	{
		Proxy->SetTransform(NewTransform);
		if (Proxy->bSetPivotMode)
		{
			OnPivotChanged.Broadcast(this);
		}
		else
		{
			OnTransformChanged.Broadcast(this);
		}
	}
}

FTransform UGizmoScaledAndUnscaledTransformSources::GetTransform() const
{
	return ScaledTransformSource->GetTransform();
}

void UGizmoScaledAndUnscaledTransformSources::SetTransform(const FTransform& NewTransform)
{
	if (UnscaledTransformSource)
	{
		FTransform Unscaled(NewTransform);
		// For the unscaled source, we want the scale to remain unaffected, i.e. same as it currently is. Note 
		//  that we don't just blindly set the scale to 1 because the unscaled source is typically a gizmo component,
		//  and we don't want to mess up its scale if it was initialized to a particular value for visual effect.
		Unscaled.SetScale3D(UnscaledTransformSource->GetTransform().GetScale3D());
		UnscaledTransformSource->SetTransform(Unscaled);
	}

	ScaledTransformSource->SetTransform(NewTransform);

	OnTransformChanged.Broadcast(this);
}
