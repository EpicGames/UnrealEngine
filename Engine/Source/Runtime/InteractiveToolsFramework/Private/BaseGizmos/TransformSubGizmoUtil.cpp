// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/TransformSubGizmoUtil.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoPrivateUtil.h" // ToAxis, GetGizmoViewContext
#include "BaseGizmos/ViewBasedTransformAdjusters.h"
#include "BaseGizmos/ViewAdjustedStaticMeshGizmoComponent.h"
#include "Components/SceneComponent.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/EnumRange.h" // MakeFlagsRange


FTransform UE::GizmoUtil::GetRotatedBasisTransform(const FTransform& TransformIn, EAxis::Type AxisToBeX)
{
	FTransform Result = TransformIn;
	if (AxisToBeX == EAxis::Y)
	{
		FQuat RotationQuat(FMatrix(FVector::YAxisVector, FVector::ZAxisVector, FVector::XAxisVector, FVector::ZeroVector));
		Result = TransformIn * RotationQuat;
		
	}
	else if (AxisToBeX == EAxis::Z)
	{
		FQuat RotationQuat(FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, FVector::ZeroVector));
		Result = TransformIn * RotationQuat;
	}

	if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		Result.Mirror(EAxis::Y, EAxis::None);
	}

	return Result;
}

void UE::GizmoUtil::InitializeSubGizmoElementsWithMesh(const FInitMeshSubGizmoParams& Params,
	TArray<TPair<ETransformGizmoSubElements, UViewAdjustedStaticMeshGizmoComponent*>>* ComponentsOut)
{
	if (!ensure(Params.ParentGizmo && Params.Mesh))
	{
		return;
	}

	ACombinedTransformGizmoActor* GizmoActor = Params.ParentGizmo->GetGizmoActor();
	UGizmoViewContext* GizmoViewContext = GetGizmoViewContext(Params.ParentGizmo->GetGizmoManager());

	if (!ensure(GizmoActor && GizmoViewContext))
	{
		return;
	}

	// Iterate over each element individually
	for (ETransformGizmoSubElements Element : MakeFlagsRange(Params.Elements))
	{
		EAxis::Type ElementAxis = ToAxis(Element);
		FLinearColor Color = ElementAxis == EAxis::None ? FLinearColor::Black 
			: UE::GizmoRenderingUtil::GetDefaultAxisColor(ElementAxis);

		UViewAdjustedStaticMeshGizmoComponent* Component = UE::GizmoRenderingUtil::CreateDefaultMaterialGizmoMeshComponent(
			Params.Mesh, GizmoViewContext, GizmoActor, Color);

		// This makes it so that the component takes the place of the proper element
		Params.ParentGizmo->SetSubGizmoComponent(Element, Component, FTransform::Identity);

		// This has to happen after the SetSubGizmoComponent call above, so that it is already attached
		Component->SetRelativeTransform(
			Params.bRotateTransformBasisBasedOnElement ? GetRotatedBasisTransform(Params.ComponentToGizmo, ElementAxis)
			: Params.ComponentToGizmo);

		UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::AddTransformAdjuster(
			Component, GizmoActor->GetRootComponent(), Params.bMirrorBasedOnOctant);

		if (Params.SubstituteInteractionMesh)
		{
			UViewAdjustedStaticMeshGizmoComponent* SubstituteComponent = nullptr;
			
			SubstituteComponent = UE::GizmoRenderingUtil::CreateDefaultMaterialGizmoMeshComponent(
				Params.SubstituteInteractionMesh, GizmoViewContext, Component, Color, 
				// No need for hover material
				false);

			Component->SetSubstituteInteractionComponent(SubstituteComponent, Params.SubstituteMeshToComponent);

			SubstituteComponent->SetRelativeTransform(Params.SubstituteMeshToComponent);
			UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::AddTransformAdjuster(
				SubstituteComponent, GizmoActor->GetRootComponent(), Params.bMirrorBasedOnOctant);
		}//end substitute interaction mesh setup

		if (ComponentsOut)
		{
			ComponentsOut->Emplace(Element, Component);
		}
	}//end for each element
}