// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/ViewAdjustedStaticMeshGizmoComponent.h"

#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/ViewBasedTransformAdjusters.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "MaterialShared.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "StaticMeshSceneProxyDesc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewAdjustedStaticMeshGizmoComponent)

namespace ViewAdjustedStaticMeshGizmoComponentLocals
{
	// Overriding a function here is necessary to be able to properly update the used materials to include
	//  the hover material.
	class FViewAdjustedStaticMeshSceneProxyDesc : public FStaticMeshSceneProxyDesc
	{
	public:
		FViewAdjustedStaticMeshSceneProxyDesc(const UViewAdjustedStaticMeshGizmoComponent* Component)
			: FStaticMeshSceneProxyDesc(Component)
		{
		}

		virtual void GetUsedMaterials(TArray<UMaterialInterface*>&OutMaterials, bool bGetDebugMaterials = false) const override
		{
			FStaticMeshSceneProxyDesc::GetUsedMaterials(OutMaterials);
			if (UViewAdjustedStaticMeshGizmoComponent* CastComponent = Cast<UViewAdjustedStaticMeshGizmoComponent>(Component))
			{
				if (UMaterialInterface* HoverMaterial = CastComponent->GetHoverOverrideMaterial())
				{
					OutMaterials.Add(HoverMaterial);
				}
			}
		}
	};

	class FViewAdjustedStaticMeshGizmoComponentProxy : public FStaticMeshSceneProxy
	{
	public:
		FViewAdjustedStaticMeshGizmoComponentProxy(UViewAdjustedStaticMeshGizmoComponent* Component)
			: FStaticMeshSceneProxy(FViewAdjustedStaticMeshSceneProxyDesc(Component), false)
			, TransformAdjuster(Component->GetTransformAdjuster())
			, HoverOverrideMaterial(Component->GetHoverOverrideMaterial())
			, bHovered(Component->IsBeingHovered())
			, bHiddenByInteraction(Component->IsHiddenByInteraction())
			, RenderVisibilityFunc(Component->GetRenderVisibilityFunction())
		{
		}

		void SetIsHovered(bool bHoveredIn)
		{
			bHovered = bHoveredIn;
		}

		void SetIsHiddenByInteraction(bool bIsHidden)
		{
			bHiddenByInteraction = bIsHidden;
		}

		void SetTransformAdjuster(TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> TransformAdjusterIn)
		{
			TransformAdjuster = TransformAdjusterIn;
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views,
			const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			if (bHiddenByInteraction)
			{
				return;
			}

			// For the most part, the below is modeled off the FStaticMeshSceneProxy version of this method, with 
			//  various things cut out (debug view modes, etc) and some deep nesting turned out into early outs.
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				const FSceneView* View = Views[ViewIndex];

				if (!(IsShown(View) && (VisibilityMap & (1 << ViewIndex))))
				{
					continue;
				}

				// We can calculate our adjusted transform at this point, now that we have the view
				const TUniformBuffer<FPrimitiveUniformShaderParameters>* AdjustedTransformBuffer = nullptr;
				bool bAdjustedTransformDeterminantIsNegative = IsLocalToWorldDeterminantNegative();
				if (TransformAdjuster.IsValid())
				{
					UE::GizmoRenderingUtil::FSceneViewWrapper WrappedView(*View);

					FTransform NewTransform = TransformAdjuster->GetAdjustedComponentToWorld_RenderThread(
						WrappedView, FTransform(GetLocalToWorld()));

					if (RenderVisibilityFunc && !RenderVisibilityFunc(WrappedView, NewTransform))
					{
						continue;
					}

					FMatrix NewTransformMatrix = NewTransform.ToMatrixWithScale();
					bAdjustedTransformDeterminantIsNegative = (NewTransformMatrix.Determinant() < 0);

					const FBoxSphereBounds3d& OriginalLocalBounds = GetLocalBounds();

					// This way of setting the transform is copied from FTriangleSetSceneProxy::GetDynamicMeshElements
					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(),
						NewTransformMatrix, NewTransformMatrix,
						GetLocalBounds().TransformBy(NewTransformMatrix), GetLocalBounds(),
						/*bReceivesDecals*/ true, /*bHasPrecomputedVolumetricLightmap*/ false,
						AlwaysHasVelocity());

					AdjustedTransformBuffer = &DynamicPrimitiveUniformBuffer.UniformBuffer;
				}//end getting adjusted transform ready
				else
				{
					// If we're not adjusting the transform, just check visibility with the original transform
					if (RenderVisibilityFunc && !RenderVisibilityFunc(UE::GizmoRenderingUtil::FSceneViewWrapper(*View), FTransform(GetLocalToWorld())))
					{
						continue;
					}
				}

				FLODMask LODMask = GetLODMask(View);

				for (int32 LODIndex = 0; LODIndex < RenderData->LODResources.Num(); LODIndex++)
				{
					if (!(LODMask.ContainsLOD(LODIndex) && LODIndex >= ClampedMinLOD))
					{
						continue;
					}

					const FStaticMeshLODResources& LODModel = RenderData->LODResources[LODIndex];

					for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
					{
						const int32 NumBatches = GetNumMeshBatches();

						for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
						{
							FMeshBatch& MeshBatch = Collector.AllocateMesh();

							const FLODInfo::FSectionInfo& Section = LODs[LODIndex].Sections[SectionIndex];

							// This selection and hit proxy id setting seems unneeded but we'll just keep it
							bool bSectionIsSelected = false;
#if WITH_EDITOR
							if (GIsEditor)
							{
								bSectionIsSelected = Section.bSelected;
								MeshBatch.BatchHitProxyId = Section.HitProxy ? Section.HitProxy->Id : FHitProxyId();
							}
#endif

							if (!GetMeshElement(LODIndex, BatchIndex, SectionIndex,
								GetStaticDepthPriorityGroup(), bSectionIsSelected,
								// Not sure what this bAllowPreCulledIndices parameter is, but this is what FStaticMeshSceneProxy does
								true,
								MeshBatch))
							{
								continue;
							}

							// Seems like there is only ever one of these...
							FMeshBatchElement& BatchElement = MeshBatch.Elements[0];

							// The above GetMeshElement does not reflect the adjusted transform determinant, so we have to
							//  redo those pieces of MeshBatch setup.
							
							// Updated output of FStaticMeshSceneProxy::ShouldRenderBackFaces() 
							bool bShouldRenderBackFaces = bReverseCulling != bAdjustedTransformDeterminantIsNegative;

							// Updated value for bUseReversedIndices
							UMaterialInterface* MaterialInterface = Section.Material;
							const FMaterialRenderProxy* MaterialRenderProxy = MaterialInterface->GetRenderProxy();
							const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
							const FMaterial& Material = MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
							const bool bUseReversedIndices = bShouldRenderBackFaces
								&& RenderData->LODResources[LODIndex].bHasReversedDepthOnlyIndices
								&& !Material.IsTwoSided();

							// This is originally done in FStaticMeshSceneProxy::SetMeshElementGeometrySource
							BatchElement.IndexBuffer = bUseReversedIndices 
								? &LODModel.AdditionalIndexBuffers->ReversedIndexBuffer 
								: &LODModel.IndexBuffer;

							// Updated output of FStaticMeshSceneProxy::IsReversedCullingNeeded
							MeshBatch.ReverseCulling = bShouldRenderBackFaces && !bUseReversedIndices;
							// Done updating based on transform determinant


							// Gizmos probably don't want to be affected by view mode?
							MeshBatch.bCanApplyViewModeOverrides = false;

							// This is where we bind our adjusted transform:
							if (AdjustedTransformBuffer)
							{
								BatchElement.PrimitiveUniformBufferResource = AdjustedTransformBuffer;
							}
							
							// Apply hover override material
							if (bHovered && HoverOverrideMaterial)
							{
								MeshBatch.MaterialRenderProxy = HoverOverrideMaterial->GetRenderProxy();
							}

							Collector.AddMesh(ViewIndex, MeshBatch);
							INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, MeshBatch.GetNumPrimitives());

						}// for each MeshBatch
					}// for each mesh section
				}// for each LOD
			}// for each view
		}//GetDynamicMeshElements()

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result = FStaticMeshSceneProxy::GetViewRelevance(View);
			Result.bDynamicRelevance = true;
			Result.bStaticRelevance = false;
			Result.bShadowRelevance = false;
			
			MaterialRelevance.SetPrimitiveViewRelevance(Result);

			return Result;
		}

		virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override {}

		virtual bool CanBeOccluded() const override 
		{
			// If we're using a transform adjuster, we're going to say that we can't be occluded because in the common case
			//  of keeping the component a constant view size, it can be arbitrarily large as we move away from it. This prevents
			//  the component from occluding itself too.
			// This isn't actually necessary if we're using a non-depth-tested material.
			return !TransformAdjuster.IsValid() && FStaticMeshSceneProxy::CanBeOccluded();
		};

		private:

		TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> TransformAdjuster;
		UMaterialInterface* HoverOverrideMaterial = nullptr;
		bool bHovered = false;
		// It is tempting to use the visibility of the component to hide it during interaction, but that turns out to be
		//  problematic because other things affect the visibility- for example the TRS gizmo constantly updates visibility
		//  depending on the current gizmo mode (translate/rotate/scale). Instead, we want this setting to be an extra flag
		//  that forces invisibility. 
		bool bHiddenByInteraction = false;

		TFunction<bool(const UE::GizmoRenderingUtil::ISceneViewInterface& View,
			const FTransform& ComponentToWorld)> RenderVisibilityFunc = nullptr;
	};

}


bool UViewAdjustedStaticMeshGizmoComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	if (!ensure(GizmoViewContext))
	{
		return Super::LineTraceComponent(OutHit, Start, End, Params);
	}

	// If needed, update the physics data, then do the line trace
	if (TransformAdjuster.IsValid())
	{
		FTransform OriginalComponentToWorld = GetComponentToWorld();
		FTransform AdjustedComponentToWorld = TransformAdjuster->GetAdjustedComponentToWorld(*GizmoViewContext, OriginalComponentToWorld);

		if (RenderVisibilityFunc && !RenderVisibilityFunc(*GizmoViewContext, AdjustedComponentToWorld))
		{
			return false;
		}

		if (!OriginalComponentToWorld.Equals(AdjustedComponentToWorld))
		{
			BodyInstance.SetBodyTransform(AdjustedComponentToWorld, ETeleportType::None);
			BodyInstance.UpdateBodyScale(AdjustedComponentToWorld.GetScale3D());
		}
	}
	else
	{
		if (RenderVisibilityFunc && !RenderVisibilityFunc(*GizmoViewContext, GetComponentToWorld()))
		{
			return false;
		}
	}

	return Super::LineTraceComponent(OutHit, Start, End, Params);
}

FBoxSphereBounds UViewAdjustedStaticMeshGizmoComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds OriginalBounds = Super::CalcBounds(LocalToWorld);
	if (!TransformAdjuster.IsValid())
	{
		return OriginalBounds;
	}
	return TransformAdjuster->GetViewIndependentBounds(LocalToWorld, OriginalBounds);
}

FPrimitiveSceneProxy* UViewAdjustedStaticMeshGizmoComponent::CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	return ::new ViewAdjustedStaticMeshGizmoComponentLocals::FViewAdjustedStaticMeshGizmoComponentProxy(this);
}

// Deprecated in 5.7
FMaterialRelevance UViewAdjustedStaticMeshGizmoComponent::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UViewAdjustedStaticMeshGizmoComponent::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	FMaterialRelevance Result = Super::GetMaterialRelevance(InShaderPlatform);
	if (HoverOverrideMaterial)
	{
		Result |= HoverOverrideMaterial->GetRelevance_Concurrent(InShaderPlatform);
	}
	return Result;
}

void UViewAdjustedStaticMeshGizmoComponent::SetTransformAdjuster(TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> Adjuster)
{
	TransformAdjuster = Adjuster;
	MarkRenderStateDirty();
}

void UViewAdjustedStaticMeshGizmoComponent::SetRenderVisibilityFunction(TFunction<bool(
	const UE::GizmoRenderingUtil::ISceneViewInterface& View, const FTransform& ComponentToWorld)> RenderVisibilityFuncIn)
{
	RenderVisibilityFunc = RenderVisibilityFuncIn;
	MarkRenderStateDirty();
}

void UViewAdjustedStaticMeshGizmoComponent::SetAllMaterials(UMaterialInterface* Material)
{
	for (int i = 0; i < GetNumMaterials(); ++i)
	{
		SetMaterial(i, Material);
	}
}

void UViewAdjustedStaticMeshGizmoComponent::SetHoverOverrideMaterial(UMaterialInterface* Material)
{
	HoverOverrideMaterial = Material;
	MarkRenderStateDirty();
}

void UViewAdjustedStaticMeshGizmoComponent::SetSubstituteInteractionComponent(UPrimitiveComponent* NewComponent,
	const FTransform& RelativeTransform)
{
	if (SubstituteInteractionComponent == NewComponent)
	{
		return;
	}

	if (SubstituteInteractionComponent)
	{
		if (SubstituteInteractionComponent->IsRegistered())
		{
			SubstituteInteractionComponent->UnregisterComponent();
		}
		SubstituteInteractionComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
	}

	SubstituteInteractionComponent = NewComponent;

	if (!NewComponent)
	{
		// If we're clearing the substitute component, we're done
		return;
	}

	if (NewComponent->IsRegistered())
	{
		NewComponent->UnregisterComponent();
	}
	NewComponent->AttachToComponent(this, FAttachmentTransformRules::KeepWorldTransform);
	NewComponent->SetRelativeTransform(RelativeTransform);
	NewComponent->RegisterComponent();

	UpdateInteractingState(false);
}

void UViewAdjustedStaticMeshGizmoComponent::UpdateInteractingState(bool bInteracting)
{
	// Note: we don't early out if we're already interacting because we want to be able to call
	// this function to update the scene proxy.
	bInteracted = bInteracting;

	if (SubstituteInteractionComponent)
	{
		SubstituteInteractionComponent->SetVisibility(bInteracting);
	}

	if (FPrimitiveSceneProxy* Proxy = GetSceneProxy())
	{
		static_cast<ViewAdjustedStaticMeshGizmoComponentLocals::FViewAdjustedStaticMeshGizmoComponentProxy*>(Proxy)
			->SetIsHiddenByInteraction(bInteracted && SubstituteInteractionComponent);
	}
}

void UViewAdjustedStaticMeshGizmoComponent::UpdateHoverState(bool bHoveringIn)
{
	if (bHoveringIn != bHovered)
	{
		bHovered = bHoveringIn;
		if (FPrimitiveSceneProxy* Proxy = GetSceneProxy())
		{
			static_cast<ViewAdjustedStaticMeshGizmoComponentLocals::FViewAdjustedStaticMeshGizmoComponentProxy*>(Proxy)
				->SetIsHovered(bHovered);
		}
	}
}

void UViewAdjustedStaticMeshGizmoComponent::UpdateWorldLocalState(bool bWorldIn)
{
	if (TransformAdjuster.IsValid())
	{
		TransformAdjuster->UpdateWorldLocalState(bWorldIn);
	}
	// If able to, forward this information to the substitute component
	// Tempting to only do this if we're interacting, but what if we get the update just once, and
	//  never update the substitute...
	if (IGizmoBaseComponentInterface* CastSubstituteComp = Cast<IGizmoBaseComponentInterface>(SubstituteInteractionComponent))
	{
		CastSubstituteComp->UpdateWorldLocalState(bWorldIn);
	}
}
