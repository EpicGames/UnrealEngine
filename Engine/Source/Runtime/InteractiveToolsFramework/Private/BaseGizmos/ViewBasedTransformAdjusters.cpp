// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/ViewBasedTransformAdjusters.h"

#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/ViewAdjustedStaticMeshGizmoComponent.h"
#include "FrameTypes.h"
#include "Misc/AxisDisplayInfo.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHICommandList.h"

namespace ViewBasedTransformAdjustersLocals
{
	FBoxSphereBounds InfiniteBounds(FSphere(FVector::ZeroVector, HALF_WORLD_MAX));

	double GetComponentToGizmoScaling(
		const UE::GizmoRenderingUtil::ISceneViewInterface& View,
		const FTransform& GizmoToWorld)
	{
		// It might seem that all we want is to calculate local scale at CurrentComponentToWorld.GetLocation().
		//  However we're in a weird situation where scaling gets applied around the gizmo origin, not the component (to
		//  preserve component positioning within the gizmo); moreover we want different sub components of the same gizmo
		//  to be scaled the same way. So, we need to use owning gizmo location as the basis of our scaling.
		// On the other hand, we can't directly calculate local scale at gizmo origin either because it's possible to see
		//  the sub gizmo while the gizmo itself is off screen, breaking that calculation. Moreover, we would like our 
		//  scaling to stay the same as we move the gizmo in the view plane. Still, we want our scaling to account for FOV.
		// The way we do this is we take the distance to gizmo origin, project that onto the view direction, and evaluate
		//  the local scale to use there. So, the same scale is applied to all sub components of the same gizmo, this
		//  scale is not affected by translations in the view plane, and the gizmo origin can be off screen.
		// Note that this doesn't fully fix all problems with large component-to-gizmo offsets. Namely, moving the component
		//  with the gizmo off screen can sometimes make it seem like the component stops moving and starts growing, which 
		//  looks unnatural if you're not looking at the gizmo and realizing that you are just modifying your angle relative
		//  to it. However the approach here tends to be the least broken overall.

		double ProjectedDistanceToGizmo = (GizmoToWorld.GetLocation() - View.GetViewLocation()).Dot(View.GetViewDirection());
		FVector3d PointAtWhichToCheckScale = View.GetViewLocation() + ProjectedDistanceToGizmo * View.GetViewDirection();
		
		// CalculateLocalPixelToWorldScale is inconsistent in ortho viewports when zoomed very far out, giving a scale
		//  that seems to change at the same zoom level. Instead we use the same approach that editor gizmos use for getting
		//  local scale.
		FVector Unused;
		return UE::GizmoRenderingUtil::CalculateViewDependentScaleAndFlatten(&View, PointAtWhichToCheckScale, 
			// 0.5 scale here happens to get us close to what CalculateLocalPixelToWorldScale would return
			//  at that same position.
			0.5, 
			Unused);
	}

	// Apply the settings to get a transform modified by view
	FTransform GetSubGizmoAdjustedTransform(
		const UE::GizmoRenderingUtil::ISceneViewInterface& View,
		const FTransform& CurrentComponentToWorld,
		const FTransform& GizmoOriginToComponent,
		const UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::FSettings& Settings, 
		bool bAllowGizmoOriginTranslation)
	{
		bool bIsOrtho = !View.IsPerspectiveProjection();
		bool bApplyOrthoGizmoTranslation = bIsOrtho && bAllowGizmoOriginTranslation && Settings.DistanceInFrontOfCameraInOrtho >= 0;

		if (!Settings.bKeepConstantViewSize && !Settings.bMirrorBasedOnOctant && !Settings.bUseWorldAxesForGizmo 
			&& !bApplyOrthoGizmoTranslation)
		{
			return CurrentComponentToWorld;
		}
		
		FTransform GizmoToWorld = GizmoOriginToComponent * CurrentComponentToWorld;
		FTransform ComponentToGizmo = CurrentComponentToWorld.GetRelativeTransform(GizmoToWorld);

		if (bApplyOrthoGizmoTranslation)
		{
			UE::Geometry::FFrame3d CameraFrame(View.GetViewLocation(), View.GetViewDirection(), View.GetViewRight(), View.GetViewUp());
			FVector CameraSpaceLocation = CameraFrame.ToFramePoint(GizmoToWorld.GetLocation());
			if (CameraSpaceLocation.X < Settings.DistanceInFrontOfCameraInOrtho)
			{
				CameraSpaceLocation.X = Settings.DistanceInFrontOfCameraInOrtho;
					GizmoToWorld.SetLocation(CameraFrame.FromFramePoint(CameraSpaceLocation));
			}
		}

		if (Settings.bUseWorldAxesForGizmo)
		{
			GizmoToWorld.SetRotation(FQuat::Identity);
		}

		if (Settings.bMirrorBasedOnOctant)
		{
			// Coordinate axis are mirrored based on view octant to allow for easier access to all gizmo controls
			FVector3d GizmoSpaceDirectionTowardCamera;
			if (View.IsPerspectiveProjection())
			{
				GizmoSpaceDirectionTowardCamera = GizmoToWorld.InverseTransformPosition(View.GetViewLocation());
			}
			else
			{
				GizmoSpaceDirectionTowardCamera = GizmoToWorld.InverseTransformVector(View.GetViewDirection());
			}

			EAxis::Type Axes[3] = { EAxis::X, EAxis::Y, EAxis::Z };
			for (int Dim = 0; Dim < 3; ++Dim)
			{
				if (GizmoSpaceDirectionTowardCamera[Dim] < -KINDA_SMALL_NUMBER)
				{
					ComponentToGizmo.Mirror(Axes[Dim], EAxis::None);
				}
			}
			if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
			{
				// LeftUpForward coordinate system display mirrors Unreal's engine Y so that axis is pointed to the left
				// This effectively undoes the mirroring that occurs in GetRotatedBaseTransform
				ComponentToGizmo.Mirror(EAxis::Y, EAxis::None);
			}
		}

		if (Settings.bKeepConstantViewSize)
		{
			double ExtraScaling = GetComponentToGizmoScaling(View, GizmoToWorld);

			ComponentToGizmo.MultiplyScale3D(FVector3d(ExtraScaling));
			ComponentToGizmo.ScaleTranslation(ExtraScaling);
		}

		return ComponentToGizmo * GizmoToWorld;
	}
}

// FSimpleConstantViewScaleAdjuster:

FTransform UE::GizmoRenderingUtil::FSimpleConstantViewScaleAdjuster::GetAdjustedComponentToWorld(
	const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld)
{
	double ExtraScaling = UE::GizmoRenderingUtil::CalculateLocalPixelToWorldScale(&View, CurrentComponentToWorld.GetLocation());
	FTransform AdjustedTransform = CurrentComponentToWorld;
	AdjustedTransform.MultiplyScale3D(FVector(ExtraScaling));

	return AdjustedTransform;
}

FBoxSphereBounds UE::GizmoRenderingUtil::FSimpleConstantViewScaleAdjuster::GetViewIndependentBounds(
	const FTransform& LocalToWorld, const FBoxSphereBounds& OriginalBounds)
{
	return ViewBasedTransformAdjustersLocals::InfiniteBounds;
}


// FSubGizmoTransformAdjuster:

void UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::SetGizmoOriginTransform(
	const FTransform& GizmoOriginToComponentIn)
{
	GizmoOriginToComponent_GameThread = GizmoOriginToComponentIn;

	// Safely update the render thread side transform.
	ENQUEUE_RENDER_COMMAND(FViewpointOctantMirrorTransformAdjusterUpdate)(
		[GizmoOriginToComponentIn, WeakThis = AsWeak()](FRHICommandListImmediate& RHICmdList)
	{
		TSharedPtr<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster> ThisPinned = WeakThis.Pin();
		if (ThisPinned.IsValid())
		{
			ThisPinned->GizmoOriginToComponent_RenderThread = GizmoOriginToComponentIn;
		}
	});
}

TSharedPtr<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster> UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::AddTransformAdjuster(
	UViewAdjustedStaticMeshGizmoComponent* ComponentIn, USceneComponent* GizmoRootComponent, bool bMirrorBasedOnOctant)
{
	if (!ComponentIn)
	{
		return nullptr;
	}

	TSharedPtr<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster> TransformAdjuster =
		MakeShared<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster>();
	UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::FSettings Settings;
	Settings.bKeepConstantViewSize = true;
	Settings.bMirrorBasedOnOctant = bMirrorBasedOnOctant;
	// DistanceInFrontOfCameraInOrtho is used to keep the gizmo from being culled by the near or far plane in the ortho
	//  viewports, by keeping it at a fixed distance in front of the camera. 1000 is an arbitrary value that's fine as long
	//  as the gizmo is not super thick.
	Settings.DistanceInFrontOfCameraInOrtho = 1000;
	// bUseWorldAxes gets updated automatically as part of UViewAdjustedStaticMeshGizmoComponent::UpdateWorldLocalState
	TransformAdjuster->SetSettings(Settings);
	if (GizmoRootComponent)
	{
		TransformAdjuster->SetGizmoOriginTransform(
			GizmoRootComponent->GetComponentTransform().GetRelativeTransform(ComponentIn->GetComponentToWorld()));
	}
	ComponentIn->SetTransformAdjuster(TransformAdjuster);

	return TransformAdjuster;
}

FTransform UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::GetAdjustedComponentToWorld(
	const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld)
{
	// For rendering, we allow the gizmo origin to move towards the camera to avoid clipping in ortho. However in
	//  line traces on the game thread, which are used in calculations in a gizmo, we typically need the hit location
	//  to be relative to an unchanging gizmo origin, because that is the origin that will be used by other calculations.
	// This is particularly important for axis rotation gizmos where the initial hit location (projected to the axis
	//  passing through gizmo origin) determines the plane in which we rotate.
	const bool bAllowGizmoOriginTranslation = false;

	return ViewBasedTransformAdjustersLocals::GetSubGizmoAdjustedTransform(View, CurrentComponentToWorld,
		GizmoOriginToComponent_GameThread, Settings, bAllowGizmoOriginTranslation);
}

FTransform UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::GetAdjustedComponentToWorld_RenderThread(
	const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld)
{
	// See comment in GetAdjustedComponentToWorld
	const bool bAllowGizmoOriginTranslation = true;

	return ViewBasedTransformAdjustersLocals::GetSubGizmoAdjustedTransform(View, CurrentComponentToWorld,
		GizmoOriginToComponent_RenderThread, Settings, bAllowGizmoOriginTranslation);
}

FBoxSphereBounds UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster::GetViewIndependentBounds(
	const FTransform& LocalToWorld, const FBoxSphereBounds& OriginalBounds)
{
	if (!Settings.bKeepConstantViewSize && !Settings.bMirrorBasedOnOctant && !Settings.bUseWorldAxesForGizmo)
	{
		return OriginalBounds;
	}
	if (Settings.bKeepConstantViewSize)
	{
		return ViewBasedTransformAdjustersLocals::InfiniteBounds;
	}

	// We could probably figure out how to make sure we only include the appropriate mirrored portions in our
	//  bounds, but we'll just make it so that our bounds include any orientation relative to the gizmo origin.
	FVector GizmoWorldLocation = LocalToWorld.TransformPosition(GizmoOriginToComponent_GameThread.GetLocation());
	double DistanceToGizmoOriginWorld = (GizmoWorldLocation - OriginalBounds.Origin).Size();
	double TotalRadius = DistanceToGizmoOriginWorld + FMath::Max(OriginalBounds.BoxExtent.Size(), OriginalBounds.SphereRadius);
	
	return FBoxSphereBounds(FSphere(OriginalBounds.Origin, TotalRadius));
}


// FConstantViewRelativeTransformAdjuster:

FTransform UE::GizmoRenderingUtil::FConstantViewRelativeTransformAdjuster::GetAdjustedComponentToWorld(
	const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld)
{
	UE::Geometry::FFrame3d ViewFrameAtComponent(CurrentComponentToWorld.GetLocation(), View.GetViewDirection(), View.GetViewRight(), View.GetViewUp());
	FTransform ViewRelativeTransformToUse = ViewRelativeTransform;

	if (bKeepConstantViewSize)
	{
		// We're assuming that the gizmo origin is at the component location, so we don't need to worry about scaling
		//  relative to a different point, but we still want our scaling to be view independent for cases where the
		//  relative transform is big enough that the origin is offscreen while the component is visible.
		double ExtraScaling = ViewBasedTransformAdjustersLocals::GetComponentToGizmoScaling(View, CurrentComponentToWorld);

		ViewRelativeTransformToUse.MultiplyScale3D(FVector3d(ExtraScaling));
		ViewRelativeTransformToUse.ScaleTranslation(ExtraScaling);
	}

	return ViewRelativeTransformToUse * ViewFrameAtComponent.ToFTransform();
}

FBoxSphereBounds UE::GizmoRenderingUtil::FConstantViewRelativeTransformAdjuster::GetViewIndependentBounds(
	const FTransform& LocalToWorld, const FBoxSphereBounds& OriginalBounds)
{
	if (bKeepConstantViewSize)
	{
		return ViewBasedTransformAdjustersLocals::InfiniteBounds;
	}

	// Otherwise, the only thing that is changing is the orientation. Unfortunately we don't know if
	//  the bounds started as a sphere or a box, so we'll choose the larger for our radius.
	return FBoxSphereBounds(FSphere(OriginalBounds.Origin, FMath::Max(OriginalBounds.BoxExtent.Size(), OriginalBounds.SphereRadius)));
}
