// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/BaseBrushTool.h"

#include "Engine/Engine.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveTool.h"
#include "GenericPlatform/GenericPlatformApplicationMisc.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "BaseGizmos/BrushStampIndicator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseBrushTool)


#define LOCTEXT_NAMESPACE "UBaseBrushTool"


UBrushBaseProperties::UBrushBaseProperties()
{
	BrushSize = 0.25f;
	bSpecifyRadius = false;
	BrushRadius = 10.0f;
	BrushStrength = 0.5f;
	BrushFalloffAmount = 1.0f;
}

void UBrushAdjusterInputBehavior::Initialize(UBaseBrushTool* InBrushTool)
{
	BrushTool = InBrushTool;
}

void UBrushAdjusterInputBehavior::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (!bAdjustingBrush)
	{
		return;
	}

	FText BrushAdjustmentMessage;
	if (bAdjustingHorizontally)
	{
		BrushAdjustmentMessage = FText::Format(LOCTEXT("AdjustRadius", "Radius: {0}"), FText::AsNumber(BrushTool->BrushProperties->BrushRadius));
	}
	else
	{
		BrushAdjustmentMessage = FText::Format(LOCTEXT("AdjustStrength", "Strength: {0}"), FText::AsNumber(BrushTool->BrushProperties->BrushStrength));
	}

	FCanvasTextItem TextItem(BrushOrigin, BrushAdjustmentMessage, GEngine->GetMediumFont(), FLinearColor::White);
	TextItem.EnableShadow(FLinearColor::Black);
	Canvas->DrawItem(TextItem);
}

void UBrushAdjusterInputBehavior::OnDragStart(FVector2D InScreenPosition)
{
	constexpr bool bAdjustHorizontal = true;
	BrushOrigin = InScreenPosition;
	ResetAdjustmentOrigin(InScreenPosition, bAdjustHorizontal);
}

void UBrushAdjusterInputBehavior::ResetAdjustmentOrigin(FVector2D InScreenPosition, bool bHorizontalAdjust)
{
	bAdjustingHorizontally = bHorizontalAdjust;
	AdjustmentOrigin = InScreenPosition;
	if (BrushTool->BrushProperties->bSpecifyRadius)
	{
		StartBrushRadius = BrushTool->BrushProperties->BrushRadius;
	}
	else
	{
		StartBrushRadius = BrushTool->BrushProperties->BrushSize;
	}
	StartBrushStrength = BrushTool->BrushProperties->BrushStrength;
}

void UBrushAdjusterInputBehavior::OnDragUpdate(FVector2D InScreenPosition)
{
	if (!bAdjustingBrush)
	{
		return;
	}

	// calculate screen space cursor delta relative to adjustment origin
	const float HorizontalDelta = InScreenPosition.X - AdjustmentOrigin.X;
	const float VerticalDelta = InScreenPosition.Y - AdjustmentOrigin.Y;
	const float HorizDeltaMag = FMath::Abs(HorizontalDelta);
	const float VertDeltaMag = FMath::Abs(VerticalDelta);

	if (bAdjustingHorizontally && HorizDeltaMag < VertDeltaMag)
	{
		// switch to adjusting vertically and re-center adjustment origin
		ResetAdjustmentOrigin(InScreenPosition, false);
	}
	else if (!bAdjustingHorizontally && VertDeltaMag < HorizDeltaMag)
	{
		// switch to adjusting horizontally and re-center adjustment origin
		ResetAdjustmentOrigin(InScreenPosition, true);
	}

	// scale for consistent screen space speed on varying monitor DPI
	// (takes device coordinates as input because multi-monitor setups may have different DPI)
	const float DPIScale = FGenericPlatformApplicationMisc::GetDPIScaleFactorAtPoint(InScreenPosition.X, InScreenPosition.Y);
	
	// apply directional adjustments
	if (bAdjustingHorizontally)
	{
		// adjust brush size based on horizontal mouse drag
		if (BrushTool->BrushProperties->bSpecifyRadius)
		{
			float NewRadius = StartBrushRadius + HorizontalDelta * (SizeAdjustSpeed * DPIScale * BrushTool->LastBrushStamp.HitResult.Distance);
			NewRadius = FMath::Max(NewRadius, 0.01f);
			BrushTool->BrushProperties->BrushRadius = NewRadius;
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushRadius)));
			BrushTool->BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		}
		else
		{
			float NewSize = StartBrushRadius + HorizontalDelta * (SizeAdjustSpeed * DPIScale);
			NewSize = FMath::Clamp(NewSize, 0.01f, 1.0f);
			BrushTool->BrushProperties->BrushSize = NewSize;
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushSize)));
			BrushTool->BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		}
	}
	else
	{
		// adjust brush strength based on vertical mouse drag
		float NewStrength = StartBrushStrength + VerticalDelta * -(StrengthAdjustSpeed * DPIScale);
		NewStrength = FMath::Min(1.0f,FMath::Max(NewStrength, 0.f));
		BrushTool->BrushProperties->BrushStrength = NewStrength;

#if WITH_EDITOR
		FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushStrength)));
		BrushTool->BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
	}	
}

EInputDevices UBrushAdjusterInputBehavior::GetSupportedDevices()
{
	return EInputDevices::Keyboard;
}

bool UBrushAdjusterInputBehavior::IsPressed(const FInputDeviceState& Input)
{
	if (Input.IsFromDevice(EInputDevices::Keyboard))
	{
		ActiveDevice = EInputDevices::Keyboard;
		return Input.Keyboard.ActiveKey.Button == EKeys::B && Input.Keyboard.ActiveKey.bDown;
	}
	
	return false;
}

bool UBrushAdjusterInputBehavior::IsReleased(const FInputDeviceState& Input)
{
	if (Input.IsFromDevice(EInputDevices::Keyboard))
	{
		return Input.Keyboard.ActiveKey.Button == EKeys::B && Input.Keyboard.ActiveKey.bReleased;
	}
	
	return false;
}

FInputCaptureRequest UBrushAdjusterInputBehavior::WantsCapture(const FInputDeviceState& Input)
{
	if (IsPressed(Input))
	{
		return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, 0.f);
	}

	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate UBrushAdjusterInputBehavior::BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side)
{
	bAdjustingBrush = true;
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UBrushAdjusterInputBehavior::UpdateCapture(
	const FInputDeviceState& Input,
	const FInputCaptureData& Data)
{
	if (IsReleased(Input))
	{
		bAdjustingBrush = false;
		return FInputCaptureUpdate::End();
	}

	return FInputCaptureUpdate::Continue();
}

void UBrushAdjusterInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	bAdjustingBrush = false;
}

UBaseBrushTool::UBaseBrushTool()
{
	PropertyClass = UBrushBaseProperties::StaticClass();
}

void UBaseBrushTool::Setup()
{
	UMeshSurfacePointTool::Setup();
	BrushProperties = NewObject<UBrushBaseProperties>(this, PropertyClass.Get(), TEXT("Brush"));
	float MaxDimension = static_cast<float>( EstimateMaximumTargetDimension());
	BrushRelativeSizeRange = TInterval<float>(MaxDimension*0.01f, MaxDimension);

	RecalculateBrushRadius();

	// initialize our properties
	AddToolPropertySource(BrushProperties);

	SetupBrushStampIndicator();

	// add input behavior to click-drag while holding hotkey to adjust brush size and strength
	if (SupportsBrushAdjustmentInput())
	{
		BrushEditBehavior = NewObject<ULocalTwoAxisPropertyEditInputBehavior>(this);
		
		BrushEditBehavior->HorizontalProperty.GetValueFunc = [this]()
		{
			if (BrushProperties->bSpecifyRadius)
			{
				return BrushProperties->BrushRadius;
			}
			else
			{
				return BrushProperties->BrushSize;
			}
		};
		BrushEditBehavior->HorizontalProperty.SetValueFunc = [this](float NewValue)
		{
			if (BrushProperties->bSpecifyRadius)
			{
				BrushProperties->BrushRadius = FMath::Max(NewValue, 0.01f);
#if WITH_EDITOR
				FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushRadius)));
				BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
			}
			else
			{
				BrushProperties->BrushSize = FMath::Clamp(NewValue, 0.f, 1.f);
#if WITH_EDITOR
				FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushSize)));
				BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
			}
		};
		BrushEditBehavior->HorizontalProperty.MutateDeltaFunc = [this](float Delta)
		{
			// Scale delta if brush size is in world units.
			return Delta * (BrushProperties->bSpecifyRadius ? LastBrushStamp.HitResult.Distance : 1.f);
		};
		BrushEditBehavior->HorizontalProperty.Name = LOCTEXT("BrushRadius", "Radius");
		BrushEditBehavior->HorizontalProperty.EditRate = 0.001f;
		BrushEditBehavior->HorizontalProperty.bEnabled = true;
		
		BrushEditBehavior->VerticalProperty.GetValueFunc = [this]()
		{
			return BrushProperties->BrushStrength;
		};
		BrushEditBehavior->VerticalProperty.SetValueFunc = [this](float NewValue)
		{
			BrushProperties->BrushStrength = FMath::Min(1.0f,FMath::Max(NewValue, 0.f));
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushStrength)));
			BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		};
		BrushEditBehavior->VerticalProperty.Name = LOCTEXT("BrushStrength", "Strength");
		BrushEditBehavior->VerticalProperty.EditRate = 0.005f;
		BrushEditBehavior->VerticalProperty.bEnabled = true;
		
		BrushEditBehavior->OnDragUpdated.AddWeakLambda(this, [this]()
		{
			RecalculateBrushRadius();
			NotifyOfPropertyChangeByTool(BrushProperties);
		});
		
		BrushEditBehavior->Initialize();
		AddInputBehavior(BrushEditBehavior.Get());
	}
}


void UBaseBrushTool::Shutdown(EToolShutdownType ShutdownType)
{
	ShutdownBrushStampIndicator();
}

void UBaseBrushTool::OnTick(float DeltaTime)
{
	ActivePressure = GetCurrentDevicePressure();
}

void UBaseBrushTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == BrushProperties)
	{
		RecalculateBrushRadius();
	}
}

FInputRayHit UBaseBrushTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (!bEnabled)
	{
		// no hit
		return FInputRayHit();
	}

	// hit-test the tool target
	return Super::CanBeginClickDragSequence(PressPos);	
}

void UBaseBrushTool::IncreaseBrushSizeAction()
{
	if (BrushProperties->bSpecifyRadius)
	{
		// Hardcoded max of 1000 chosen to match the BrushRadius "UIMax" specified in UBrushBaseProperties
		BrushProperties->BrushRadius = FMath::Min(BrushProperties->BrushRadius * 1.1f, 1000.f);
	}
	else
	{
		BrushProperties->BrushSize = FMath::Clamp(BrushProperties->BrushSize + 0.025f, 0.0f, 1.0f);
	}
	RecalculateBrushRadius();
	NotifyOfPropertyChangeByTool(BrushProperties);
}

void UBaseBrushTool::DecreaseBrushSizeAction()
{
	if (BrushProperties->bSpecifyRadius)
	{
		BrushProperties->BrushRadius = FMath::Max(BrushProperties->BrushRadius / 1.1f, 1.f);
	}
	else
	{
		BrushProperties->BrushSize = FMath::Clamp(BrushProperties->BrushSize - 0.025f, 0.0f, 1.0f);
	}
	RecalculateBrushRadius();
	NotifyOfPropertyChangeByTool(BrushProperties);
}


void UBaseBrushTool::IncreaseBrushStrengthAction()
{
	const float ChangeAmount = 0.02f;
	const float OldValue = BrushProperties->BrushStrength;

	float NewValue = OldValue + ChangeAmount;
	BrushProperties->BrushStrength = FMath::Clamp(NewValue, 0.f, 1.f);
	NotifyOfPropertyChangeByTool(BrushProperties);
}

void UBaseBrushTool::DecreaseBrushStrengthAction()
{
	const float ChangeAmount = 0.02f;
	const float OldValue = BrushProperties->BrushStrength;

	float NewValue = OldValue - ChangeAmount;
	BrushProperties->BrushStrength = FMath::Clamp(NewValue, 0.f, 1.f);
	NotifyOfPropertyChangeByTool(BrushProperties);
}


void UBaseBrushTool::IncreaseBrushFalloffAction()
{
	const float ChangeAmount = 0.02f;
	const float OldValue = BrushProperties->BrushFalloffAmount;

	float NewValue = OldValue + ChangeAmount;
	BrushProperties->BrushFalloffAmount = FMath::Clamp(NewValue, 0.f, 1.f);
	NotifyOfPropertyChangeByTool(BrushProperties);
}

void UBaseBrushTool::DecreaseBrushFalloffAction()
{
	const float ChangeAmount = 0.02f;
	const float OldValue = BrushProperties->BrushFalloffAmount;

	float NewValue = OldValue - ChangeAmount;
	BrushProperties->BrushFalloffAmount = FMath::Clamp(NewValue, 0.f, 1.f);
	NotifyOfPropertyChangeByTool(BrushProperties);
}

double UBaseBrushTool::GetCurrentBrushRadius() const
{
	if (BrushProperties && BrushProperties->bToolSupportsPressureSensitivity 
		&& BrushProperties->bEnablePressureSensitivity)
	{
		return CurrentBrushRadius * GetActivePressure();
	}
	return CurrentBrushRadius;
}

void UBaseBrushTool::SetBrushEnabled(bool bIsEnabled)
{
	bEnabled = bIsEnabled;
	BrushStampIndicator->bVisible = bIsEnabled;
}

void UBaseBrushTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	constexpr EToolActionKeyRepeatMode KeyRepeatOnBrushAdjust = EToolActionKeyRepeatMode::Enabled;

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 10,
		TEXT("BrushIncreaseSize"), 
		LOCTEXT("BrushIncreaseSize", "Increase Brush Size"),
		LOCTEXT("BrushIncreaseSizeTooltip", "Press this key to increase brush radius by a percentage of its current size."),
		EModifierKey::None, EKeys::RightBracket,
		[this]() { IncreaseBrushSizeAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 11,
		TEXT("BrushDecreaseSize"), 
		LOCTEXT("BrushDecreaseSize", "Decrease Brush Size"),
		LOCTEXT("BrushDecreaseSizeTooltip", "Press this key to decrease brush radius by a percentage of its current size."),
		EModifierKey::None, EKeys::LeftBracket,
		[this]() { DecreaseBrushSizeAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 12,
		TEXT("BrushIncreaseFalloff"),
		LOCTEXT("BrushIncreaseFalloff", "Increase Brush Falloff"),
		LOCTEXT("BrushIncreaseFalloffTooltip", "Press this key to increase brush falloff by a fixed increment."),
		EModifierKey::Shift | EModifierKey::Control, EKeys::RightBracket,
		[this]() { IncreaseBrushFalloffAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 13,
		TEXT("BrushDecreaseFalloff"),
		LOCTEXT("BrushDecreaseFalloff", "Decrease Brush Falloff"),
		LOCTEXT("BrushDecreaseFalloffTooltip", "Press this key to decrease brush falloff by a fixed increment."),
		EModifierKey::Shift | EModifierKey::Control, EKeys::LeftBracket,
		[this]() { DecreaseBrushFalloffAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 14,
		TEXT("BrushIncreaseStrength"),
		LOCTEXT("BrushIncreaseStrength", "Increase Brush Strength"),
		LOCTEXT("BrushIncreaseStrengthTooltip", "Press this key to increase brush strength by a fixed increment."),
		EModifierKey::Control, EKeys::RightBracket,
		[this]() { IncreaseBrushStrengthAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 15,
		TEXT("BrushDecreaseStrength"),
		LOCTEXT("BrushDecreaseStrength", "Decrease Brush Strength"),
		LOCTEXT("BrushDecreaseStrengthTooltip", "Press this key to decrease brush strength by a fixed increment."),
		EModifierKey::Control, EKeys::LeftBracket,
		[this]() { DecreaseBrushStrengthAction(); },
		KeyRepeatOnBrushAdjust);
}




void UBaseBrushTool::RecalculateBrushRadius()
{
	TInterval<float> ScaledBrushSizeRange(BrushRelativeSizeRange.Min/WorldToLocalScale, BrushRelativeSizeRange.Max/WorldToLocalScale);
	
	if (BrushProperties->bSpecifyRadius)
	{
		CurrentBrushRadius = BrushProperties->BrushRadius;
		BrushProperties->BrushSize = static_cast<float>( (2 * CurrentBrushRadius - ScaledBrushSizeRange.Min) / ScaledBrushSizeRange.Size() );
	}
	else
	{
		CurrentBrushRadius = 0.5 * ScaledBrushSizeRange.Interpolate(BrushProperties->BrushSize);
		BrushProperties->BrushRadius = static_cast<float>( CurrentBrushRadius );
	}
}


void UBaseBrushTool::OnBeginDrag(const FRay& Ray)
{
	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		LastBrushStamp.Radius = BrushProperties->BrushRadius;
		LastBrushStamp.WorldPosition = OutHit.ImpactPoint;
		LastBrushStamp.WorldNormal = OutHit.Normal;
		LastBrushStamp.HitResult = OutHit;
		LastBrushStamp.Falloff = BrushProperties->BrushFalloffAmount;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bInBrushStroke = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UBaseBrushTool::OnUpdateDrag(const FRay& Ray)
{
	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		LastBrushStamp.Radius = BrushProperties->BrushRadius;
		LastBrushStamp.WorldPosition = OutHit.ImpactPoint;
		LastBrushStamp.WorldNormal = OutHit.Normal;
		LastBrushStamp.HitResult = OutHit;
		LastBrushStamp.Falloff = BrushProperties->BrushFalloffAmount;
	}
}

void UBaseBrushTool::OnEndDrag(const FRay& Ray)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bInBrushStroke = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UBaseBrushTool::OnCancelDrag()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bInBrushStroke = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UBaseBrushTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (BrushEditBehavior.IsValid())
	{
		if (BrushEditBehavior->IsEditing())
		{
			return true;
		}

		// When not in adjustment mode, keep the brush & adjustment origin synchronized with
		// the brush so that the initial BrushEdit HUD display tracks the brush stamp.
		BrushEditBehavior->ResetOrigin(DevicePos.ScreenPosition);
	}
	
	FHitResult OutHit;
	if (HitTest(DevicePos.WorldRay, OutHit))
	{
		LastBrushStamp.Radius = BrushProperties->BrushRadius;
		LastBrushStamp.WorldPosition = OutHit.ImpactPoint;
		LastBrushStamp.WorldNormal = OutHit.Normal;
		LastBrushStamp.HitResult = OutHit;
		LastBrushStamp.Falloff = BrushProperties->BrushFalloffAmount;
	}
	return true;
}

void UBaseBrushTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (bEnabled)
	{
		UMeshSurfacePointTool::Render(RenderAPI);
		UpdateBrushStampIndicator();
	}
}

void UBaseBrushTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	Super::DrawHUD(Canvas, RenderAPI);
	if (BrushEditBehavior.IsValid())
	{
		BrushEditBehavior->DrawHUD(Canvas, RenderAPI);	
	}
}

const FString BaseBrushIndicatorGizmoType = "BrushIndicatorGizmoType";

void UBaseBrushTool::SetupBrushStampIndicator()
{
	if (!BrushStampIndicator)
	{
		// register and spawn brush indicator gizmo
		GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(BaseBrushIndicatorGizmoType, NewObject<UBrushStampIndicatorBuilder>());
		BrushStampIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UBrushStampIndicator>(BaseBrushIndicatorGizmoType, FString(), this);
	}
}

void UBaseBrushTool::UpdateBrushStampIndicator()
{
	if (BrushStampIndicator)
	{
		if (BrushEditBehavior.IsValid())
		{
			BrushStampIndicator->LineColor = BrushEditBehavior->IsEditing() ? FLinearColor::White : FLinearColor::Green;	
		}
		
		BrushStampIndicator->Update(BrushProperties->BrushRadius, LastBrushStamp.WorldPosition, LastBrushStamp.WorldNormal, BrushProperties->BrushFalloffAmount, BrushProperties->BrushStrength);
	}
}

void UBaseBrushTool::ShutdownBrushStampIndicator()
{
	if (BrushStampIndicator)
	{
		GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(BrushStampIndicator);
		BrushStampIndicator = nullptr;
		GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(BaseBrushIndicatorGizmoType);
	}
}


#undef LOCTEXT_NAMESPACE

