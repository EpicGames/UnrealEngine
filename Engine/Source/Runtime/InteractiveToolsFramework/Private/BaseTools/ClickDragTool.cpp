// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/ClickDragTool.h"
#include "InteractiveToolManager.h"
#include "BaseBehaviors/ClickDragBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClickDragTool)


#define LOCTEXT_NAMESPACE "UClickDragTool"


/*
 * ToolBuilder
 */

bool UClickDragToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UClickDragToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UClickDragTool* NewTool = NewObject<UClickDragTool>(SceneState.ToolManager);
	return NewTool;
}



/*
 * Tool
 */


void UClickDragTool::Setup()
{
	UInteractiveTool::Setup();

	// add default mouse input behavior
	UClickDragInputBehavior* MouseBehavior = NewObject<UClickDragInputBehavior>();
	MouseBehavior->Initialize(this);
	AddInputBehavior(MouseBehavior);
}


FInputRayHit UClickDragTool::CanBeginClickDragSequence(const FInputDeviceRay& ClickPos)
{
	return FInputRayHit(0.0f);
}


void UClickDragTool::OnClickPress(const FInputDeviceRay& ClickPos)
{
	// print debug message
	GetToolManager()->DisplayMessage( 
		FText::Format(LOCTEXT("OnClickPressMessage", "UClickDragTool::OnClickPress: clicked at ({0},{1})"),
			FText::AsNumber(ClickPos.ScreenPosition.X), FText::AsNumber(ClickPos.ScreenPosition.Y)),
		EToolMessageLevel::Internal);
}

void UClickDragTool::OnClickDrag(const FInputDeviceRay& ClickPos)
{
}


void UClickDragTool::OnClickRelease(const FInputDeviceRay& ClickPos)
{
	// print debug message
	GetToolManager()->DisplayMessage(
		FText::Format(LOCTEXT("OnClickReleaseMessage", "UClickDragTool::OnClickRelease: released at ({0},{1})"),
			FText::AsNumber(ClickPos.ScreenPosition.X), FText::AsNumber(ClickPos.ScreenPosition.Y)),
		EToolMessageLevel::Internal);
}


void UClickDragTool::OnTerminateDragSequence()
{

}


#undef LOCTEXT_NAMESPACE
