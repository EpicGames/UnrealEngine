// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/MeshSurfacePointTool.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "Engine/HitResult.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"

#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSurfacePointTool)

#define LOCTEXT_NAMESPACE "UMeshSurfacePointTool"


/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UMeshSurfacePointToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(UPrimitiveComponentBackedTarget::StaticClass());
	return TypeRequirements;
}

bool UMeshSurfacePointToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

UInteractiveTool* UMeshSurfacePointToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMeshSurfacePointTool* NewTool = CreateNewTool(SceneState);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}

UMeshSurfacePointTool* UMeshSurfacePointToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshSurfacePointTool>(SceneState.ToolManager);
}

void UMeshSurfacePointToolBuilder::InitializeNewTool(UMeshSurfacePointTool* NewTool, const FToolBuilderState& SceneState) const
{
	UToolTarget* Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	check(Target);
	NewTool->SetTarget(Target);
	NewTool->SetStylusAPI(this->StylusAPI);
	NewTool->SetWorld(SceneState.World);
}

/*
 * Tool
 */
void UMeshSurfacePointTool::Setup()
{
	UInteractiveTool::Setup();

	bShiftToggle = false;
	bCtrlToggle = false;

	// add input behaviors
	UClickDragInputBehavior* DragBehavior = NewObject<UClickDragInputBehavior>();
	DragBehavior->Modifiers.RegisterModifier(ShiftModifier, FInputDeviceState::IsShiftKeyDown);
	DragBehavior->Modifiers.RegisterModifier(CtrlModifier, FInputDeviceState::IsCtrlKeyDown);
	DragBehavior->Initialize(this);
	AddInputBehavior(DragBehavior);

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Modifiers.RegisterModifier(ShiftModifier, FInputDeviceState::IsShiftKeyDown);
	HoverBehavior->Modifiers.RegisterModifier(CtrlModifier, FInputDeviceState::IsCtrlKeyDown);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);
}

void UMeshSurfacePointTool::SetStylusAPI(IToolStylusStateProviderAPI* StylusAPIIn)
{
	this->StylusAPI = StylusAPIIn;
}

bool UMeshSurfacePointTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	return Cast<IPrimitiveComponentBackedTarget>(Target)->HitTestComponent(Ray, OutHit);
}


bool UMeshSurfacePointTool::GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut)
{
	FHitResult HitResult;
	if (HitTest(WorldRay, HitResult))
	{
		PointOut = HitResult.ImpactPoint;
		return true;
	}
	return false;
}



void UMeshSurfacePointTool::OnBeginDrag(const FRay& Ray)
{

}


void UMeshSurfacePointTool::OnUpdateDrag(const FRay& Ray)
{
	FHitResult OutHit;
	if ( HitTest(Ray, OutHit) ) 
	{
		GetToolManager()->DisplayMessage( 
			FText::Format(LOCTEXT("OnUpdateDragMessage", "UMeshSurfacePointTool::OnUpdateDrag: Hit triangle index {0} at ray distance {1}"),
				FText::AsNumber(OutHit.FaceIndex), FText::AsNumber(OutHit.Distance)),
			EToolMessageLevel::Internal);
	}
}

void UMeshSurfacePointTool::OnEndDrag(const FRay& Ray)
{
	//GetToolManager()->DisplayMessage(TEXT("UMeshSurfacePointTool::OnEndDrag!"), EToolMessageLevel::Internal);
}


void UMeshSurfacePointTool::SetShiftToggle(bool bShiftDown)
{
	bShiftToggle = bShiftDown;
}

void UMeshSurfacePointTool::SetCtrlToggle(bool bCtrlDown)
{
	bCtrlToggle = bCtrlDown;
}



void UMeshSurfacePointTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == ShiftModifier)
	{
		bShiftToggle = bIsOn;
	}
	else if (ModifierID == CtrlModifier)
	{
		bCtrlToggle = bIsOn;
	}
}


FInputRayHit UMeshSurfacePointTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FHitResult OutHit;
	if (HitTest(PressPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}
	return FInputRayHit();
}

void UMeshSurfacePointTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	LastWorldRay = PressPos.WorldRay;
	OnBeginDrag(PressPos.WorldRay);
}

void UMeshSurfacePointTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	LastWorldRay = DragPos.WorldRay;
	OnUpdateDrag(DragPos.WorldRay);
}

void UMeshSurfacePointTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	LastWorldRay = ReleasePos.WorldRay;
	OnEndDrag(ReleasePos.WorldRay);
}

void UMeshSurfacePointTool::OnTerminateDragSequence()
{
	OnCancelDrag();
}




FInputRayHit UMeshSurfacePointTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	LastWorldRay = PressPos.WorldRay;
	FHitResult OutHit;
	if (HitTest(PressPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}
	return FInputRayHit();
}


float UMeshSurfacePointTool::GetCurrentDevicePressure() const
{
	return (StylusAPI != nullptr) ? FMath::Clamp(StylusAPI->GetCurrentPressure(), 0.0f, 1.0f) : 1.0f;
}

void UMeshSurfacePointTool::SetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* UMeshSurfacePointTool::GetTargetWorld()
{
	return TargetWorld.Get();
}



#undef LOCTEXT_NAMESPACE
