// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/SingleClickTool.h"
#include "InteractiveToolManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleClickTool)

#define LOCTEXT_NAMESPACE "USingleClickTool"



/*
 * ToolBuilder
 */

bool USingleClickToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* USingleClickToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	USingleClickTool* NewTool = NewObject<USingleClickTool>(SceneState.ToolManager);
	return NewTool;
}



/*
 * Tool
 */


void USingleClickTool::Setup()
{
	UInteractiveTool::Setup();

	// add default button input behaviors for devices
	USingleClickInputBehavior* MouseBehavior = NewObject<USingleClickInputBehavior>();
	MouseBehavior->Initialize(this);
	AddInputBehavior(MouseBehavior);
}


FInputRayHit USingleClickTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	return FInputRayHit(0.0f);
}


void USingleClickTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	// print debug message
	GetToolManager()->DisplayMessage(
		FText::Format(LOCTEXT("OnClickedMessage", "USingleClickTool::OnClicked at ({0},{1})"),
			FText::AsNumber(ClickPos.ScreenPosition.X), FText::AsNumber(ClickPos.ScreenPosition.Y)),
		EToolMessageLevel::Internal );
}




#undef LOCTEXT_NAMESPACE
