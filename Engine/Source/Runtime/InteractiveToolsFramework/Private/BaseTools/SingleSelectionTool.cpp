// Copyright Epic Games, Inc. All Rights Reserved.

#include "SingleSelectionTool.h"
#include "InteractiveToolManager.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleSelectionTool)

#define LOCTEXT_NAMESPACE "USingleSelectionTool"

bool USingleSelectionTool::SupportsWorldSpaceFocusBox()
{
	return Target && Cast<ISceneComponentBackedTarget>(Target) != nullptr;
}


FBox USingleSelectionTool::GetWorldSpaceFocusBox()
{
	if (Target)
	{
		ISceneComponentBackedTarget* PrimTarget = Cast<ISceneComponentBackedTarget>(Target);
		if (PrimTarget)
		{
			USceneComponent* Component = PrimTarget->GetOwnerSceneComponent();
			if (Component)
			{
				return Component->Bounds.GetBox();
			}
		}
	}
	return FBox();
}


bool USingleSelectionTool::SupportsWorldSpaceFocusPoint()
{
	return Target && Cast<IPrimitiveComponentBackedTarget>(Target) != nullptr;

}
bool USingleSelectionTool::GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut)
{
	if (Target)
	{
		IPrimitiveComponentBackedTarget* PrimTarget = Cast<IPrimitiveComponentBackedTarget>(Target);
		if (PrimTarget)
		{
			FHitResult HitResult;
			if (PrimTarget->HitTestComponent(WorldRay, HitResult))
			{
				PointOut = HitResult.ImpactPoint;
				return true;
			}
		}
	}
	return false;
}




#undef LOCTEXT_NAMESPACE
