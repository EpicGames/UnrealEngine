// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/TransformChange.h"
#include "Components/SceneComponent.h"

FComponentWorldTransformChange::FComponentWorldTransformChange()
{
	FromWorldTransform = ToWorldTransform = FTransform::Identity;
}

FComponentWorldTransformChange::FComponentWorldTransformChange(const FTransform& From, const FTransform& To)
{
	FromWorldTransform = From;
	ToWorldTransform = To;
}

void FComponentWorldTransformChange::Apply(UObject* Object)
{
	USceneComponent* SceneComponent = CastChecked<USceneComponent>(Object);
	SceneComponent->SetWorldTransform(ToWorldTransform);

	if (OnChangeAppliedFunc)
	{
		OnChangeAppliedFunc(this, Object, true);
	}
}

void FComponentWorldTransformChange::Revert(UObject* Object)
{
	USceneComponent* SceneComponent = CastChecked<USceneComponent>(Object);
	SceneComponent->SetWorldTransform(FromWorldTransform);

	if (OnChangeAppliedFunc)
	{
		OnChangeAppliedFunc(this, Object, false);
	}
}


FString FComponentWorldTransformChange::ToString() const
{
	return FString("Transform Change");
}

