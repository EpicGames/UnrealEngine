// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentSourceInterfaces.h"

#include "Components/PrimitiveComponent.h"
#include "Containers/Array.h"
#include "TargetInterfaces/MaterialProvider.h"


namespace
{
	TMap<uint32, TUniquePtr<FComponentTargetFactory>> Factories;

	static int32 FactoryUniqueKeyGenerator = 1;
}


int32 AddComponentTargetFactory( TUniquePtr<FComponentTargetFactory> Factory )
{
	int32 NewFactoryKey = FactoryUniqueKeyGenerator++;
	Factories.Add(NewFactoryKey, MoveTemp(Factory));
	return NewFactoryKey;
}


FComponentTargetFactory* FindComponentTargetFactoryByKey(int32 Key)
{
	TUniquePtr<FComponentTargetFactory>* Found = Factories.Find(Key);
	return (Found != nullptr) ? Found->Get() : nullptr;
}


bool RemoveComponentTargetFactoryByKey(int32 Key)
{
	return Factories.Remove(Key) != 0;
}

void RemoveAllComponentTargetFactoryies()
{
	Factories.Reset();
}


bool CanMakeComponentTarget(UActorComponent* Component)
{
	for ( const auto& FactoryPair : Factories )
	{
		if (FactoryPair.Value->CanBuild(Component) )
		{
			return true;
		}
	}
	return false;
}

TUniquePtr<FPrimitiveComponentTarget> MakeComponentTarget(UPrimitiveComponent* Component)
{
	for (const auto& FactoryPair : Factories)
	{
		if (FactoryPair.Value->CanBuild( Component ) )
		{
			return FactoryPair.Value->Build( Component );
		}
	}
	return {};
}



bool FPrimitiveComponentTarget::IsValid() const
{
	return IsValidChecked(Component) && !Component->IsUnreachable() && Component->IsValidLowLevel();
}

AActor* FPrimitiveComponentTarget::GetOwnerActor() const
{
	return IsValid() ? Component->GetOwner() : nullptr;
}

UPrimitiveComponent* FPrimitiveComponentTarget::GetOwnerComponent() const
{
	return IsValid() ? Component : nullptr;
}


void FPrimitiveComponentTarget::SetOwnerVisibility(bool bVisible) const
{
	if (IsValid())
	{
		Component->SetVisibility(bVisible);
	}
}


int32 FPrimitiveComponentTarget::GetNumMaterials() const
{
	return IsValid() ? Component->GetNumMaterials() : 0;
}

UMaterialInterface* FPrimitiveComponentTarget::GetMaterial(int32 MaterialIndex) const
{
	return IsValid() ? Component->GetMaterial(MaterialIndex) : nullptr;
}

void FPrimitiveComponentTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bAssetMaterials) const
{
	if (IsValid())
	{
		int32 NumMaterials = Component->GetNumMaterials();
		MaterialSetOut.Materials.SetNum(NumMaterials);
		for (int32 k = 0; k < NumMaterials; ++k)
		{
			MaterialSetOut.Materials[k] = Component->GetMaterial(k);
		}
	}
}


void FPrimitiveComponentTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	check(false);		// not implemented
}




FTransform FPrimitiveComponentTarget::GetWorldTransform() const
{
	return IsValid() ? Component->GetComponentTransform() : FTransform::Identity;
}

bool FPrimitiveComponentTarget::HitTest(const FRay& WorldRay, FHitResult& OutHit) const
{
	FVector End = WorldRay.PointAt(HALF_WORLD_MAX);
	if (IsValid() && Component->LineTraceComponent(OutHit, WorldRay.Origin, End, FCollisionQueryParams(SCENE_QUERY_STAT(HitTest), true)))
	{
		return true;
	}
	return false;
}
