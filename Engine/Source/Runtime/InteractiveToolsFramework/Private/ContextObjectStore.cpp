// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextObjectStore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextObjectStore)

UObject* UContextObjectStore::FindContextByClass(UClass* InClass) const
{
	if (InClass->HasAnyClassFlags(CLASS_Interface))
	{
		for (UObject* ContextObject : ContextObjects)
		{
			if (ContextObject && ContextObject->GetClass()->ImplementsInterface(InClass))
			{
				return ContextObject;
			}
		}
	}
	else
	{
		for (UObject* ContextObject : ContextObjects)
		{
			if (ContextObject && ContextObject->IsA(InClass))
			{
				return ContextObject;
			}
		}
	}
	
	if (UContextObjectStore* ParentStore = Cast<UContextObjectStore>(GetOuter()))
	{
		return ParentStore->FindContextByClass(InClass);
	}

	return nullptr;
}

bool UContextObjectStore::AddContextObject(UObject* InContextObject)
{
	if (InContextObject)
	{
		return ContextObjects.AddUnique(InContextObject) != INDEX_NONE;
	}

	return false;
}

bool UContextObjectStore::RemoveContextObject(UObject* InContextObject)
{
	return ContextObjects.RemoveSingle(InContextObject) != 0;
}

bool UContextObjectStore::RemoveContextObjectsOfType(const UClass* InClass)
{
	return (ContextObjects.RemoveAll([InClass](UObject* InObject)
	{
		return InObject->IsA(InClass);
	}) > 0);
}

void UContextObjectStore::Shutdown()
{
	ContextObjects.Empty();
}

