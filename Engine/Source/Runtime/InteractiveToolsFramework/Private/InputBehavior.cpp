// Copyright Epic Games, Inc. All Rights Reserved.


#include "InputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputBehavior)


bool operator<(const FInputCaptureRequest& l, const FInputCaptureRequest& r)
{
	if (l.Source->GetPriority() == r.Source->GetPriority())
	{
		return l.HitDepth < r.HitDepth;
	}
	else
	{
		return l.Source->GetPriority() < r.Source->GetPriority();
	}
}



UInputBehavior::UInputBehavior()
{
	DefaultPriority = FInputCapturePriority(100);
}

FInputCapturePriority UInputBehavior::GetPriority()
{
	return DefaultPriority;
}

void UInputBehavior::SetDefaultPriority(const FInputCapturePriority& Priority)
{
	DefaultPriority = Priority;
}

EInputDevices UInputBehavior::GetSupportedDevices()
{
	return EInputDevices::Mouse;
}


FInputCaptureRequest UInputBehavior::WantsCapture(const FInputDeviceState& input)
{
	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate UInputBehavior::BeginCapture(const FInputDeviceState& input, EInputCaptureSide eSide)
{
	return FInputCaptureUpdate::Ignore();
}

FInputCaptureUpdate UInputBehavior::UpdateCapture(const FInputDeviceState& input, const FInputCaptureData& data)
{
	return FInputCaptureUpdate::End();
}

void UInputBehavior::ForceEndCapture(const FInputCaptureData& data)
{
	return;
}


bool UInputBehavior::WantsHoverEvents()
{
	return false;
}

FInputCaptureRequest UInputBehavior::WantsHoverCapture(const FInputDeviceState& input)
{
	return FInputCaptureRequest::Ignore();
}


FInputCaptureUpdate UInputBehavior::BeginHoverCapture(const FInputDeviceState& input, EInputCaptureSide eSide)
{
	return FInputCaptureUpdate::Ignore();
}


FInputCaptureUpdate UInputBehavior::UpdateHoverCapture(const FInputDeviceState& input)
{
	return FInputCaptureUpdate::End();
}

void UInputBehavior::EndHoverCapture()
{
	return;
}

bool UInputBehavior::WantsForceEndCapture()
{
	return false;
}

bool UInputBehavior::AllowsCaptureStealing()
{
	return CaptureStealing.IsSet() ? CaptureStealing.Get() : false;
}

void UInputBehavior::SetAllowsCaptureStealing(TAttribute<bool> InAllowsCaptureStealing)
{
	CaptureStealing = InAllowsCaptureStealing;
}
