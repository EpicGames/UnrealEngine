// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputBehaviorSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputBehaviorSet)


UInputBehaviorSet::UInputBehaviorSet()
{
}

UInputBehaviorSet::~UInputBehaviorSet()
{
}


bool UInputBehaviorSet::IsEmpty() const
{
	return Behaviors.Num() == 0;
}

void UInputBehaviorSet::Add(UInputBehavior* Behavior, void* Source, const FString& Group)
{
	//if (source == null)
	//	source = DefaultSource;

	FBehaviorInfo Info;
	Info.Behavior = Behavior;
	Info.Source = Source;
	Info.Group = Group;

	Behaviors.Add(Info);
	BehaviorsModified();
}


void UInputBehaviorSet::Add(const UInputBehaviorSet* OtherSet, void* NewSource, const FString& NewGroup)
{
	for (const FBehaviorInfo& OtherInfo : OtherSet->Behaviors)
	{
		FBehaviorInfo Info;
		Info.Behavior = OtherInfo.Behavior;
		Info.Source = (NewSource == nullptr) ? OtherInfo.Source : NewSource;
		Info.Group = (NewGroup.IsEmpty()) ? OtherInfo.Group : NewGroup;
		Behaviors.Add(Info);
	}
	if (OtherSet->Behaviors.Num() > 0)
	{
		BehaviorsModified();
	}
}


bool UInputBehaviorSet::Remove(UInputBehavior* behavior)
{
	int32 idx = Behaviors.IndexOfByPredicate( 
		[behavior] (const FBehaviorInfo& b) { return b.Behavior == behavior; }
	);
	if ( idx != INDEX_NONE ) {
		Behaviors.RemoveAt(idx);
		BehaviorsModified();
		return true;
	}
	return false;
}


bool UInputBehaviorSet::RemoveByGroup(const FString& Group)
{
	int RemovedCount = Behaviors.RemoveAll(
		[Group](const FBehaviorInfo& b) { return b.Group == Group; }
	);
	if (RemovedCount > 0) 
	{
		BehaviorsModified();
	}
	return (RemovedCount > 0);
}



bool UInputBehaviorSet::RemoveBySource(void* Source)
{
	int RemovedCount = Behaviors.RemoveAll(
		[Source](const FBehaviorInfo& b) { return b.Source == Source; }
	);
	if (RemovedCount > 0)
	{
		BehaviorsModified();
	}
	return (RemovedCount > 0);	
}


void UInputBehaviorSet::RemoveAll()
{
	Behaviors.Reset();
	BehaviorsModified();
}


void UInputBehaviorSet::CollectWantsCapture(const FInputDeviceState& input, TArray<FInputCaptureRequest>& result)
{
	for (FBehaviorInfo& BehaviorInfo : Behaviors)
	{
		// only call WantsCapture if the Behavior supports the current input device
		if (SupportsInputType(BehaviorInfo.Behavior, input)) 
		{
			FInputCaptureRequest request = BehaviorInfo.Behavior->WantsCapture(input);
			if ( request.Type != EInputCaptureRequestType::Ignore ) 
			{
				request.Owner = BehaviorInfo.Source;
				result.Add(request);
			}
		}
	}
}



void UInputBehaviorSet::CollectWantsHoverCapture(const FInputDeviceState& input, TArray<FInputCaptureRequest>& result)
{
	for (FBehaviorInfo& BehaviorInfo : Behaviors)
	{
		// only call WantsCapture if the Behavior supports the current input device
		if (BehaviorInfo.Behavior->WantsHoverEvents() && SupportsInputType(BehaviorInfo.Behavior, input))
		{
			FInputCaptureRequest request = BehaviorInfo.Behavior->WantsHoverCapture(input);
			if (request.Type != EInputCaptureRequestType::Ignore)
			{
				request.Owner = BehaviorInfo.Source;
				result.Add(request);
			}
		}
	}
}

void UInputBehaviorSet::CollectWantsForceEndCapture(TArray<UInputBehavior*>& ResultOut)
{
	for (const FBehaviorInfo& BehaviorInfo : Behaviors)
	{
		if (UInputBehavior* const Behavior = BehaviorInfo.Behavior)
		{
			if (Behavior->WantsForceEndCapture())
			{
				ResultOut.Add(Behavior);
			}
		}
	}
}

void UInputBehaviorSet::BehaviorsModified()
{
	// sort by priority
	Behaviors.StableSort();

	//send some kind of event...
	//FUtil.SafeSendAnyEvent(OnSetChanged, this);
}

