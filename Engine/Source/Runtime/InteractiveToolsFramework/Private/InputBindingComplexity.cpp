// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputBehaviorBindingComplexity.h"
#include "HAL/IConsoleManager.h"

namespace UE::ITF::Private
{

static int32 ITFWideBindingComplexityPriority = 1;
static FAutoConsoleVariableRef CVarITFWideBindingComplexityPriority(
	TEXT("ITF.UseITFWideBindingComplexityPriority"),
	ITFWideBindingComplexityPriority,
	TEXT("Enable or disable the use of binding complexity based behavior priority across the base ITF classes.")
);

bool UseInputBindingComplexity()
{
	return ITFWideBindingComplexityPriority > 0;
}

}

