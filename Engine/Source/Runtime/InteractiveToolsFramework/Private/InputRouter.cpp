// Copyright Epic Games, Inc. All Rights Reserved.


#include "InputRouter.h"
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputRouter)


#define LOCTEXT_NAMESPACE "UInputRouter"

UInputRouter::UInputRouter()
{
	ActiveLeftCapture = nullptr;
	ActiveLeftCaptureOwner = nullptr;
	ActiveRightCapture = nullptr;
	ActiveRightCaptureOwner = nullptr;
	ActiveKeyboardCapture = nullptr;
	ActiveKeyboardCaptureOwner = nullptr;

	bAutoInvalidateOnHover = false;
	bAutoInvalidateOnCapture = false;

	ActiveInputBehaviors = NewObject<UInputBehaviorSet>(this, "InputBehaviors");

	if (FSlateApplication::IsInitialized())
	{
		// Terminate all captures when editor loses focus
		FSlateApplication::Get().OnApplicationActivationStateChanged().AddUObject(this, &UInputRouter::OnApplicationFocusChanged);
	}
}



void UInputRouter::Initialize(IToolsContextTransactionsAPI* TransactionsAPIIn)
{
	this->TransactionsAPI = TransactionsAPIIn;
}

void UInputRouter::Shutdown()
{
	this->TransactionsAPI = nullptr;
}




void UInputRouter::RegisterSource(IInputBehaviorSource* Source)
{
	ActiveInputBehaviors->Add(Source->GetInputBehaviors(), Source);
}


void UInputRouter::DeregisterSource(IInputBehaviorSource* Source)
{
	ForceTerminateSource(Source);

	ActiveInputBehaviors->RemoveBySource(Source);
}


bool UInputRouter::PostInputEvent(const FInputDeviceState& Input)
{
	if (ActiveInputBehaviors->IsEmpty())
	{
		return false;
	}

	if (Input.IsFromDevice(EInputDevices::Mouse))
	{
		const bool bHadMouseCapture = HasActiveMouseCapture();
		PostInputEvent_Mouse(Input);
		LastMouseInputState = Input;
		return HasActiveMouseCapture() || bHadMouseCapture;
	}
	else if (Input.IsFromDevice(EInputDevices::Keyboard))
	{
		// if we are actively capturing Mouse and the key event is from a modifier key, 
		// we want to update those modifiers
		if ( (HasActiveMouseCapture() || ActiveLeftHoverCapture != nullptr)
			   && Input.Keyboard.ActiveKey.Button.IsModifierKey() )
		{
			if ((LastMouseInputState.bAltKeyDown != Input.bAltKeyDown) ||
				(LastMouseInputState.bShiftKeyDown != Input.bShiftKeyDown) ||
				(LastMouseInputState.bCtrlKeyDown != Input.bCtrlKeyDown) ||
				(LastMouseInputState.bCmdKeyDown != Input.bCmdKeyDown))
			{
				LastMouseInputState.SetModifierKeyStates(Input.bShiftKeyDown, Input.bAltKeyDown, Input.bCtrlKeyDown, Input.bCmdKeyDown);

				// cannot call PostInputEvent_Mouse() to propagate modifier key state update because if
				// there is no active capture it may result in one starting!
				//PostInputEvent_Mouse(LastMouseInputState);

				if (ActiveLeftCapture != nullptr )
				{
					HandleCapturedMouseInput(Input);
				}
				else
				{
					bool bHoverStateUpdated = UpdateExistingHoverCaptureIfPresent(Input);
					if (bHoverStateUpdated && bAutoInvalidateOnHover)
					{
						TransactionsAPI->PostInvalidation();
					}
				}
			}
		}

		PostInputEvent_Keyboard(Input);
		return (ActiveKeyboardCapture != nullptr);
	}
	else
	{
		unimplemented();
		TransactionsAPI->DisplayMessage(LOCTEXT("PostInputEventMessage", "UInteractiveToolManager::PostInputEvent - input device is not currently supported."), EToolMessageLevel::Internal);
		return false;
	}
}



//
// Keyboard event handling
// 


void UInputRouter::PostInputEvent_Keyboard(const FInputDeviceState& Input)
{
	if (ActiveKeyboardCapture != nullptr)
	{
		HandleCapturedKeyboardInput(Input);
	}
	else
	{
		ActiveKeyboardCaptureData = FInputCaptureData();
		CheckForKeyboardCaptures(Input);
	}
}


void UInputRouter::CheckForKeyboardCaptures(const FInputDeviceState& Input)
{
	TArray<FInputCaptureRequest> CaptureRequests;
	ActiveInputBehaviors->CollectWantsCapture(Input, CaptureRequests);
	if (CaptureRequests.Num() == 0)
	{
		return;
	}

	CaptureRequests.StableSort();

	bool bAccepted = false;
	for (int i = 0; i < CaptureRequests.Num() && bAccepted == false; ++i)
	{
		FInputCaptureUpdate Result =
			CaptureRequests[i].Source->BeginCapture(Input, EInputCaptureSide::Left);
		if (Result.State == EInputCaptureState::Begin)
		{
			ActiveKeyboardCapture = Result.Source;
			ActiveKeyboardCaptureOwner = CaptureRequests[i].Owner;
			ActiveKeyboardCaptureData = Result.Data;
			bAccepted = true;
		}
	}
}


void UInputRouter::HandleCapturedKeyboardInput(const FInputDeviceState& Input)
{
	if (ActiveKeyboardCapture == nullptr)
	{
		return;
	}

	// Perform Capture Stealing if needed
	bool bStoleCapture = false;
	if (ActiveKeyboardCapture->AllowsCaptureStealing())
	{
		TArray<FInputCaptureRequest> CaptureRequests;
		ActiveInputBehaviors->CollectWantsCapture(Input, CaptureRequests);

		if (!CaptureRequests.IsEmpty())
		{
			CaptureRequests.StableSort();

			const FInputCaptureRequest& TopRequest = CaptureRequests[0];
			UInputBehavior* const TopCapture = TopRequest.Source;

			if (TopCapture != ActiveKeyboardCapture)
			{
				if (TopCapture->GetPriority() < ActiveKeyboardCapture->GetPriority())
				{
					ActiveKeyboardCapture->ForceEndCapture(ActiveKeyboardCaptureData);

					ActiveKeyboardCapture = nullptr;
					ActiveKeyboardCaptureOwner = nullptr;
					ActiveKeyboardCaptureData = FInputCaptureData();

					FInputCaptureUpdate Result = TopRequest.Source->BeginCapture(Input, EInputCaptureSide::Left);
					if (Result.State == EInputCaptureState::Begin)
					{
						ActiveKeyboardCapture = Result.Source;
						ActiveKeyboardCaptureOwner = TopRequest.Owner;
						ActiveKeyboardCaptureData = Result.Data;
					}

					bStoleCapture = true;
				}
			}
		}
	}

	if (!bStoleCapture)
	{
		FInputCaptureUpdate Result =
		ActiveKeyboardCapture->UpdateCapture(Input, ActiveKeyboardCaptureData);

		if (Result.State == EInputCaptureState::End)
		{
			ActiveKeyboardCapture = nullptr;
			ActiveKeyboardCaptureOwner = nullptr;
			ActiveKeyboardCaptureData = FInputCaptureData();
		}
		else if (Result.State != EInputCaptureState::Continue)
		{
			TransactionsAPI->DisplayMessage(LOCTEXT("HandleCapturedKeyboardInputMessage", "UInteractiveToolManager::HandleCapturedKeyboardInput - unexpected capture state!"), EToolMessageLevel::Internal);
		}
	}
	
	if (bAutoInvalidateOnCapture)
	{
		TransactionsAPI->PostInvalidation();
	}
}



//
// Mouse event handling
//



void UInputRouter::PostInputEvent_Mouse(const FInputDeviceState& Input)
{
	if (ActiveLeftCapture != nullptr)
	{
		HandleCapturedMouseInput(Input);
	}
	else
	{
		ActiveLeftCaptureData = FInputCaptureData();
		CheckForMouseCaptures(Input);
	}

	// update hover if nobody is capturing
	if (ActiveLeftCapture == nullptr)
	{
		bool bHoverStateUpdated = ProcessMouseHover(Input);
		if (bHoverStateUpdated && bAutoInvalidateOnHover)
		{
			TransactionsAPI->PostInvalidation();
		}
	}

}


void UInputRouter::PostHoverInputEvent(const FInputDeviceState& Input)
{
	bool bHoverStateUpdated = ProcessMouseHover(Input);
	if (bHoverStateUpdated && bAutoInvalidateOnHover)
	{
		TransactionsAPI->PostInvalidation();
	}
}




bool UInputRouter::HasActiveMouseCapture() const
{
	return (ActiveLeftCapture != nullptr);
}


void UInputRouter::CheckForMouseCaptures(const FInputDeviceState& Input)
{
	TArray<FInputCaptureRequest> CaptureRequests;
	ActiveInputBehaviors->CollectWantsCapture(Input, CaptureRequests);
	if (CaptureRequests.Num() == 0)
	{
		return;
	}

	CaptureRequests.StableSort();

	// end outstanding hover
	TerminateHover(EInputCaptureSide::Left);

	bool bAccepted = false;
	for (int i = 0; i < CaptureRequests.Num() && bAccepted == false; ++i)
	{
		FInputCaptureUpdate Result =
			CaptureRequests[i].Source->BeginCapture(Input, EInputCaptureSide::Left);
		if (Result.State == EInputCaptureState::Begin)
		{
			ActiveLeftCapture = Result.Source;
			ActiveLeftCaptureOwner = CaptureRequests[i].Owner;
			ActiveLeftCaptureData = Result.Data;
			bAccepted = true;
		}

	}
}


void UInputRouter::HandleCapturedMouseInput(const FInputDeviceState& Input)
{
	if (Input.IsFromDevice(EInputDevices::Mouse))
	{
		// Calling this to make sure behaviors treating mouse buttons as modifiers are always updated,
		// even if there is some drag-based behavior currently capturing
		TArray<FInputCaptureRequest> CaptureRequests;
		ActiveInputBehaviors->CollectWantsCapture(Input, CaptureRequests);
	}

	if (ActiveLeftCapture == nullptr)
	{
		return;
	}

	// Perform Capture Stealing if needed
	bool bStoleCapture = false;
	if (ActiveLeftCapture->AllowsCaptureStealing())
	{
		TArray<FInputCaptureRequest> CaptureRequests;
		ActiveInputBehaviors->CollectWantsCapture(Input, CaptureRequests);

		if (!CaptureRequests.IsEmpty())
		{
			CaptureRequests.StableSort();

			const FInputCaptureRequest& TopRequest = CaptureRequests[0];
			UInputBehavior* const TopCapture = TopRequest.Source;

			if (TopCapture != ActiveLeftCapture)
			{
				if (TopCapture->GetPriority() < ActiveLeftCapture->GetPriority())
				{
					ActiveLeftCapture->ForceEndCapture(ActiveLeftCaptureData);
					
					ActiveLeftCapture = nullptr;
					ActiveLeftCaptureOwner = nullptr;
					ActiveLeftCaptureData = FInputCaptureData();

					FInputCaptureUpdate Result = TopRequest.Source->BeginCapture(Input, EInputCaptureSide::Left);
					if (Result.State == EInputCaptureState::Begin)
					{
						ActiveLeftCapture = Result.Source;
						ActiveLeftCaptureOwner = TopRequest.Owner;
						ActiveLeftCaptureData = Result.Data;
					}

					bStoleCapture = true;
				}
			}
		}
	}

	// have active capture - give it this event

	if (!bStoleCapture)
	{
		FInputCaptureUpdate Result =
		ActiveLeftCapture->UpdateCapture(Input, ActiveLeftCaptureData);

		if (Result.State == EInputCaptureState::End)
		{
			ActiveLeftCapture = nullptr;
			ActiveLeftCaptureOwner = nullptr;
			ActiveLeftCaptureData = FInputCaptureData();
		}
		else if (Result.State != EInputCaptureState::Continue)
		{
			TransactionsAPI->DisplayMessage(LOCTEXT("HandleCapturedMouseInputMessage", "UInteractiveToolManager::HandleCapturedMouseInput - unexpected capture state!"), EToolMessageLevel::Internal);
		}
	}
	
	if (bAutoInvalidateOnCapture)
	{
		TransactionsAPI->PostInvalidation();
	}
}




void UInputRouter::TerminateHover(EInputCaptureSide Side)
{
	if (Side == EInputCaptureSide::Left && ActiveLeftHoverCapture != nullptr)
	{
		ActiveLeftHoverCapture->EndHoverCapture();
		ActiveLeftHoverCapture = nullptr;
		ActiveLeftHoverCaptureOwner = nullptr;
	}
}

// Returns true if hover state is updated
bool UInputRouter::UpdateExistingHoverCaptureIfPresent(const FInputDeviceState& Input)
{
	if (ActiveLeftHoverCapture != nullptr)
	{
		FInputCaptureUpdate Result = ActiveLeftHoverCapture->UpdateHoverCapture(Input);
		if (Result.State == EInputCaptureState::End)
		{
			TerminateHover(EInputCaptureSide::Left);
			return true;
		}
	}

	return false;
}

void UInputRouter::OnApplicationFocusChanged(bool bInIsFocused)
{
	if (!bInIsFocused)
	{
		ForceTerminateAll();
	}
}

bool UInputRouter::ProcessMouseHover(const FInputDeviceState& Input)
{
	TArray<FInputCaptureRequest> CaptureRequests;
	ActiveInputBehaviors->CollectWantsHoverCapture(Input, CaptureRequests);

	if (CaptureRequests.Num() == 0 )
	{
		if (ActiveLeftHoverCapture != nullptr)
		{
			TerminateHover(EInputCaptureSide::Left);
			return true;
		}
		return false;
	}

	UInputBehavior* PreviousCapture = ActiveLeftHoverCapture;
	CaptureRequests.StableSort();

	// if we have an active hover, either update it, or terminate if we got a new best hit
	if (CaptureRequests[0].Source == ActiveLeftHoverCapture)
	{
		UpdateExistingHoverCaptureIfPresent(Input);
		// We don't return early because the update may have ended the hover, so we may need a replacement. 
	}
	else
	{
		TerminateHover(EInputCaptureSide::Left); // does nothing if no capture present
	}

	// See if we need a new capture
	if (ActiveLeftHoverCapture == nullptr)
	{
		for (int i = 0; i < CaptureRequests.Num(); ++i)
		{
			FInputCaptureUpdate Result =
				CaptureRequests[i].Source->BeginHoverCapture(Input, EInputCaptureSide::Left);
			if (Result.State == EInputCaptureState::Begin)
			{
				ActiveLeftHoverCapture = Result.Source;
				ActiveLeftHoverCaptureOwner = CaptureRequests[i].Owner;

				// We say that the hover state has been modified, despite the fact that it's theoretically possible
				// to end up with the same ActiveLeftHoverCapture if behaviors do some unpleasant things like claim
				// that they want capture and then refuse it on BeginHoverCapture, or terminate capture in an update
				// but then accept it on BeginHoverCapture... 
				return true;
			}
		}
	}
	
	return PreviousCapture != ActiveLeftHoverCapture;
}





void UInputRouter::ForceTerminateAll()
{
	if (ActiveKeyboardCapture != nullptr)
	{
		ActiveKeyboardCapture->ForceEndCapture(ActiveKeyboardCaptureData);
		ActiveKeyboardCapture = nullptr;
		ActiveKeyboardCaptureOwner = nullptr;
		ActiveKeyboardCaptureData = FInputCaptureData();
	}

	if (ActiveLeftCapture != nullptr)
	{
		ActiveLeftCapture->ForceEndCapture(ActiveLeftCaptureData);
		ActiveLeftCapture = nullptr;
		ActiveLeftCaptureOwner = nullptr;
		ActiveLeftCaptureData = FInputCaptureData();
	}

	if (ActiveRightCapture != nullptr)
	{
		ActiveRightCapture->ForceEndCapture(ActiveRightCaptureData);
		ActiveRightCapture = nullptr;
		ActiveRightCaptureOwner = nullptr;
		ActiveRightCaptureData = FInputCaptureData();
	}

	if (ActiveLeftHoverCapture != nullptr)
	{
		TerminateHover(EInputCaptureSide::Left);
	}

	TArray<UInputBehavior*> ForceEndCaptureBehaviors;
	ActiveInputBehaviors->CollectWantsForceEndCapture(ForceEndCaptureBehaviors);
	for (UInputBehavior* const Behavior : ForceEndCaptureBehaviors)
	{
		Behavior->ForceEndCapture(FInputCaptureData());
	}
}


void UInputRouter::ForceTerminateSource(IInputBehaviorSource* Source)
{
	if (ActiveKeyboardCapture != nullptr && ActiveKeyboardCaptureOwner == Source)
	{
		ActiveKeyboardCapture->ForceEndCapture(ActiveKeyboardCaptureData);
		ActiveKeyboardCapture = nullptr;
		ActiveKeyboardCaptureOwner = nullptr;
		ActiveKeyboardCaptureData = FInputCaptureData();
	}

	if (ActiveLeftCapture != nullptr && ActiveLeftCaptureOwner == Source)
	{
		ActiveLeftCapture->ForceEndCapture(ActiveLeftCaptureData);
		ActiveLeftCapture = nullptr;
		ActiveLeftCaptureOwner = nullptr;
		ActiveLeftCaptureData = FInputCaptureData();
	}

	if (ActiveRightCapture != nullptr && ActiveRightCaptureOwner == Source)
	{
		ActiveRightCapture->ForceEndCapture(ActiveRightCaptureData);
		ActiveRightCapture = nullptr;
		ActiveRightCaptureOwner = nullptr;
		ActiveRightCaptureData = FInputCaptureData();
	}

	if (ActiveLeftHoverCapture != nullptr && ActiveLeftHoverCaptureOwner == Source)
	{
		TerminateHover(EInputCaptureSide::Left);
	}
}



#undef LOCTEXT_NAMESPACE

