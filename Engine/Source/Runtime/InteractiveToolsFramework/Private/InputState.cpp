// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputState)

namespace UE::Editor::Private
{
	FDeviceButtonState DownState(const FKey& InKey, bool bInDown)
	{
		FDeviceButtonState State(InKey);
		State.bDown = bInDown;
		return State;
	}
}

FDeviceButtonState FInputDeviceState::GetButtonState(const FKey& Key) const
{
	using namespace UE::Editor::Private;
	
	if (IsFromDevice(EInputDevices::Mouse))
	{
		if (Key == EKeys::LeftMouseButton)
		{
			return Mouse.Left;
		}
		if (Key == EKeys::MiddleMouseButton)
		{
			return Mouse.Middle;
		}
		if (Key == EKeys::RightMouseButton)
		{
			return Mouse.Right;
		}	
	}
	
	if (IsFromDevice(EInputDevices::Keyboard) && Key == Keyboard.ActiveKey.Button)
	{
		return Keyboard.ActiveKey;
	}
	
	// Treat left/right modifier keys as the same
	if (Key == EKeys::LeftControl || Key == EKeys::RightControl)
	{
		return DownState(Key, bCtrlKeyDown);
	}
	if (Key == EKeys::LeftShift || Key == EKeys::RightShift)
	{
		return DownState(Key, bShiftKeyDown);
	}
	if (Key == EKeys::LeftAlt || Key == EKeys::RightAlt)
	{
		return DownState(Key, bAltKeyDown);
	}
	if (Key == EKeys::LeftCommand || Key == EKeys::RightCommand)
	{
		return DownState(Key, bCmdKeyDown);
	}
	
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	if (OnGetButtonStateFallback.IsBound())
	{
		return OnGetButtonStateFallback.Execute(Key);
	}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	
	return FDeviceButtonState();
}
