// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractionMechanic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractionMechanic)


UInteractionMechanic::UInteractionMechanic()
{
	// undo/redo doesn't work on uproperties unless UObject is transactional
	//SetFlags(RF_Transactional);
}


void UInteractionMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	ParentTool = ParentToolIn;
}

void UInteractionMechanic::Shutdown()
{
	ParentTool = nullptr;
}


void UInteractionMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
}


void UInteractionMechanic::Tick(float DeltaTime)
{

}


UInteractiveTool* UInteractionMechanic::GetParentTool() const
{
	return ParentTool.Get();
}



void UInteractionMechanic::AddToolPropertySource(UInteractiveToolPropertySet* PropertySet)
{
	if (ParentTool.IsValid())
	{
		ParentTool->AddToolPropertySource(PropertySet);
	}
}

bool UInteractionMechanic::SetToolPropertySourceEnabled(UInteractiveToolPropertySet* PropertySet, bool bEnabled)
{
	if (ParentTool.IsValid())
	{
		return ParentTool->SetToolPropertySourceEnabled(PropertySet, bEnabled);
	}
	return false;
}
