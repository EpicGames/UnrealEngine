// Copyright Epic Games, Inc. All Rights Reserved.


#include "InteractiveGizmo.h"
#include "InteractiveGizmoManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveGizmo)


UInteractiveGizmo::UInteractiveGizmo()
{
	// Gizmos don't get saved but this isn't necessary because they are created in the transient package...
	SetFlags(RF_Transient);

	InputBehaviors = NewObject<UInputBehaviorSet>(this, TEXT("GizmoInputBehaviors"));
}

void UInteractiveGizmo::Setup()
{
}

void UInteractiveGizmo::Shutdown()
{
	InputBehaviors->RemoveAll();
}

void UInteractiveGizmo::Render(IToolsContextRenderAPI* RenderAPI)
{
}

void UInteractiveGizmo::DrawHUD( FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI )
{
}

void UInteractiveGizmo::AddInputBehavior(UInputBehavior* Behavior)
{
	InputBehaviors->Add(Behavior);
}

void UInteractiveGizmo::RemoveInputBehavior(UInputBehavior* Behavior)
{
	InputBehaviors->Remove(Behavior);
}

const UInputBehaviorSet* UInteractiveGizmo::GetInputBehaviors() const
{
	return InputBehaviors;
}



void UInteractiveGizmo::Tick(float DeltaTime)
{
}


UInteractiveGizmoManager* UInteractiveGizmo::GetGizmoManager() const
{
	UInteractiveGizmoManager* GizmoManager = Cast<UInteractiveGizmoManager>(GetOuter());
	ensure(GizmoManager != nullptr);
	return GizmoManager;
}

