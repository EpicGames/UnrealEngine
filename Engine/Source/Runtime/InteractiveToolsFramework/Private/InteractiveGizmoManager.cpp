// Copyright Epic Games, Inc. All Rights Reserved.


#include "InteractiveGizmoManager.h"
#include "InteractiveToolsContext.h"

#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/AxisAngleGizmo.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/RepositionableTransformGizmo.h"
#include "BaseGizmos/ScalableSphereGizmo.h"
#include "ContextObjectStore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveGizmoManager)

#define LOCTEXT_NAMESPACE "UInteractiveGizmoManager"


UInteractiveGizmoManager::UInteractiveGizmoManager()
{
	QueriesAPI = nullptr;
	TransactionsAPI = nullptr;
	InputRouter = nullptr;
}


void UInteractiveGizmoManager::Initialize(IToolsContextQueriesAPI* QueriesAPIIn, IToolsContextTransactionsAPI* TransactionsAPIIn, UInputRouter* InputRouterIn)
{
	this->QueriesAPI = QueriesAPIIn;
	this->TransactionsAPI = TransactionsAPIIn;
	this->InputRouter = InputRouterIn;
}


void UInteractiveGizmoManager::Shutdown()
{
	this->QueriesAPI = nullptr;

	TArray<FActiveGizmo> AllGizmos = ActiveGizmos;
	for (FActiveGizmo& ActiveGizmo : AllGizmos)
	{
		DestroyGizmo(ActiveGizmo.Gizmo);
	}
	ActiveGizmos.Reset();

	this->TransactionsAPI = nullptr;

	if (bDefaultGizmosRegistered)
	{
		DeregisterGizmoType(DefaultAxisPositionBuilderIdentifier);
		DeregisterGizmoType(DefaultPlanePositionBuilderIdentifier);
		DeregisterGizmoType(DefaultAxisAngleBuilderIdentifier);
		DeregisterGizmoType(DefaultThreeAxisTransformBuilderIdentifier);
		DeregisterGizmoType(DefaultScalableSphereBuilderIdentifier);
	}
}



void UInteractiveGizmoManager::RegisterGizmoType(const FString& Identifier, UInteractiveGizmoBuilder* Builder)
{
	if (ensure(GizmoBuilders.Contains(Identifier) == false))
	{
		GizmoBuilders.Add(Identifier, Builder);
	}
}


bool UInteractiveGizmoManager::DeregisterGizmoType(const FString& BuilderIdentifier)
{
	if (GizmoBuilders.Contains(BuilderIdentifier) == false)
	{
		DisplayMessage(
			FText::Format(LOCTEXT("DeregisterFailedMessage", "UInteractiveGizmoManager::DeregisterGizmoType: could not find requested type {0}"), FText::FromString(BuilderIdentifier) ),
			EToolMessageLevel::Internal);
		return false;
	}
	GizmoBuilders.Remove(BuilderIdentifier);
	return true;
}




UInteractiveGizmo* UInteractiveGizmoManager::CreateGizmo(const FString& BuilderIdentifier, const FString& InstanceIdentifier, void* Owner)
{
	if ( GizmoBuilders.Contains(BuilderIdentifier) == false )
	{
		DisplayMessage(
			FText::Format(LOCTEXT("CreateGizmoCannotFindFailedMessage", "UInteractiveGizmoManager::CreateGizmo: could not find requested type {0}"), FText::FromString(BuilderIdentifier) ),
			EToolMessageLevel::Internal);
		return nullptr;
	}
	UInteractiveGizmoBuilder* FoundBuilder = GizmoBuilders[BuilderIdentifier];

	// check if we have used this instance identifier
	if (InstanceIdentifier.IsEmpty() == false)
	{
		for (FActiveGizmo& ActiveGizmo : ActiveGizmos)
		{
			if (ActiveGizmo.InstanceIdentifier == InstanceIdentifier)
			{
				DisplayMessage(
					FText::Format(LOCTEXT("CreateGizmoExistsMessage", "UInteractiveGizmoManager::CreateGizmo: instance identifier {0} already in use!"), FText::FromString(InstanceIdentifier) ),
					EToolMessageLevel::Internal);
				return nullptr;
			}
		}
	}

	FToolBuilderState CurrentSceneState;
	QueriesAPI->GetCurrentSelectionState(CurrentSceneState);

	UInteractiveGizmo* NewGizmo = FoundBuilder->BuildGizmo(CurrentSceneState);
	if (NewGizmo == nullptr)
	{
		DisplayMessage(LOCTEXT("CreateGizmoReturnNullMessage", "UInteractiveGizmoManager::CreateGizmo: BuildGizmo() returned null"), EToolMessageLevel::Internal);
		return nullptr;
	}

	NewGizmo->Setup();

	// register new active input behaviors
	InputRouter->RegisterSource(NewGizmo);

	PostInvalidation();

	FActiveGizmo ActiveGizmo = { NewGizmo, BuilderIdentifier, InstanceIdentifier, Owner };
	ActiveGizmos.Add(ActiveGizmo);

	return NewGizmo;
}



bool UInteractiveGizmoManager::DestroyGizmo(UInteractiveGizmo* Gizmo)
{
	auto Pred = [Gizmo](const FActiveGizmo& ActiveGizmo) {return ActiveGizmo.Gizmo == Gizmo; };
	if (!ensure(ActiveGizmos.FindByPredicate(Pred)))
	{
		return false;
	}

	InputRouter->DeregisterSource(Gizmo);

	Gizmo->Shutdown();

	ActiveGizmos.RemoveAll(Pred);

	PostInvalidation();

	return true;
}




TArray<UInteractiveGizmo*> UInteractiveGizmoManager::FindAllGizmosOfType(const FString& BuilderIdentifier)
{
	TArray<UInteractiveGizmo*> Found;
	for (int i = 0; i < ActiveGizmos.Num(); ++i)
	{
		if (ActiveGizmos[i].BuilderIdentifier == BuilderIdentifier)
		{
			Found.Add(ActiveGizmos[i].Gizmo);
		}
	}
	return Found;
}


void UInteractiveGizmoManager::DestroyAllGizmosOfType(const FString& BuilderIdentifier)
{
	TArray<UInteractiveGizmo*> ToRemove = FindAllGizmosOfType(BuilderIdentifier);

	for (int i = 0; i < ToRemove.Num(); ++i)
	{
		DestroyGizmo(ToRemove[i]);
	}
}


void UInteractiveGizmoManager::DestroyAllGizmosByOwner(void* Owner)
{
	TArray<UInteractiveGizmo*> Found;
	for ( const FActiveGizmo& ActiveGizmo : ActiveGizmos )
	{
		if (ActiveGizmo.Owner == Owner)
		{
			Found.Add(ActiveGizmo.Gizmo);
		}
	}
	for (UInteractiveGizmo* Gizmo : Found)
	{
		DestroyGizmo(Gizmo);
	}
}



UInteractiveGizmo* UInteractiveGizmoManager::FindGizmoByInstanceIdentifier(const FString& Identifier) const
{
	for (int i = 0; i < ActiveGizmos.Num(); ++i)
	{
		if (ActiveGizmos[i].InstanceIdentifier == Identifier)
		{
			return ActiveGizmos[i].Gizmo;
		}
	}
	return nullptr;
}



void UInteractiveGizmoManager::Tick(float DeltaTime)
{
	for (FActiveGizmo& ActiveGizmo : ActiveGizmos)
	{
		ActiveGizmo.Gizmo->Tick(DeltaTime);
	}
}


void UInteractiveGizmoManager::Render(IToolsContextRenderAPI* RenderAPI)
{
	for (FActiveGizmo& ActiveGizmo : ActiveGizmos)
	{
		ActiveGizmo.Gizmo->Render(RenderAPI);
	}

}

void UInteractiveGizmoManager::DrawHUD( FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI )
{
	for (FActiveGizmo& ActiveGizmo : ActiveGizmos)
	{
		ActiveGizmo.Gizmo->DrawHUD(Canvas, RenderAPI);
	}
}

void UInteractiveGizmoManager::DisplayMessage(const FText& Message, EToolMessageLevel Level)
{
	TransactionsAPI->DisplayMessage(Message, Level);
}

void UInteractiveGizmoManager::PostInvalidation()
{
	TransactionsAPI->PostInvalidation();
}


void UInteractiveGizmoManager::BeginUndoTransaction(const FText& Description)
{
	TransactionsAPI->BeginUndoTransaction(Description);
}

void UInteractiveGizmoManager::EndUndoTransaction()
{
	TransactionsAPI->EndUndoTransaction();
}

void UInteractiveGizmoManager::CancelUndoTransaction()
{
	TransactionsAPI->CancelUndoTransaction();
}

void UInteractiveGizmoManager::EmitObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description)
{
	TransactionsAPI->AppendChange(TargetObject, MoveTemp(Change), Description );
}

UContextObjectStore* UInteractiveGizmoManager::GetContextObjectStore() const
{
	return Cast<UInteractiveToolsContext>(GetOuter())->ContextObjectStore;
}



FString UInteractiveGizmoManager::DefaultAxisPositionBuilderIdentifier = TEXT("StandardXFormAxisTranslationGizmo");
FString UInteractiveGizmoManager::DefaultPlanePositionBuilderIdentifier = TEXT("StandardXFormPlaneTranslationGizmo");
FString UInteractiveGizmoManager::DefaultAxisAngleBuilderIdentifier = TEXT("StandardXFormAxisRotationGizmo");
FString UInteractiveGizmoManager::DefaultThreeAxisTransformBuilderIdentifier = TEXT("DefaultThreeAxisTransformBuilderIdentifier");
const FString UInteractiveGizmoManager::CustomThreeAxisTransformBuilderIdentifier = TEXT("CustomThreeAxisTransformBuilderIdentifier");
const FString UInteractiveGizmoManager::CustomRepositionableThreeAxisTransformBuilderIdentifier = TEXT("CustomRepositionableThreeAxisTransformBuilderIdentifier");
FString UInteractiveGizmoManager::DefaultScalableSphereBuilderIdentifier = TEXT("DefaultScalableSphereBuilderIdentifier");

void UInteractiveGizmoManager::RegisterDefaultGizmos()
{
	check(bDefaultGizmosRegistered == false);

	UAxisPositionGizmoBuilder* AxisTranslationBuilder = NewObject<UAxisPositionGizmoBuilder>();
	RegisterGizmoType(DefaultAxisPositionBuilderIdentifier, AxisTranslationBuilder);

	UPlanePositionGizmoBuilder* PlaneTranslationBuilder = NewObject<UPlanePositionGizmoBuilder>();
	RegisterGizmoType(DefaultPlanePositionBuilderIdentifier, PlaneTranslationBuilder);

	UAxisAngleGizmoBuilder* AxisRotationBuilder = NewObject<UAxisAngleGizmoBuilder>();
	RegisterGizmoType(DefaultAxisAngleBuilderIdentifier, AxisRotationBuilder);

	UCombinedTransformGizmoBuilder* TransformBuilder = NewObject<UCombinedTransformGizmoBuilder>();
	TransformBuilder->AxisPositionBuilderIdentifier = DefaultAxisPositionBuilderIdentifier;
	TransformBuilder->PlanePositionBuilderIdentifier = DefaultPlanePositionBuilderIdentifier;
	TransformBuilder->AxisAngleBuilderIdentifier = DefaultAxisAngleBuilderIdentifier;
	RegisterGizmoType(DefaultThreeAxisTransformBuilderIdentifier, TransformBuilder);

	UGizmoViewContext* GizmoViewContext = GetContextObjectStore()->FindContext<UGizmoViewContext>();
	if (!GizmoViewContext)
	{
		GizmoViewContext = NewObject<UGizmoViewContext>();
		GetContextObjectStore()->AddContextObject(GizmoViewContext);
	}
	GizmoActorBuilder = MakeShared<FCombinedTransformGizmoActorFactory>(GizmoViewContext);

	UCombinedTransformGizmoBuilder* CustomThreeAxisBuilder = NewObject<UCombinedTransformGizmoBuilder>();
	CustomThreeAxisBuilder->AxisPositionBuilderIdentifier = DefaultAxisPositionBuilderIdentifier;
	CustomThreeAxisBuilder->PlanePositionBuilderIdentifier = DefaultPlanePositionBuilderIdentifier;
	CustomThreeAxisBuilder->AxisAngleBuilderIdentifier = DefaultAxisAngleBuilderIdentifier;
	CustomThreeAxisBuilder->GizmoActorBuilder = GizmoActorBuilder;
	RegisterGizmoType(CustomThreeAxisTransformBuilderIdentifier, CustomThreeAxisBuilder);

	URepositionableTransformGizmoBuilder* CustomRepositionableThreeAxisBuilder = NewObject<URepositionableTransformGizmoBuilder>();
	CustomRepositionableThreeAxisBuilder->AxisPositionBuilderIdentifier = DefaultAxisPositionBuilderIdentifier;
	CustomRepositionableThreeAxisBuilder->PlanePositionBuilderIdentifier = DefaultPlanePositionBuilderIdentifier;
	CustomRepositionableThreeAxisBuilder->AxisAngleBuilderIdentifier = DefaultAxisAngleBuilderIdentifier;
	CustomRepositionableThreeAxisBuilder->GizmoActorBuilder = GizmoActorBuilder;
	RegisterGizmoType(CustomRepositionableThreeAxisTransformBuilderIdentifier, CustomRepositionableThreeAxisBuilder);

	UScalableSphereGizmoBuilder* ScalableSphereBuilder = NewObject<UScalableSphereGizmoBuilder>();
	RegisterGizmoType(DefaultScalableSphereBuilderIdentifier, ScalableSphereBuilder);

	bDefaultGizmosRegistered = true;
}

UCombinedTransformGizmo* UInteractiveGizmoManager::Create3AxisTransformGizmo(void* Owner, const FString& InstanceIdentifier)
{
	check(bDefaultGizmosRegistered);
	UInteractiveGizmo* NewGizmo = CreateGizmo(DefaultThreeAxisTransformBuilderIdentifier, InstanceIdentifier, Owner);
	check(NewGizmo);
	return Cast<UCombinedTransformGizmo>(NewGizmo);
}

UCombinedTransformGizmo* UInteractiveGizmoManager::CreateCustomTransformGizmo(ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	check(bDefaultGizmosRegistered);
	GizmoActorBuilder->EnableElements = Elements;
	UInteractiveGizmo* NewGizmo = CreateGizmo(CustomThreeAxisTransformBuilderIdentifier, InstanceIdentifier, Owner);
	check(NewGizmo);
	return Cast<UCombinedTransformGizmo>(NewGizmo);
}

UCombinedTransformGizmo* UInteractiveGizmoManager::CreateCustomRepositionableTransformGizmo(ETransformGizmoSubElements Elements, void* Owner, const FString& InstanceIdentifier)
{
	check(bDefaultGizmosRegistered);
	GizmoActorBuilder->EnableElements = Elements;
	UInteractiveGizmo* NewGizmo = CreateGizmo(CustomRepositionableThreeAxisTransformBuilderIdentifier, InstanceIdentifier, Owner);
	check(NewGizmo);
	return Cast<UCombinedTransformGizmo>(NewGizmo);
}


#undef LOCTEXT_NAMESPACE

