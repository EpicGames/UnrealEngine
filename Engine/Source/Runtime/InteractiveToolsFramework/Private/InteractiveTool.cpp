// Copyright Epic Games, Inc. All Rights Reserved.


#include "InteractiveTool.h"

#if WITH_EDITOR
#include "EditorToolDelegates.h"
#endif
#include "InteractiveToolManager.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveTool)


#define LOCTEXT_NAMESPACE "UInteractiveTool"

#if WITH_EDITOR
void UInteractiveToolPropertySet::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnModified.Broadcast(this, PropertyChangedEvent.Property);
}
#endif

UInteractiveTool::UInteractiveTool()
{
	// tools need to be transactional or undo/redo won't work on their uproperties
	SetFlags(RF_Transactional);

	// tools don't get saved but this isn't necessary because they are created in the transient package...
	//SetFlags(RF_Transient);

	InputBehaviors = NewObject<UInputBehaviorSet>(this, TEXT("InputBehaviors"));

	// initialize ToolInfo
#if WITH_EDITORONLY_DATA
	DefaultToolInfo.ToolDisplayName = GetClass()->GetDisplayNameText();
#else
	DefaultToolInfo.ToolDisplayName = FText(LOCTEXT("DefaultInteractiveToolName", "DefaultToolName"));
#endif
}

void UInteractiveTool::Setup()
{
#if WITH_EDITOR
	FEditorToolDelegates::OnEditorToolStarted.Broadcast(GetClass()->GetName());
#endif
}

void UInteractiveTool::Shutdown(EToolShutdownType ShutdownType)
{
	InputBehaviors->RemoveAll();
	ToolPropertyObjects.Reset();
}

void UInteractiveTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}

void UInteractiveTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
}

void UInteractiveTool::AddInputBehavior(UInputBehavior* Behavior, void* Source)
{
	InputBehaviors->Add(Behavior, Source);
}

void UInteractiveTool::RemoveInputBehaviorsBySource(void* Source)
{
	InputBehaviors->RemoveBySource(Source);
}

const UInputBehaviorSet* UInteractiveTool::GetInputBehaviors() const
{
	return InputBehaviors;
}


void UInteractiveTool::AddToolPropertySource(UObject* PropertyObject)
{
	check(ToolPropertyObjects.Contains(PropertyObject) == false);
	ToolPropertyObjects.Add(PropertyObject);

	OnPropertySetsModified.Broadcast();
}

void UInteractiveTool::AddToolPropertySource(UInteractiveToolPropertySet* PropertySet)
{
	check(ToolPropertyObjects.Contains(PropertySet) == false);
	ToolPropertyObjects.Add(PropertySet);
	// @todo do we need to create a lambda every time for this?
	PropertySet->GetOnModified().AddLambda([this](UObject* PropertySetArg, FProperty* PropertyArg)
	{
		OnPropertyModified(PropertySetArg, PropertyArg);
	});

	OnPropertySetsModified.Broadcast();
}

bool UInteractiveTool::RemoveToolPropertySource(UInteractiveToolPropertySet* PropertySet)
{
	int32 NumRemoved = ToolPropertyObjects.Remove(PropertySet);
	if (NumRemoved == 0)
	{
		return false;
	}

	PropertySet->GetOnModified().Clear();
	OnPropertySetsModified.Broadcast();
	return true;
}

bool UInteractiveTool::ReplaceToolPropertySource(UInteractiveToolPropertySet* CurPropertySet, UInteractiveToolPropertySet* ReplaceWith, bool bSetToEnabled)
{
	int32 Index = ToolPropertyObjects.Find(CurPropertySet);
	if (Index == INDEX_NONE)
	{
		return false;
	}
	CurPropertySet->GetOnModified().Clear();

	ReplaceWith->GetOnModified().AddLambda([this](UObject* PropertySetArg, FProperty* PropertyArg)
	{
		OnPropertyModified(PropertySetArg, PropertyArg);
	});

	ToolPropertyObjects[Index] = ReplaceWith;

	if (bSetToEnabled)
	{
		ReplaceWith->bIsPropertySetEnabled = true;
	}

	OnPropertySetsModified.Broadcast();
	return true;
}

bool UInteractiveTool::SetToolPropertySourceEnabled(UInteractiveToolPropertySet* PropertySet, bool bEnabled)
{
	int32 Index = ToolPropertyObjects.Find(PropertySet);
	if (Index == INDEX_NONE)
	{
		return false;
	}
	if (PropertySet->bIsPropertySetEnabled != bEnabled)
	{
		PropertySet->bIsPropertySetEnabled = bEnabled;
		OnPropertySetsModified.Broadcast();
	}
	return true;
}

void UInteractiveTool::NotifyOfPropertyChangeByTool(UInteractiveToolPropertySet* PropertySet) const
{
	OnPropertyModifiedDirectlyByTool.Broadcast(PropertySet);
}

TArray<UObject*> UInteractiveTool::GetToolProperties(bool bEnabledOnly) const
{
	if (bEnabledOnly == false)
	{
		return ToolPropertyObjects;
	}

	TArray<UObject*> Properties;
	for (UObject* Object : ToolPropertyObjects)
	{
		UInteractiveToolPropertySet* Prop = Cast<UInteractiveToolPropertySet>(Object);
		if (Prop == nullptr || Prop->IsPropertySetEnabled())
		{
			Properties.Add(Object);
		}
	}

	return Properties;
}

void UInteractiveToolPropertySet::SaveProperties(UInteractiveTool* SaveFromTool, const FString& CacheIdentifier)
{
	SaveRestoreProperties(SaveFromTool, CacheIdentifier, true);
}

void UInteractiveToolPropertySet::RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier)
{
	SaveRestoreProperties(RestoreToTool, CacheIdentifier, false);
}

void UInteractiveToolPropertySet::SaveRestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier, bool bSaving)
{
	UInteractiveToolPropertySet* PropertyCache = bSaving ? FindOrAddDynamicPropertyCache(CacheIdentifier) : FindDynamicPropertyCache(CacheIdentifier);
	if (PropertyCache == nullptr)
	{
		return;
	}
	for ( FProperty* Prop : TFieldRange<FProperty>(GetClass()) )
	{
#if WITH_EDITOR
		if (!Prop->HasMetaData(TEXT("TransientToolProperty")))
#endif
		{
			void* DestValue = Prop->ContainerPtrToValuePtr<void>(this);
			void* SrcValue = Prop->ContainerPtrToValuePtr<void>(PropertyCache);
			if ( bSaving )
			{
				Swap(SrcValue, DestValue);
			}
			Prop->CopySingleValue(DestValue, SrcValue);
		}
	}
}


TObjectPtr<UInteractiveToolPropertySet> UInteractiveToolPropertySet::GetDynamicPropertyCache(const FString& CacheIdentifier, bool& bWasCreatedOut)
{
	bWasCreatedOut = false;
	UInteractiveToolPropertySet* CDO = GetMutableDefault<UInteractiveToolPropertySet>(GetClass());
	TObjectPtr<UInteractiveToolPropertySet>* Found = CDO->CachedPropertiesMap.Find(CacheIdentifier);
	if (Found == nullptr)
	{
		TObjectPtr<UInteractiveToolPropertySet> NewPropCache = NewObject<UInteractiveToolPropertySet>((UObject*)GetTransientPackage(), GetClass());
		ensure(NewPropCache != nullptr);
		CDO->CachedPropertiesMap.Add(CacheIdentifier, NewPropCache);
		bWasCreatedOut = true;
		return NewPropCache;
	}

	if ( ensure(*Found != nullptr) == false )
	{
		// this case seems to occur sometimes for Blueprintable Tools, uncertain why, but perhaps this ensure will help to find it
		TObjectPtr<UInteractiveToolPropertySet> NewPropCache = NewObject<UInteractiveToolPropertySet>((UObject*)GetTransientPackage(), GetClass());
		ensure(NewPropCache != nullptr);
		CDO->CachedPropertiesMap[CacheIdentifier] = NewPropCache;
		bWasCreatedOut = true;
		return NewPropCache;
	}

	return *Found;
}


TObjectPtr<UInteractiveToolPropertySet> UInteractiveToolPropertySet::FindDynamicPropertyCache(const FString& CacheIdentifier) const
{
	UInteractiveToolPropertySet* CDO = GetMutableDefault<UInteractiveToolPropertySet>(GetClass());
	TObjectPtr<UInteractiveToolPropertySet>* Found = CDO->CachedPropertiesMap.Find(CacheIdentifier);
	return Found ? *Found : nullptr;
}


TObjectPtr<UInteractiveToolPropertySet> UInteractiveToolPropertySet::FindOrAddDynamicPropertyCache(const FString& CacheIdentifier) const
{
	UInteractiveToolPropertySet* CDO = GetMutableDefault<UInteractiveToolPropertySet>(GetClass());
	TObjectPtr<UInteractiveToolPropertySet>* Found = CDO->CachedPropertiesMap.Find(CacheIdentifier);
	if (Found == nullptr)
	{
		TObjectPtr<UInteractiveToolPropertySet> NewPropCache = NewObject<UInteractiveToolPropertySet>((UObject*)GetTransientPackage(), GetClass());
		ensure(NewPropCache != nullptr);
		CDO->CachedPropertiesMap.Add(CacheIdentifier, NewPropCache);
		return NewPropCache;
	}

	if ( ensure(*Found != nullptr) == false )
	{
		// this case seems to occur sometimes for Blueprintable Tools, uncertain why, but perhaps this ensure will help to find it
		TObjectPtr<UInteractiveToolPropertySet> NewPropCache = NewObject<UInteractiveToolPropertySet>((UObject*)GetTransientPackage(), GetClass());
		ensure(NewPropCache != nullptr);
		CDO->CachedPropertiesMap[CacheIdentifier] = NewPropCache;
		return NewPropCache;
	}

	return *Found;
}


void UInteractiveTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
}

FInteractiveToolActionSet* UInteractiveTool::GetActionSet()
{
	if (ToolActionSet == nullptr)
	{
		ToolActionSet = new FInteractiveToolActionSet();
		RegisterActions(*ToolActionSet);
	}
	return ToolActionSet;
}

void UInteractiveTool::ExecuteAction(int32 ActionID)
{
	GetActionSet()->ExecuteAction(ActionID);
}



bool UInteractiveTool::HasCancel() const
{
	return false;
}

bool UInteractiveTool::HasAccept() const
{
	return false;
}

bool UInteractiveTool::CanAccept() const
{
	return false;
}


void UInteractiveTool::UpdateAcceptWarnings(EAcceptWarning Warning)
{
	switch (Warning)
	{
	case EAcceptWarning::NoWarning:
		if (bLastShowedAcceptWarning)
		{
			// clear warning
			GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
		}
		break;
	case EAcceptWarning::EmptyForbidden:
		GetToolManager()->DisplayMessage(LOCTEXT("CannotCreateEmptyMesh", "WARNING: Tool doesn't allow creation of an empty mesh."),
			EToolMessageLevel::UserWarning);
		break;
	default:
		check(false);
	}
	bLastShowedAcceptWarning = Warning != EAcceptWarning::NoWarning;
}


void UInteractiveTool::Tick(float DeltaTime)
{
	for (auto& Object : ToolPropertyObjects)
	{
		auto* Propset = Cast<UInteractiveToolPropertySet>(ToRawPtr(Object));
		if ( Propset != nullptr )
		{
			if ( Propset->IsPropertySetEnabled() )
			{
				Propset->CheckAndUpdateWatched();
			}
			else
			{
				Propset->SilentUpdateWatched();
			}
		}
	}
	OnTick(DeltaTime);
}

UInteractiveToolManager* UInteractiveTool::GetToolManager() const
{
	UInteractiveToolManager* ToolManager = Cast<UInteractiveToolManager>(GetOuter());
	check(ToolManager != nullptr);
	return ToolManager;
}


#undef LOCTEXT_NAMESPACE

