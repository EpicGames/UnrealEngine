// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractiveToolActionSet.h"
#include "InteractiveTool.h"



void FInteractiveToolActionSet::RegisterAction(UInteractiveTool* Tool, int32 ActionID,
	const FString& ActionName, const FText& ShortUIName, const FText& DescriptionText,
	EModifierKey::Type Modifiers, const FKey& ShortcutKey,
	TFunction<void()> ActionFunction,
	EToolActionKeyRepeatMode KeyRepeatMode )
{
	checkf(FindActionByID(ActionID) == nullptr, TEXT("InteractiveToolActionSet::RegisterAction: Action ID is already registered!"));

	FInteractiveToolAction NewAction;
	NewAction.ClassType = Tool->GetClass();
	NewAction.ActionID = ActionID;
	NewAction.ActionName = ActionName;
	NewAction.ShortName = ShortUIName;
	NewAction.Description = DescriptionText;
	NewAction.DefaultModifiers = Modifiers;
	NewAction.DefaultKey = ShortcutKey;
	NewAction.KeyRepeatMode = KeyRepeatMode;
	NewAction.OnAction = ActionFunction;

	Actions.Add(NewAction);
}


const FInteractiveToolAction* FInteractiveToolActionSet::FindActionByID(int32 ActionID) const
{
	for (const FInteractiveToolAction& Action : Actions)
	{
		if (Action.ActionID == ActionID)
		{
			return &Action;
		}
	}
	return nullptr;
}



void FInteractiveToolActionSet::CollectActions(TArray<FInteractiveToolAction>& OutActions) const
{
	for (const FInteractiveToolAction& Action : Actions)
	{
		OutActions.Add(Action);
	}
}


void FInteractiveToolActionSet::ExecuteAction(int32 ActionID) const
{
	const FInteractiveToolAction* Found = FindActionByID(ActionID);
	if (Found != nullptr)
	{
		Found->OnAction();
	}
}