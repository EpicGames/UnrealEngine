// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractiveToolManager.h"

#include "Engine/Engine.h"
#include "InteractiveToolsContext.h"
#include "InteractiveToolQueryInterfaces.h"
#include "ContextObjectStore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveToolManager)

#define LOCTEXT_NAMESPACE "UInteractiveToolManager"


UInteractiveToolManager::UInteractiveToolManager()
{
	QueriesAPI = nullptr;
	TransactionsAPI = nullptr;
	InputRouter = nullptr;

	ActiveLeftBuilder = nullptr;
	ActiveLeftTool = nullptr;

	ActiveRightBuilder = nullptr;
	ActiveRightTool = nullptr;

	ActiveToolChangeTrackingMode = EToolChangeTrackingMode::UndoToExit;
}


void UInteractiveToolManager::Initialize(IToolsContextQueriesAPI* queriesAPI, IToolsContextTransactionsAPI* transactionsAPI, UInputRouter* InputRouterIn)
{
	this->QueriesAPI = queriesAPI;
	this->TransactionsAPI = transactionsAPI;
	this->InputRouter = InputRouterIn;

	bIsActive = true;
}


void UInteractiveToolManager::Shutdown()
{
	this->QueriesAPI = nullptr;

	if (ActiveLeftTool != nullptr)
	{
		DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
	}
	if (ActiveRightTool != nullptr)
	{
		DeactivateTool(EToolSide::Right, EToolShutdownType::Cancel);
	}

	this->TransactionsAPI = nullptr;

	bIsActive = false;
}

void UInteractiveToolManager::DoPostBuild(EToolSide Side, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& InBuilderState)
{
	InToolBuilder->PostBuildTool(InBuiltTool, InBuilderState);
	OnToolPostBuild.Broadcast(this, Side, InBuiltTool, InToolBuilder, InBuilderState);
}

void UInteractiveToolManager::DoPostSetup(EToolSide Side, UInteractiveTool* InInteractiveTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& InBuilderState)
{
	InToolBuilder->PostSetupTool(InInteractiveTool, InBuilderState);
	OnToolPostSetup.Broadcast(this, Side, InInteractiveTool);
}


void UInteractiveToolManager::RegisterToolType(const FString& Identifier, UInteractiveToolBuilder* Builder)
{
	if (ensure(ToolBuilders.Contains(Identifier) == false))
	{
		ToolBuilders.Add(Identifier, Builder);
	}
}

void UInteractiveToolManager::UnregisterToolType(const FString& Identifier)
{
	if (ActiveLeftToolName == Identifier)
	{
		DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
	}
	if (ActiveRightToolName == Identifier)
	{
		DeactivateTool(EToolSide::Right, EToolShutdownType::Cancel);
	}
	ToolBuilders.Remove(Identifier);
}


bool UInteractiveToolManager::SelectActiveToolType(EToolSide Side, const FString& Identifier)
{
	if (ToolBuilders.Contains(Identifier))
	{
		UInteractiveToolBuilder* Builder = ToolBuilders[Identifier];
		if (Side == EToolSide::Right)
		{
			ActiveRightBuilder = Builder;
			ActiveRightBuilderName = Identifier;
		}
		else
		{
			ActiveLeftBuilder = Builder;
			ActiveLeftBuilderName = Identifier;
		}
		return true;
	}
	return false;
}


const FString& UInteractiveToolManager::GetActiveToolType(EToolSide Side) const
{
	return Side == EToolSide::Right ? ActiveRightBuilderName : ActiveLeftBuilderName;
}


bool UInteractiveToolManager::CanActivateTool(EToolSide Side, FString Identifier)
{
	if (ensure(Side == EToolSide::Left) == false)  // TODO: support right-side tool
	{
		return false;
	}

	if (ToolBuilders.Contains(Identifier))
	{
		FToolBuilderState InputState;
		QueriesAPI->GetCurrentSelectionState(InputState);

		UInteractiveToolBuilder* Builder = ToolBuilders[Identifier];
		return Builder->CanBuildTool(InputState);
	}

	return false;
}


bool UInteractiveToolManager::ActivateTool(EToolSide Side)
{
	if (ensure(Side == EToolSide::Left) == false)  // TODO: support right-side tool
	{
		return false;
	}

	// wrap tool change in a transaction so that deactivate and activate are grouped
	bool bInTransaction = false;
	if (ActiveToolChangeTrackingMode == EToolChangeTrackingMode::FullUndoRedo)
	{
		BeginUndoTransaction(LOCTEXT("ToolChange", "Change Tool"));
		bInTransaction = true;
	}

	if (ActiveLeftTool != nullptr)
	{
		DeactivateTool(EToolSide::Left, GetPreferredShutdownType(EToolSide::Left, EToolShutdownReason::SwitchTool));
	}

	if (ActiveLeftBuilder == nullptr || ActivateToolInternal(Side) == false)
	{
		if (bInTransaction)
		{
			EndUndoTransaction();
		}
		return false;
	}

	if (ActiveToolChangeTrackingMode == EToolChangeTrackingMode::FullUndoRedo)
	{
		if (ensure(TransactionsAPI))
		{
			TransactionsAPI->AppendChange(this, MakeUnique<FActivateToolChange>(Side, ActiveLeftToolName), LOCTEXT("ActivateToolChange", "Activate Tool"));
		}
	} 
	else if (ActiveToolChangeTrackingMode == EToolChangeTrackingMode::UndoToExit)
	{
		EmitObjectChange(this, MakeUnique<FBeginToolChange>(), LOCTEXT("ActivateToolChange", "Activate Tool"));
	}

	if (bInTransaction)
	{
		EndUndoTransaction();
	}

	return true;
}


bool UInteractiveToolManager::ActivateToolInternal(EToolSide Side)
{
	// construct input state we will pass to tools
	FToolBuilderState InputState;
	QueriesAPI->GetCurrentSelectionState(InputState);

	if (ActiveLeftBuilder->CanBuildTool(InputState) == false)
	{
		TransactionsAPI->DisplayMessage(LOCTEXT("ActivateToolCanBuildFailMessage", "UInteractiveToolManager::ActivateTool: CanBuildTool returned false."), EToolMessageLevel::Internal);
		return false;
	}

	ActiveLeftTool = ActiveLeftBuilder->BuildTool(InputState);
	if (ActiveLeftTool == nullptr)
	{
		return false;
	}
	ActiveLeftToolName = ActiveLeftBuilderName;
	DoPostBuild(Side, ActiveLeftTool, ActiveLeftBuilder, InputState);

	bInToolSetup = true;
	ActiveLeftTool->Setup();
	bInToolSetup = false;
	if (bToolRequestedTerminationDuringSetup)
	{
		bToolRequestedTerminationDuringSetup = false;
		ActiveLeftTool->Shutdown(EToolShutdownType::Cancel);		// need to give Tool a chance to clean up...
		ActiveLeftTool = nullptr;
		ActiveLeftToolName.Empty();
		return false;
	}

	DoPostSetup(Side, ActiveLeftTool, ActiveLeftBuilder, InputState);

	// register new active input behaviors
	InputRouter->RegisterSource(ActiveLeftTool);

	PostInvalidation();

	OnToolStarted.Broadcast(this, ActiveLeftTool);

	return true;
}



void UInteractiveToolManager::DeactivateTool(EToolSide Side, EToolShutdownType ShutdownType)
{
	if (ensure(Side == EToolSide::Left) == false)  // TODO: support right-side tool
	{
		return;
	}
	if (bInToolShutdown)
	{
		return;
	}

	if (ActiveLeftTool != nullptr)
	{
		if (ActiveToolChangeTrackingMode == EToolChangeTrackingMode::FullUndoRedo)
		{
			if (ensure(TransactionsAPI))
			{
				TransactionsAPI->AppendChange(this, MakeUnique<FActivateToolChange>(Side, ActiveLeftToolName, ShutdownType), LOCTEXT("DeactivateToolChange", "Deactivate Tool"));
			}
		}

		DeactivateToolInternal(Side, ShutdownType);
	}
}

EToolShutdownType UInteractiveToolManager::GetPreferredShutdownType(EToolSide Side, EToolShutdownReason ShutdownReason)
{
	// Decide how to shut down the currently active tool based on ToolSwitchMode
	EToolShutdownType ShutdownType = EToolShutdownType::Completed;
	switch (ToolSwitchMode)
	{
	case EToolManagerToolSwitchMode::AcceptIfAble:
	case EToolManagerToolSwitchMode::CustomizableAcceptIfAble:
		ShutdownType = 
			CanAcceptActiveTool(Side) ? EToolShutdownType::Accept : 
			CanCancelActiveTool(Side) ? EToolShutdownType::Cancel : EToolShutdownType::Completed;
		break;
	case EToolManagerToolSwitchMode::CancelIfAble:
	case EToolManagerToolSwitchMode::CustomizableCancelIfAble:
		ShutdownType = CanCancelActiveTool(Side) ? EToolShutdownType::Cancel : EToolShutdownType::Completed;
		break;
	default:
		ensure(false);
	}

	// If the tool switch mode is set to be customizable, see if the tool wants to customize it
	if (ToolSwitchMode == EToolManagerToolSwitchMode::CustomizableAcceptIfAble
		|| ToolSwitchMode == EToolManagerToolSwitchMode::CustomizableCancelIfAble)
	{
		if (IInteractiveToolShutdownQueryAPI* ShutdownQueryAPI = Cast<IInteractiveToolShutdownQueryAPI>(ActiveLeftTool))
		{
			ShutdownType = ShutdownQueryAPI->GetPreferredShutdownType(
				ShutdownReason, ShutdownType);
		}
	}
	
	return ShutdownType;
}

void UInteractiveToolManager::DeactivateToolInternal(EToolSide Side, EToolShutdownType ShutdownType)
{
	if (Side == EToolSide::Left)
	{
		if (!ensure(ActiveLeftTool))
		{
			return;
		}

		bInToolShutdown = true;

		// Note that the deregister is done because a tool that generates a modal window during Shutdown
		// could otherwise re-trigger input recapture, since modals continue pumping input messages.
		InputRouter->DeregisterSource(ActiveLeftTool);
		
		ActiveLeftTool->Shutdown(ShutdownType);


		UInteractiveTool* DoneTool = ActiveLeftTool;
		ActiveLeftTool = nullptr;
		ActiveLeftToolName.Empty();

		PostInvalidation();

		OnToolEndedWithStatus.Broadcast(this, DoneTool, ShutdownType);
		OnToolEnded.Broadcast(this, DoneTool);

		bInToolShutdown = false;
	}
}



bool UInteractiveToolManager::HasActiveTool(EToolSide Side) const
{
	return (Side == EToolSide::Left) ? (ActiveLeftTool != nullptr) : (ActiveRightTool != nullptr);
}

bool UInteractiveToolManager::HasAnyActiveTool() const
{
	return ActiveLeftTool != nullptr || ActiveRightTool != nullptr;
}


UInteractiveTool* UInteractiveToolManager::GetActiveTool(EToolSide Side)
{
	return (Side == EToolSide::Left) ? ActiveLeftTool : ActiveRightTool;
}

UInteractiveToolBuilder* UInteractiveToolManager::GetActiveToolBuilder(EToolSide Side)
{
	return (Side == EToolSide::Left) ? ActiveLeftBuilder : ActiveRightBuilder;
}

FString UInteractiveToolManager::GetActiveToolName(EToolSide Side)
{
	if (GetActiveTool(Side) == nullptr)
	{
		return FString();
	}
	return (Side == EToolSide::Left) ? ActiveLeftToolName : ActiveRightToolName;
}


bool UInteractiveToolManager::CanAcceptActiveTool(EToolSide Side)
{
	if (ActiveLeftTool != nullptr)
	{
		return ActiveLeftTool->HasAccept() && ActiveLeftTool->CanAccept();
	}
	return false;
}

bool UInteractiveToolManager::CanCancelActiveTool(EToolSide Side)
{
	if (ActiveLeftTool != nullptr)
	{
		return ActiveLeftTool->HasCancel();
	}
	return false;
}



void UInteractiveToolManager::ConfigureChangeTrackingMode(EToolChangeTrackingMode ChangeMode)
{
	ActiveToolChangeTrackingMode = ChangeMode;
}

EToolChangeTrackingMode UInteractiveToolManager::GetChangeTrackingMode() const
{
	return ActiveToolChangeTrackingMode;
}


void UInteractiveToolManager::Tick(float DeltaTime)
{
	if (ActiveLeftTool != nullptr)
	{
		ActiveLeftTool->Tick(DeltaTime);
	}

	if (ActiveRightTool!= nullptr)
	{
		ActiveRightTool->Tick(DeltaTime);
	}
}


void UInteractiveToolManager::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (ActiveLeftTool != nullptr)
	{
		ActiveLeftTool->Render(RenderAPI);
	}

	if (ActiveRightTool != nullptr)
	{
		ActiveRightTool->Render(RenderAPI);
	}
}

void UInteractiveToolManager::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (ActiveLeftTool != nullptr)
	{
		ActiveLeftTool->DrawHUD(Canvas, RenderAPI);
	}

	if (ActiveRightTool != nullptr)
	{
		ActiveRightTool->DrawHUD(Canvas, RenderAPI);
	}
}



UInteractiveGizmoManager* UInteractiveToolManager::GetPairedGizmoManager()
{
	return Cast<UInteractiveToolsContext>(GetOuter())->GizmoManager;
}

UContextObjectStore* UInteractiveToolManager::GetContextObjectStore() const
{
	return Cast<UInteractiveToolsContext>(GetOuter())->ContextObjectStore;
}

void UInteractiveToolManager::DisplayMessage(const FText& Message, EToolMessageLevel Level)
{
	TransactionsAPI->DisplayMessage(Message, Level);
}

void UInteractiveToolManager::PostInvalidation()
{
	TransactionsAPI->PostInvalidation();
}


void UInteractiveToolManager::BeginUndoTransaction(const FText& Description)
{
	TransactionsAPI->BeginUndoTransaction(Description);
}

void UInteractiveToolManager::EndUndoTransaction()
{
	TransactionsAPI->EndUndoTransaction();
}

void UInteractiveToolManager::CancelUndoTransaction()
{
	TransactionsAPI->CancelUndoTransaction();
}

void UInteractiveToolManager::EmitObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description)
{
	// Currently we do not support undo/redo of changes emitted /during/ a Tool after the Tool exits, 
	// because we do not support "undo back into a Tool". So once a Tool exits, any Changes it emitted are considered invalid.
	// We do not assume we can modify the external history/transaction system (because in the Editor we cannot)
	// so all we can do is mark those changes as "expired" and hope that the external transaction system respects that.
	// The Tools themselves will not be around to know whether they have exited, so we wrap
	// the changes they emit in a FToolChangeWrapperChange which will allow the ToolManager
	// to decide whether the change is still valid/applicable.

	// Note: if you do not want this wrapping behavior for particular changes, you can use the TransactionsAPI directly 
	// via GetContextTransactionsAPI()

	// If you hit this ensure from a Gizmo, you may have accidentally registered the ToolManager as the Gizmo's TransactionProvider,
	// this should only be done if the Gizmo is embedded inside a Tool
	if (ensure(HasActiveTool(EToolSide::Left)))
	{
		TUniquePtr<FToolChangeWrapperChange> Wrapper = MakeUnique<FToolChangeWrapperChange>();
		Wrapper->ToolManager = this;
		Wrapper->ActiveTool = GetActiveTool(EToolSide::Left);
		Wrapper->ToolChange = MoveTemp(Change);

		TransactionsAPI->AppendChange(TargetObject, MoveTemp(Wrapper), Description);
	}
}

bool UInteractiveToolManager::RequestSelectionChange(const FSelectedObjectsChangeList& SelectionChange)
{
	return TransactionsAPI->RequestSelectionChange(SelectionChange);
}


bool UInteractiveToolManager::PostActiveToolShutdownRequest(UInteractiveTool* Tool, EToolShutdownType ShutdownType,
	bool bShowUnexpectedShutdownMessage, const FText& UnexpectedShutdownMessage)
{
	if (bShowUnexpectedShutdownMessage)
	{
		if (OnToolUnexpectedShutdownMessage.IsBound() == false )
		{
			if ( UnexpectedShutdownMessage.IsEmpty() )
			{
				FString ToolName = (Tool != nullptr) ? Tool->GetToolInfo().ToolDisplayName.ToString() : FString("(Null Tool)");
				if ( bInToolSetup )
				{
					UE_LOGF(LogTemp, Error, "[InteractiveToolManager] Tool %ls Could not be Initialized", *ToolName);
				}
				else
				{
					UE_LOGF(LogTemp, Error, "[InteractiveToolManager] Tool %ls was Shut Down Automatically, no message provided", *ToolName);
				}
			}
			else
			{
				UE_LOGF(LogTemp, Error, "[InteractiveToolManager] %ls", *UnexpectedShutdownMessage.ToString());
			}
		}
		else
		{
			OnToolUnexpectedShutdownMessage.Broadcast(this, Tool, UnexpectedShutdownMessage, bInToolSetup);
		}
	}

	// if Tool calls this function from it's Setup() function, then we need to do some special-case handling because
	// the code below can't be run yet
	if (bInToolSetup)
	{
		bToolRequestedTerminationDuringSetup = true;
		return true;
	}

	bool bIsActiveTool = (Tool != nullptr) && (ActiveLeftTool == Tool || ActiveRightTool == Tool);
	if (!bIsActiveTool)
	{
		return false;
	}

	bool bHandled = false;
	if (OnToolShutdownRequest.IsBound())
	{
		bHandled = OnToolShutdownRequest.Execute(this, Tool, ShutdownType);
	}
	if (!bHandled)
	{
		EToolSide WhichSide = (ActiveLeftTool == Tool) ? EToolSide::Left : EToolSide::Right;
		DeactivateTool(WhichSide, ShutdownType);
	}
	return true;
}





void FBeginToolChange::Apply(UObject* Object)
{
	// do nothing on apply, we do not want to re-enter the tool
}

void FBeginToolChange::Revert(UObject* Object)
{
	// On revert, if a tool is active, we cancel it.
	// Note that this should only happen once, because any further tool activations
	// would be pushing their own FBeginToolChange
	UInteractiveToolManager* ToolManager = CastChecked<UInteractiveToolManager>(Object);
	if (ToolManager->HasAnyActiveTool())
	{
		ToolManager->DeactivateToolInternal(EToolSide::Left, EToolShutdownType::Cancel);
	}
}

bool FBeginToolChange::HasExpired( UObject* Object ) const
{
	UInteractiveToolManager* ToolManager = CastChecked<UInteractiveToolManager>(Object);
	return (ToolManager == nullptr) || (ToolManager->IsActive() == false) || (ToolManager->HasAnyActiveTool() == false);
}

FString FBeginToolChange::ToString() const
{
	return FString("Begin Tool");
}





void FActivateToolChange::Apply(UObject* Object)
{
	UInteractiveToolManager* ToolManager = CastChecked<UInteractiveToolManager>(Object);
	if (ToolManager)
	{
		if (bIsDeactivate)
		{
			ToolManager->DeactivateToolInternal(Side, ShutdownType);
		}
		else
		{
			ToolManager->SelectActiveToolType(Side, ToolType);
			ToolManager->ActivateToolInternal(Side);
		}
	}
}

void FActivateToolChange::Revert(UObject* Object)
{
	UInteractiveToolManager* ToolManager = CastChecked<UInteractiveToolManager>(Object);
	if (ToolManager)
	{
		if (bIsDeactivate)
		{
			ToolManager->SelectActiveToolType(Side, ToolType);
			ToolManager->ActivateToolInternal(Side);
		}
		else
		{
			ToolManager->DeactivateToolInternal(Side, ShutdownType);
		}
	}
}

bool FActivateToolChange::HasExpired(UObject* Object) const
{
	UInteractiveToolManager* ToolManager = CastChecked<UInteractiveToolManager>(Object);
	return (ToolManager == nullptr) || (ToolManager->IsActive() == false);
}

FString FActivateToolChange::ToString() const
{
	return FString("Change Tool");
}







void FToolChangeWrapperChange::Apply(UObject* Object)
{
	if (ToolChange.IsValid())
	{
		ToolChange->Apply(Object);
	}
}

void FToolChangeWrapperChange::Revert(UObject* Object)
{
	if (ToolChange.IsValid())
	{
		ToolChange->Revert(Object);
	}
}

bool FToolChangeWrapperChange::HasExpired(UObject* Object) const
{
	if (ToolChange.IsValid() && ToolManager.IsValid() && ActiveTool.IsValid())
	{
		if (ToolChange->HasExpired(Object))
		{
			return true;
		}
		if (ToolManager->GetActiveTool(EToolSide::Left) == ActiveTool.Get())
		{
			return false;
		}
	}
	return true;
}

FString FToolChangeWrapperChange::ToString() const
{
	if (ToolChange.IsValid())
	{
		return ToolChange->ToString();
	}
	return FString();
}




#undef LOCTEXT_NAMESPACE

