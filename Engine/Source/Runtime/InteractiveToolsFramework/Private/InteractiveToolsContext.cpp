// Copyright Epic Games, Inc. All Rights Reserved.


#include "InteractiveToolsContext.h"
#include "ToolTargetManager.h"
#include "ContextObjectStore.h"
#include "ToolGroupManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InteractiveToolsContext)

UInteractiveToolsContext::UInteractiveToolsContext()
{
	InputRouter = nullptr;
	ToolManager = nullptr;
	GizmoManager = nullptr;
	TargetManager = nullptr;
	ToolGroupManager = nullptr;
	ContextObjectStore = nullptr;
	ToolManagerClass = UInteractiveToolManager::StaticClass();

	SetCreateInputRouterFunc([](const FContextInitInfo& ContextInfo)
	{
		UInputRouter* NewInputRouter = NewObject<UInputRouter>(ContextInfo.ToolsContext);
		NewInputRouter->Initialize(ContextInfo.TransactionsAPI);
		return NewInputRouter;
	});
	SetShutdownInputRouterFunc([](UInputRouter* InputRouterIn)
	{
		InputRouterIn->ForceTerminateAll();
		InputRouterIn->Shutdown();
	});

	SetCreateToolManagerFunc([this](const FContextInitInfo& ContextInfo)
	{
		UInteractiveToolManager* NewToolManager = NewObject<UInteractiveToolManager>(ContextInfo.ToolsContext, this->ToolManagerClass.Get());
		NewToolManager->Initialize(ContextInfo.QueriesAPI, ContextInfo.TransactionsAPI, ContextInfo.InputRouter);
		return NewToolManager;
	});
	SetShutdownToolManagerFunc([](UInteractiveToolManager* ToolManagerIn)
	{
		ToolManagerIn->Shutdown();
	});

	SetCreateToolTargetManagerFunc([](const FContextInitInfo& ContextInfo)
	{
		UToolTargetManager* NewTargetManager = NewObject<UToolTargetManager>(ContextInfo.ToolsContext);
		NewTargetManager->Initialize();
		return NewTargetManager;
	});
	SetShutdownToolTargetManagerFunc([](UToolTargetManager* TargetManagerIn)
	{
	});

	SetCreateGizmoManagerFunc([](const FContextInitInfo& ContextInfo)
	{
		UInteractiveGizmoManager* NewGizmoManager = NewObject<UInteractiveGizmoManager>(ContextInfo.ToolsContext);
		NewGizmoManager->Initialize(ContextInfo.QueriesAPI, ContextInfo.TransactionsAPI, ContextInfo.InputRouter);
		NewGizmoManager->RegisterDefaultGizmos();
		return NewGizmoManager;
	});
	SetShutdownGizmoManagerFunc([](UInteractiveGizmoManager* GizmoManagerIn)
	{
		GizmoManagerIn->Shutdown();
	});

	SetCreateContextStoreFunc([](const FContextInitInfo& ContextInfo)
	{
		UContextObjectStore* NewContextStore = NewObject<UContextObjectStore>(ContextInfo.ToolsContext);
		return NewContextStore;
	});
	SetShutdownContextStoreFunc([](UContextObjectStore* ContextStoreIn)
	{
		ContextStoreIn->Shutdown();
	});

	SetCreateToolGroupManagerFunc(  [](const FContextInitInfo& ContextInfo)
	{
		UToolGroupManager* NewToolGroupManager = NewObject<UToolGroupManager>(ContextInfo.ToolsContext);
		return NewToolGroupManager;
	});

	SetShutdownToolGroupManagerFunc( [] (UToolGroupManager* ToolGroupManagerIn)
	{
		ToolGroupManagerIn->Shutdown();	
	});
}


void UInteractiveToolsContext::SetCreateInputRouterFunc(TUniqueFunction<UInputRouter* (const FContextInitInfo&)> Func)
{
	CreateInputRouterFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetCreateToolManagerFunc(TUniqueFunction<UInteractiveToolManager* (const FContextInitInfo&)> Func)
{
	CreateToolManagerFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetCreateToolTargetManagerFunc(TUniqueFunction<UToolTargetManager* (const FContextInitInfo&)> Func)
{
	CreateToolTargetManagerFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetCreateGizmoManagerFunc(TUniqueFunction<UInteractiveGizmoManager* (const FContextInitInfo&)> Func)
{
	CreateGizmoManagerFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetCreateContextStoreFunc(TUniqueFunction<UContextObjectStore* (const FContextInitInfo&)> Func)
{
	CreateContextStoreFunc = MoveTemp(Func);
}

void UInteractiveToolsContext::SetCreateToolGroupManagerFunc(TUniqueFunction<UToolGroupManager*(const FContextInitInfo&)> Func)
{
	CreateToolGroupManagerFunc = MoveTemp(Func);
}


void UInteractiveToolsContext::SetShutdownInputRouterFunc(TUniqueFunction<void(UInputRouter*)> Func)
{
	ShutdownInputRouterFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetShutdownToolManagerFunc(TUniqueFunction<void(UInteractiveToolManager*)> Func)
{
	ShutdownToolManagerFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetShutdownToolTargetManagerFunc(TUniqueFunction<void(UToolTargetManager*)> Func)
{
	ShutdownToolTargetManagerFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetShutdownGizmoManagerFunc(TUniqueFunction<void(UInteractiveGizmoManager*)> Func)
{
	ShutdownGizmoManagerFunc = MoveTemp(Func);
}
void UInteractiveToolsContext::SetShutdownContextStoreFunc(TUniqueFunction<void(UContextObjectStore*)> Func)
{
	ShutdownContextStoreFunc = MoveTemp(Func);
}

void UInteractiveToolsContext::SetShutdownToolGroupManagerFunc(TUniqueFunction<void(UToolGroupManager*)> Func)
{
	ShutdownToolGroupManagerFunc = MoveTemp(Func);
}


void UInteractiveToolsContext::Initialize(IToolsContextQueriesAPI* QueriesAPI, IToolsContextTransactionsAPI* TransactionsAPI)
{
	FContextInitInfo InitInfo;
	InitInfo.ToolsContext = this;
	InitInfo.QueriesAPI = QueriesAPI;
	InitInfo.TransactionsAPI = TransactionsAPI;

	InputRouter = CreateInputRouterFunc(InitInfo);
	InitInfo.InputRouter = InputRouter;

	// Context store needs to be built before managers in case they want to use it
	ContextObjectStore = CreateContextStoreFunc(InitInfo);
	ToolManager = CreateToolManagerFunc(InitInfo);
	TargetManager = CreateToolTargetManagerFunc(InitInfo);
	GizmoManager = CreateGizmoManagerFunc(InitInfo);
	ToolGroupManager = CreateToolGroupManagerFunc(InitInfo);
}


void UInteractiveToolsContext::Shutdown()
{
	ShutdownToolGroupManagerFunc(ToolGroupManager);
	ToolGroupManager = nullptr;
	
	ShutdownInputRouterFunc(InputRouter);
	InputRouter = nullptr;

	ShutdownGizmoManagerFunc(GizmoManager);
	GizmoManager = nullptr;

	ShutdownToolManagerFunc(ToolManager);
	ToolManager = nullptr;

	ShutdownToolTargetManagerFunc(TargetManager);
	TargetManager = nullptr;

	ShutdownContextStoreFunc(ContextObjectStore);
	ContextObjectStore = nullptr;
}


void UInteractiveToolsContext::DeactivateActiveTool(EToolSide WhichSide, EToolShutdownType ShutdownType)
{
	ToolManager->DeactivateTool(WhichSide, ShutdownType);
}

void UInteractiveToolsContext::DeactivateAllActiveTools(EToolShutdownType ShutdownType)
{
	if (!ToolManager)
	{
		return;
	}
	auto DeactivateTool = [this, ShutdownType](EToolSide WhichSide) {
		if (ToolManager->HasActiveTool(WhichSide))
		{
			EToolShutdownType ShutdownTypeToUse = ShutdownType;

			// We do not allow passing an "accept" shutdown type if the tool says that it cannot accept.
			// For complete-style tools, which can never accept, it should theoretically not matter what
			// we pass because we would like them to respond the same way to all three shutdown types.
			// However, we currently have one complete-style tool that responds differently to cancel, and we
			// can't change that in a hotfix. So, we pass "complete" to complete-style tools.
			if (ShutdownType != EToolShutdownType::Cancel)
			{
				if (!ToolManager->GetActiveTool(WhichSide)->HasAccept())
				{
					ShutdownTypeToUse = EToolShutdownType::Completed;
				}
				else if (ToolManager->CanAcceptActiveTool(WhichSide))
				{
					ShutdownTypeToUse = EToolShutdownType::Accept;
				}
				else
				{
					ShutdownTypeToUse = EToolShutdownType::Cancel;
				}
			}

			ToolManager->DeactivateTool(WhichSide, ShutdownTypeToUse);
		}
	};
	
	DeactivateTool(EToolSide::Left);
	DeactivateTool(EToolSide::Right);
}


bool UInteractiveToolsContext::CanStartTool(EToolSide WhichSide, const FString& ToolTypeIdentifier) const
{
	return ToolManager->CanActivateTool(WhichSide, ToolTypeIdentifier);
}

bool UInteractiveToolsContext::HasActiveTool(EToolSide WhichSide) const
{
	return ToolManager->HasActiveTool(WhichSide);
}

FString UInteractiveToolsContext::GetActiveToolName(EToolSide WhichSide) const
{
	return ToolManager->GetActiveToolName(WhichSide);
}

bool UInteractiveToolsContext::ActiveToolHasAccept(EToolSide WhichSide) const
{
	return ToolManager->HasActiveTool(WhichSide) &&
		ToolManager->GetActiveTool(WhichSide)->HasAccept();
}

bool UInteractiveToolsContext::CanAcceptActiveTool(EToolSide WhichSide) const
{
	return ToolManager->CanAcceptActiveTool(WhichSide);
}

bool UInteractiveToolsContext::CanCancelActiveTool(EToolSide WhichSide) const
{
	return ToolManager->CanCancelActiveTool(WhichSide);
}

bool UInteractiveToolsContext::CanCompleteActiveTool(EToolSide WhichSide) const
{
	return ToolManager->HasActiveTool(WhichSide) && CanCancelActiveTool(WhichSide) == false;
}

bool UInteractiveToolsContext::StartTool(EToolSide WhichSide, const FString& ToolTypeIdentifier)
{
	if (ToolManager->SelectActiveToolType(WhichSide, ToolTypeIdentifier) == false)
	{
		UE_LOGF(LogTemp, Warning, "ToolManager: Unknown Tool Type %ls", *ToolTypeIdentifier);
		return false;
	}
	else
	{
		ToolManager->ActivateTool(WhichSide);
		return true;
	}
}

void UInteractiveToolsContext::EndTool(EToolSide WhichSide, EToolShutdownType ShutdownType)
{
	DeactivateActiveTool(WhichSide, ShutdownType);
}

// Note: this takes FString by value so that it can be bound to delegates using CreateUObject. If you change it
// to a reference, you'll need to use CreateWeakLambda instead and capture ToolIdentifier by value there.
bool UInteractiveToolsContext::IsToolActive(EToolSide WhichSide, const FString ToolIdentifier) const
{
	return GetActiveToolName(WhichSide) == ToolIdentifier;
}

void UInteractiveToolsContext::PostToolNotificationMessage(const FText& Message)
{
	OnToolNotificationMessage.Broadcast(Message);
}

void UInteractiveToolsContext::PostToolWarningMessage(const FText& Message)
{
	OnToolWarningMessage.Broadcast(Message);
}

