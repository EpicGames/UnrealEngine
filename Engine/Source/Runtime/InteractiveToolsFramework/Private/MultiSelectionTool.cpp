// Copyright Epic Games, Inc. All Rights Reserved.
#include "MultiSelectionTool.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiSelectionTool)


bool UMultiSelectionTool::SupportsWorldSpaceFocusBox()
{
	int32 PrimitiveCount = 0;
	for (const TObjectPtr<UToolTarget>& Target : Targets)
	{
		if (Cast<ISceneComponentBackedTarget>(Target) != nullptr)
		{
			PrimitiveCount++;
		}
	}
	return PrimitiveCount > 0;
}


FBox UMultiSelectionTool::GetWorldSpaceFocusBox()
{
	FBox AccumBox(EForceInit::ForceInit);
	for (const TObjectPtr<UToolTarget>& Target : Targets)
	{
		ISceneComponentBackedTarget* PrimTarget = Cast<ISceneComponentBackedTarget>(Target);
		if (PrimTarget)
		{
			USceneComponent* Component = PrimTarget->GetOwnerSceneComponent();
			if (Component)
			{
				FBox ComponentBounds = Component->Bounds.GetBox();
				AccumBox += ComponentBounds;
			}
		}
	}
	return AccumBox;
}


bool UMultiSelectionTool::SupportsWorldSpaceFocusPoint()
{
	int32 PrimitiveCount = 0;
	for (const TObjectPtr<UToolTarget>& Target : Targets)
	{
		if (Cast<IPrimitiveComponentBackedTarget>(Target) != nullptr)
		{
			PrimitiveCount++;
		}
	}
	return PrimitiveCount > 0;
}

bool UMultiSelectionTool::GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut)
{
	double NearestRayParam = (double)HALF_WORLD_MAX;
	PointOut = FVector::ZeroVector;

	for (const TObjectPtr<UToolTarget>& Target : Targets)
	{
		IPrimitiveComponentBackedTarget* PrimTarget = Cast<IPrimitiveComponentBackedTarget>(Target);
		if (PrimTarget)
		{
			FHitResult HitResult;
			if (PrimTarget->HitTestComponent(WorldRay, HitResult))
			{
				double HitRayParam = (double)WorldRay.GetParameter(HitResult.ImpactPoint);
				if (HitRayParam < NearestRayParam)
				{
					NearestRayParam = HitRayParam;
					PointOut = HitResult.ImpactPoint;
				}
			}
		}
	}

	return (NearestRayParam < (double)HALF_WORLD_MAX);
}

