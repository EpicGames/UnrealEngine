// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneQueries/SceneSnappingManager.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "ContextObjectStore.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneSnappingManager)



namespace UELocal
{

static bool IsHiddenActor(const AActor* Actor)
{
#if WITH_EDITOR
	return ( Actor->IsHiddenEd() );
#else
	return ( Actor->IsHidden() );
#endif
}

static bool IsHiddenComponent(const UPrimitiveComponent* Component)
{
#if WITH_EDITOR
	return (Component->IsVisibleInEditor() == false);
#else
	return (Component->IsVisible() == false);
#endif
}


static bool IsVisibleObject( const UPrimitiveComponent* Component )
{
	if (Component == nullptr)
	{
		return false;
	}
	AActor* Actor = Component->GetOwner();
	if (Actor == nullptr)
	{
		return false;
	}

	if (IsHiddenActor(Actor))
	{
		return false;
	}

	if ( IsHiddenComponent(Component) )
	{
		return false;
	}
	return true;
}

}



bool FSceneQueryVisibilityFilter::IsVisible(const UPrimitiveComponent* Component) const
{
	if (Component == nullptr)
	{
		return false;
	}

	bool bIsVisible = UELocal::IsVisibleObject(Component);
	if (bIsVisible || (InvisibleComponentsToInclude != nullptr && InvisibleComponentsToInclude->Contains(Component)))
	{
		return (ComponentsToIgnore == nullptr) || (ComponentsToIgnore->Contains(Component) == false);
	}
	return false;
}






void FSceneHitQueryResult::InitializeHitResult(const FSceneHitQueryRequest& FromRequest)
{
	HitResult = FHitResult(TargetActor, TargetComponent, (FVector)Position, (FVector)Normal);

	HitResult.Distance = static_cast<float>( FromRequest.WorldRay.GetParameter(Position) );
	HitResult.FaceIndex = HitTriIndex;
	// initialize .Time? Need start/end for that...
}

ESceneSnapQueryTargetResult FSceneSnapQueryTargetHandler::SnapPosition(
	const FSceneSnapQueryRequest& InRequest,
	TArray<FSceneSnapQueryResult>& OutResults) const
{
	FSceneSnapQueryResult SnapResult;
	SnapResult.TargetType = GetTargetTypes();

	const ESceneSnapQueryTargetResult TargetResult = SnapPosition(InRequest.Position, SnapResult.Position, InRequest.AxisList);
	if (TargetResult == ESceneSnapQueryTargetResult::Snapped)
	{
		OutResults.Emplace(SnapResult);
	}

	return TargetResult;
}

ESceneSnapQueryTargetResult FSceneSnapQueryTargetHandler::SnapRotation(
	const FSceneSnapQueryRequest& InRequest,
	TArray<FSceneSnapQueryResult>& OutResults) const
{
	FSceneSnapQueryResult SnapResult;
	SnapResult.TargetType = GetTargetTypes();

	const ESceneSnapQueryTargetResult TargetResult = SnapRotation(InRequest.DeltaRotation, SnapResult.DeltaRotation, InRequest.AxisList);
	if (TargetResult == ESceneSnapQueryTargetResult::Snapped)
	{
		OutResults.Emplace(SnapResult);
	}

	return TargetResult;
}

ESceneSnapQueryTargetResult FSceneSnapQueryTargetHandler::SnapScale(
	const FSceneSnapQueryRequest& InRequest,
	TArray<FSceneSnapQueryResult>& OutResults) const
{
	FSceneSnapQueryResult SnapResult;
    SnapResult.TargetType = GetTargetTypes();

    const ESceneSnapQueryTargetResult TargetResult = SnapScale(InRequest.Scale, SnapResult.Scale, InRequest.AxisList);
	if (TargetResult == ESceneSnapQueryTargetResult::Snapped)
	{
		OutResults.Emplace(SnapResult);
	}

    return TargetResult;
}

EToolContextCoordinateSystem FSceneSnapQueryRequest::GetResultCoordinateSpace() const
{
	return ResultCoordinateSpace.Get(RequestCoordinateSpace);
}

EToolContextCoordinateSystem FSceneSnapQueryResult::GetCoordinateSpace(const FSceneSnapQueryRequest& InFromRequest) const
{
	return CoordinateSpace.Get(InFromRequest.GetResultCoordinateSpace());
}

USceneSnappingManager* USceneSnappingManager::Find(UInteractiveToolManager* ToolManager)
{
	if (ensure(ToolManager))
	{
		USceneSnappingManager* Found = ToolManager->GetContextObjectStore()->FindContext<USceneSnappingManager>();
		if (Found != nullptr)
		{
			return Found;
		}
	}
	return nullptr;
}


USceneSnappingManager* USceneSnappingManager::Find(UInteractiveGizmoManager* GizmoManager)
{
	if (ensure(GizmoManager))
	{
		USceneSnappingManager* Found = GizmoManager->GetContextObjectStore()->FindContext<USceneSnappingManager>();
		if (Found != nullptr)
		{
			return Found;
		}
	}
	return nullptr;
}

