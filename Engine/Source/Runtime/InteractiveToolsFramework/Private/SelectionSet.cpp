// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SelectionSet)


UMeshSelectionSet::UMeshSelectionSet()
{
	//SetFlags(RF_Transactional);
}


TArray<int>& UMeshSelectionSet::GetElements(EMeshSelectionElementType ElementType)
{
	switch (ElementType)
	{
	default:
	case EMeshSelectionElementType::Vertex:
		return Vertices;
	case EMeshSelectionElementType::Edge:
		return Edges;
	case EMeshSelectionElementType::Face:
		return Faces;
	case EMeshSelectionElementType::Group:
		return Groups;
	}
}

const TArray<int>& UMeshSelectionSet::GetElements(EMeshSelectionElementType ElementType) const
{
	switch (ElementType)
	{
	default:
	case EMeshSelectionElementType::Vertex:
		return Vertices;
	case EMeshSelectionElementType::Edge:
		return Edges;
	case EMeshSelectionElementType::Face:
		return Faces;
	case EMeshSelectionElementType::Group:
		return Groups;
	}
}



void UMeshSelectionSet::AddIndices(EMeshSelectionElementType ElementType, const TArray<int32>& Indices)
{
	TArray<int32>& CurElements = GetElements(ElementType);

	int N = Indices.Num();
	for (int k = 0; k < N; ++k)
	{
		CurElements.Add(Indices[k]);
	}
	NotifySelectionSetModified();
}

void UMeshSelectionSet::AddIndices(EMeshSelectionElementType ElementType, const TSet<int32>& Indices)
{
	TArray<int>& CurElements = GetElements(ElementType);

	for ( int32 Index : Indices )
	{
		CurElements.Add(Index);
	}
	NotifySelectionSetModified();
}


void UMeshSelectionSet::RemoveIndices(EMeshSelectionElementType ElementType, const TArray<int32>& Indices)
{
	TSet<int32> IndicesSet(Indices);
	RemoveIndices(ElementType, IndicesSet);
}


void UMeshSelectionSet::RemoveIndices(EMeshSelectionElementType ElementType, const TSet<int32>& Indices)
{
	TArray<int32>& CurElements = GetElements(ElementType);

	TArray<int32> NewElements;
	NewElements.Reserve(FMath::Max(0, CurElements.Num() - Indices.Num()));
	for (int32 Index : CurElements)
	{
		if (Indices.Contains(Index) == false)
		{
			NewElements.Add(Index);
		}
	}
	CurElements = MoveTemp(NewElements);

	NotifySelectionSetModified();
}
