// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolBuilderUtil.h"
#include "CoreMinimal.h"
#include "Algo/Accumulate.h"
#include "GameFramework/Actor.h"
#include "Engine/Selection.h"
#include "Components/StaticMeshComponent.h"
#include "Components/BrushComponent.h"
#include "Components/MeshComponent.h"

int ToolBuilderUtil::CountComponents(const FToolBuilderState& InputState, const TFunction<bool(UActorComponent*)>& Predicate)
{
	int nTypedComponents{};

	if (InputState.SelectedComponents.Num() > 0)
	{
		nTypedComponents = static_cast<int>( Algo::CountIf(InputState.SelectedComponents, Predicate) );
	}
	else
	{
		nTypedComponents =
			Algo::TransformAccumulate(InputState.SelectedActors,
									  [&Predicate](AActor* Actor)
									  {
										  return static_cast<int>( Algo::CountIf(Actor->GetComponents(), Predicate));
									  },
									  0);
	}
	return nTypedComponents;
}




UActorComponent* ToolBuilderUtil::FindFirstComponent(const FToolBuilderState& InputState, const TFunction<bool(UActorComponent*)>& Predicate)
{
	if (InputState.SelectedComponents.Num() > 0)
	{
		UActorComponent* const* ComponentPtr = InputState.SelectedComponents.FindByPredicate(Predicate);
		if (ComponentPtr)
		{
			return *ComponentPtr;
		}
	}
	else
	{
		for ( AActor* Actor : InputState.SelectedActors )
		{
			UActorComponent* const* ComponentPtr = Algo::FindByPredicate(Actor->GetComponents(), Predicate);
			if (ComponentPtr)
			{
				return *ComponentPtr;
			}
		}
	}
	return nullptr;
}




TArray<UActorComponent*> ToolBuilderUtil::FindAllComponents(const FToolBuilderState& InputState, const TFunction<bool(UActorComponent*)>& Predicate)
{
	if (InputState.SelectedComponents.Num() > 0)
	{
		return InputState.SelectedComponents.FilterByPredicate(Predicate);
	}
	else
	{
		return Algo::TransformAccumulate(InputState.SelectedActors,
										 [&Predicate](AActor* Actor)
										 {
											 TInlineComponentArray<UActorComponent*> ActorComponents;
											 Actor->GetComponents(ActorComponents);
											 return ActorComponents.FilterByPredicate(Predicate);
										 },
										 TArray<UActorComponent*>{},
										 [](TArray<UActorComponent*> FoundComponents, TArray<UActorComponent*> ActorComponents)
										 {
											 FoundComponents.Insert(MoveTemp(ActorComponents), FoundComponents.Num());
											 return FoundComponents;
										 });
	}
}


void ToolBuilderUtil::EnumerateComponents(const FToolBuilderState& InputState, TFunctionRef<void(UActorComponent*)> ComponentFunc)
{
	if (InputState.SelectedComponents.Num() > 0)
	{
		for (UActorComponent* Component : InputState.SelectedComponents)
		{
			ComponentFunc(Component);
		}
	}
	else
	{
		for (AActor* Actor : InputState.SelectedActors)
		{
			Actor->ForEachComponent(true,
				[&ComponentFunc](UActorComponent* Component) {
				ComponentFunc(Component);
			});
		}
	}
}




int32 ToolBuilderUtil::CountActors(const FToolBuilderState& InputState, const TFunction<bool(AActor*)>& Predicate)
{
	return static_cast<int32>( Algo::CountIf(InputState.SelectedActors, Predicate) );
}


AActor* ToolBuilderUtil::FindFirstActor(const FToolBuilderState& InputState, const TFunction<bool(AActor*)>& Predicate)
{
	AActor* const* Actor = InputState.SelectedActors.FindByPredicate(Predicate);
	if (Actor)
	{
		return *Actor;
	}
	return nullptr;

}

TArray<AActor*> ToolBuilderUtil::FindAllActors(const FToolBuilderState& InputState, const TFunction<bool(AActor*)>& Predicate)
{
	return InputState.SelectedActors.FilterByPredicate(Predicate);
}


bool ToolBuilderUtil::ComponentTypeCouldHaveUVs(const UActorComponent& Component)
{
	return Cast<UMeshComponent>(&Component) != nullptr;
}


bool ToolBuilderUtil::IsVolume(const UActorComponent& Component)
{
	return Cast<UBrushComponent>(&Component) != nullptr;
}