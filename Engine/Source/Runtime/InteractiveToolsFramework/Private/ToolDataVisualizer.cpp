// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolDataVisualizer.h"
#include "MaterialShared.h"
#include "ToolContextInterfaces.h"
#include "PrimitiveDrawingUtils.h"   // DrawCircle
#include "BaseGizmos/GizmoMath.h"


const int BoxFaces[6][4] =
{
	{ 0, 1, 2, 3 },     // back, -z
	{ 5, 4, 7, 6 },     // front, +z
	{ 4, 0, 3, 7 },     // left, -x
	{ 1, 5, 6, 2 },     // right, +x,
	{ 4, 5, 1, 0 },     // bottom, -y
	{ 3, 2, 6, 7 }      // top, +y
};

FToolDataVisualizer::FToolDataVisualizer()
{
	LineColor = FLinearColor(0.95f, 0.05f, 0.05f);
	PointColor = FLinearColor(0.95f, 0.05f, 0.05f);
	PopAllTransforms();
}

FToolDataVisualizer::FToolDataVisualizer(const FToolDataVisualizer&) = default;
FToolDataVisualizer::~FToolDataVisualizer() = default;

void FToolDataVisualizer::BeginFrame(IToolsContextRenderAPI* RenderAPI, const FViewCameraState& CameraStateIn)
{
	checkf(CurrentPDI == nullptr, TEXT("FToolDataVisualizer::BeginFrame: matching EndFrame was not called last frame!"));
	CurrentPDI = RenderAPI->GetPrimitiveDrawInterface();
	CameraState = CameraStateIn;
	PDISizeScale = CameraState.GetPDIScalingFactor();
	bHaveCameraState = true;
}

void FToolDataVisualizer::BeginFrame(IToolsContextRenderAPI* RenderAPI)
{
	checkf(CurrentPDI == nullptr, TEXT("FToolDataVisualizer::BeginFrame: matching EndFrame was not called last frame!"));
	CurrentPDI = RenderAPI->GetPrimitiveDrawInterface();
	CameraState = RenderAPI->GetCameraState();
	PDISizeScale = CameraState.GetPDIScalingFactor();
	bHaveCameraState = true;
}

void FToolDataVisualizer::EndFrame()
{
	// not safe to hold PDI
	CurrentPDI = nullptr;
}


void FToolDataVisualizer::SetTransform(const FTransform& Transform)
{
	TransformStack.Reset();
	TransformStack.Add(Transform);
	TotalTransform = Transform;
}


void FToolDataVisualizer::PushTransform(const FTransform& Transform)
{
	TransformStack.Add(Transform);
	TotalTransform *= Transform;
}

void FToolDataVisualizer::PopTransform()
{
	TransformStack.Pop(EAllowShrinking::No);
	TotalTransform = FTransform::Identity;
	for (const FTransform& Transform : TransformStack)
	{
		TotalTransform *= Transform;
	}
}

void FToolDataVisualizer::PopAllTransforms()
{
	TransformStack.Reset();
	TotalTransform = FTransform::Identity;
}




void FToolDataVisualizer::InternalDrawTransformedLine(const FVector& A, const FVector& B, const FLinearColor& ColorIn, float LineThicknessIn, bool bDepthTestedIn)
{
	CurrentPDI->DrawLine(A, B, ColorIn,
		uint8( (bDepthTestedIn) ? SDPG_World : SDPG_Foreground),
		LineThicknessIn * PDISizeScale, DepthBias, true);
}

void FToolDataVisualizer::InternalDrawDirectionalArrow(const FVector& A, const FVector& B, const FVector& InNormal, const FLinearColor& Color, const float InHeadSize, float InLineThickness, bool bInDepthTested)
{
	FVector Direction = B - A;
	Direction.Normalize();

	FVector Up = FVector::UpVector;
	FVector Right = Direction ^ Up;

	if (!Right.IsNormalized())
	{
		Direction.FindBestAxisVectors(Up, Right);
	}

	FMatrix Transform = FMatrix::Identity;
	Transform.SetAxes(&Direction, &Right, &Up, &B);
	
	CurrentPDI->DrawLine(
		A,
		B,
		Color,
		uint8( (bInDepthTested) ? SDPG_World : SDPG_Foreground),
		InLineThickness * PDISizeScale,
		DepthBias,
		true);

	const float ArrowSqrt = FMath::Sqrt(InHeadSize);

	const FVector ArrowPos0 = FVector(-ArrowSqrt, +ArrowSqrt, +ArrowSqrt);
	const FVector ArrowPos1 = FVector(-ArrowSqrt, +ArrowSqrt, -ArrowSqrt);
	const FVector ArrowPos2 = FVector(-ArrowSqrt, -ArrowSqrt, +ArrowSqrt);
	const FVector ArrowPos3 = FVector(-ArrowSqrt, -ArrowSqrt, -ArrowSqrt);

	auto DrawHeadLine = [&](const FVector& InB)
	{
		const FVector HeadA = Transform.TransformPosition(FVector::ZeroVector);
		const FVector HeadB = Transform.TransformPosition(InB);
		
		CurrentPDI->DrawLine(
			HeadA,
			HeadB,
			Color,
			uint8( (bInDepthTested) ? SDPG_World : SDPG_Foreground),
			InLineThickness * PDISizeScale,
			DepthBias,
			true);
	};

	DrawHeadLine(ArrowPos0);
	DrawHeadLine(ArrowPos1);
	DrawHeadLine(ArrowPos2);
	DrawHeadLine(ArrowPos3);
}


void FToolDataVisualizer::InternalDrawTransformedPoint(const FVector& Position, const FLinearColor& ColorIn, float PointSizeIn, bool bDepthTestedIn)
{
	CurrentPDI->DrawPoint(Position, ColorIn, PointSizeIn * PDISizeScale,
		uint8( (bDepthTestedIn) ? SDPG_World : SDPG_Foreground) );
}


void FToolDataVisualizer::InternalDrawCircle(const FVector& Position, const FVector& Normal, float Radius, int Steps, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	FVector Tan1, Tan2;
	GizmoMath::MakeNormalPlaneBasis((FVector)TransformN(Normal), Tan1, Tan2);
	Tan1.Normalize(); Tan2.Normalize();

	// this function is from SceneManagement.h
	::DrawCircle(CurrentPDI, TransformP(Position), (FVector)Tan1, (FVector)Tan2, 
		Color, Radius, Steps,
		uint8( (bDepthTestedIn) ? SDPG_World : SDPG_Foreground),
		LineThicknessIn * PDISizeScale, DepthBias, true);
}

void FToolDataVisualizer::InternalDrawArc(const FVector& InPosition, const FVector& InNormal, const float InRadius, const int InSteps, const float InMinAngle, const float InMaxAngle, const FLinearColor& Color, float LineThicknessIn, const bool bDepthTestedIn)
{
	FVector Tan1, Tan2;
	GizmoMath::MakeNormalPlaneBasis(TransformN(InNormal), Tan1, Tan2);
	Tan1.Normalize(); Tan2.Normalize();

	// this function is from SceneManagement.h
	::DrawArc(CurrentPDI, TransformP(InPosition), Tan1, Tan2,
		InMinAngle, InMaxAngle,
		InRadius, InSteps, Color,
		static_cast<uint8>(bDepthTestedIn ? SDPG_World : SDPG_Foreground));
}

void FToolDataVisualizer::InternalDrawWireBox(const FBox& Box, const FLinearColor& ColorIn, float LineThicknessIn, bool bDepthTestedIn)
{
	// corners [ (-x,-y), (x,-y), (x,y), (-x,y) ], -z, then +z
	FVector Corners[8] =
	{
		TransformP(Box.Min),
		TransformP(FVector(Box.Max.X, Box.Min.Y, Box.Min.Z)),
		TransformP(FVector(Box.Max.X, Box.Max.Y, Box.Min.Z)),
		TransformP(FVector(Box.Min.X, Box.Max.Y, Box.Min.Z)),
		TransformP(FVector(Box.Min.X, Box.Min.Y, Box.Max.Z)),
		TransformP(FVector(Box.Max.X, Box.Min.Y, Box.Max.Z)),
		TransformP(Box.Max),
		TransformP(FVector(Box.Min.X, Box.Max.Y, Box.Max.Z))
	};
	for (int FaceIdx = 0; FaceIdx < 6; FaceIdx++)
	{
		for (int Last = 3, Cur = 0; Cur < 4; Last = Cur++)
		{
			InternalDrawTransformedLine(Corners[BoxFaces[FaceIdx][Last]], Corners[BoxFaces[FaceIdx][Cur]], 
										ColorIn, LineThicknessIn, bDepthTestedIn);
		}
	}
}

void FToolDataVisualizer::InternalDrawSquare(const FVector& Center, const FVector& SideA, const FVector& SideB, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	FVector CC = TransformP(Center);
	FVector SA = TransformV(SideA);
	FVector SB = TransformV(SideB);
	FVector HalfDiag = (SA + SB) * .5f;
	FVector C00 = CC - HalfDiag;
	FVector C11 = CC + HalfDiag;
	FVector C01 = C00 + SB;
	FVector C10 = C00 + SA;
	InternalDrawTransformedLine(C00, C01, Color, LineThicknessIn, bDepthTestedIn);
	InternalDrawTransformedLine(C01, C11, Color, LineThicknessIn, bDepthTestedIn);
	InternalDrawTransformedLine(C10, C11, Color, LineThicknessIn, bDepthTestedIn);
	InternalDrawTransformedLine(C00, C10, Color, LineThicknessIn, bDepthTestedIn);
}

// @note: This is taken from MeshDebugDraw::DrawSimpleGrid
void FToolDataVisualizer::InternalDrawGrid(const FVector& Center, const FVector& Normal, const FVector& Size, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	FVector Tan1, Tan2;
	GizmoMath::MakeNormalPlaneBasis(Normal, Tan1, Tan2);

	InternalDrawGrid(Center, Normal, Tan1, Tan2, Size, Color, LineThicknessIn, bDepthTestedIn);
}

void FToolDataVisualizer::InternalDrawGrid(const FVector& Center, const FVector& Normal, const FVector& Up, const FVector& Side, const FVector& Size, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	const ESceneDepthPriorityGroup DepthPriority = (bDepthTestedIn) ? SDPG_World : SDPG_Foreground;

	const FVector CC = TransformP(Center);
	const FVector SA = TransformV(Up);
	const FVector SB = TransformV(Side);

	// Always odd
	constexpr int32 NumGridLines = (FMath::DivideAndRoundUp(16, 2) * 2) + 1;
	const float GridLineSpacing = Size.X / (NumGridLines - 1);

	const double Width = static_cast<double>(NumGridLines - 1) * GridLineSpacing;
	const double Extent = Width * 0.5;

	constexpr int32 LineSteps = FMath::DivideAndRoundUp(NumGridLines, 2);
	for (int32 LineIndex = 0; LineIndex < LineSteps; ++LineIndex)
	{
		const double DistanceX = static_cast<double>(LineIndex) * GridLineSpacing;
		FVector3d A = CC - Extent * SB - DistanceX * SA;
		FVector3d B = CC + Extent * SB - DistanceX * SA;
		CurrentPDI->DrawLine(A, B, Color, DepthPriority, LineThicknessIn, 0, true);

		A = CC - Extent * SB + DistanceX * SA;
		B = CC + Extent * SB + DistanceX * SA;
		CurrentPDI->DrawLine(A, B, Color, DepthPriority, LineThicknessIn, 0, true);

		A = CC - Extent * SA - DistanceX * SB;
		B = CC + Extent * SA - DistanceX * SB;
		CurrentPDI->DrawLine(A, B, Color, DepthPriority, LineThicknessIn, 0, true);

		A = CC - Extent * SA + DistanceX * SB;
		B = CC + Extent * SA + DistanceX * SB;
		CurrentPDI->DrawLine(A, B, Color, DepthPriority, LineThicknessIn, 0, true);
	}
}

void FToolDataVisualizer::InternalDrawWireCylinder(const FVector& Position, const FVector& Normal, float Radius, float Height, int Steps, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	FVector Tan1, Tan2;
	GizmoMath::MakeNormalPlaneBasis(Normal, Tan1, Tan2);
	
	const float	AngleDelta = 2.0f * PI / Steps;
	FVector X(Tan1), Y(Tan2);
	FVector	LastVertex = TransformP(Position + X * Radius);
	FVector LastVertexB = TransformP(Position + X * Radius + Normal * Height);

	for (int32 Step = 0; Step < Steps; Step++)
	{
		float Angle = (Step + 1) * AngleDelta;
		FVector A = Position + (X * FMath::Cos(Angle) + Y * FMath::Sin(Angle)) * Radius;
		FVector B = A + Normal * Height;
		FVector Vertex = TransformP(A);
		FVector VertexB = TransformP(B);
		InternalDrawTransformedLine(LastVertex, Vertex, Color, LineThicknessIn, bDepthTestedIn);
		InternalDrawTransformedLine(Vertex, VertexB, Color, LineThicknessIn, bDepthTestedIn);
		InternalDrawTransformedLine(LastVertexB, VertexB, Color, LineThicknessIn, bDepthTestedIn);
		LastVertex = Vertex;
		LastVertexB = VertexB;
	}

}


void FToolDataVisualizer::InternalDrawViewFacingCircle(const FVector& Position, float Radius, int Steps, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	checkf(bHaveCameraState, TEXT("To call this function, you must first call the version of BeginFrame that takes the CameraState"));

	FVector WorldPosition = TransformP(Position);
	FVector WorldNormal = (CameraState.Position - WorldPosition);
	WorldNormal.Normalize();
	FVector Tan1, Tan2;
	GizmoMath::MakeNormalPlaneBasis(WorldNormal, Tan1, Tan2);

	// this function is from SceneManagement.h
	::DrawCircle(CurrentPDI, WorldPosition, (FVector)Tan1, (FVector)Tan2,
		Color, Radius, Steps,
		uint8( (bDepthTestedIn) ? SDPG_World : SDPG_Foreground),
		LineThicknessIn * PDISizeScale, DepthBias, true);
}


void FToolDataVisualizer::InternalDrawViewFacingArc(const FVector& InPosition, const float InRadius, const int InSteps, const float InMinAngle, const float InMaxAngle, const FLinearColor& Color, float LineThicknessIn, const bool bDepthTestedIn)
{
	checkf(bHaveCameraState, TEXT("To call this function, you must first call the version of BeginFrame that takes the CameraState"));

	const FVector WorldPosition = TransformP(InPosition);
	FVector WorldNormal = (CameraState.Position - WorldPosition);
	WorldNormal.Normalize();
	FVector Tan1, Tan2;
	GizmoMath::MakeNormalPlaneBasis(WorldNormal, Tan1, Tan2);

	// this function is from SceneManagement.h
	::DrawArc(CurrentPDI, WorldPosition, Tan1, Tan2,
		InMinAngle, InMaxAngle,
		InRadius, InSteps, Color,
		static_cast<uint8>(bDepthTestedIn ? SDPG_World : SDPG_Foreground));
}
 
void FToolDataVisualizer::InternalDrawViewFacingX(const FVector& Position, float Width, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
{
	checkf(bHaveCameraState, TEXT("To call this function, you must first call the version of BeginFrame that takes the CameraState"));
	
	FVector WorldPosition = TransformP(Position);

	FVector UpOffset = CameraState.Up() * Width / 2;
	FVector RightOffset = CameraState.Right() * Width / 2;
	InternalDrawTransformedLine(
		WorldPosition - UpOffset - RightOffset, 
		WorldPosition + UpOffset + RightOffset, 
		Color, LineThicknessIn, bDepthTestedIn);

	InternalDrawTransformedLine(
		WorldPosition + UpOffset - RightOffset,
		WorldPosition - UpOffset + RightOffset,
		Color, LineThicknessIn, bDepthTestedIn);
}

void FToolDataVisualizer::InternalDrawDisc(const FVector& Position, const FVector& Normal, float Radius, int Steps, const FColor& Color, FMaterialRenderProxy* RenderProxy, bool bDepthTestedIn)
{
	const FVector TransformNormal = TransformN(Normal);
	FVector Tan1;
	FVector Tan2;
	GizmoMath::MakeNormalPlaneBasis(TransformNormal, Tan1, Tan2);
	Tan1.Normalize(); 
	Tan2.Normalize();

	// this function is from SceneManagement.h
	const uint8 DepthPriority = uint8( (bDepthTestedIn) ? SDPG_World : SDPG_Foreground );
	const FVector TransformPosition = TransformP(Position);
	::DrawDisc(CurrentPDI, TransformPosition, Tan1, Tan2, Color, Radius, Steps, RenderProxy,
	           DepthPriority);
}