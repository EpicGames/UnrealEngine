// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolGroupManager.h"

#include "ContextObjectStore.h"
#include "InteractiveToolsContext.h"
#include "InputBehaviorSet.h"
#include "InputRouter.h"

namespace
{
	/** Simple adapter that exposes a pre-built behavior set through the IInputBehaviorSource interface. */
	class FBehaviorSetSource : public IInputBehaviorSource
	{
	public:
		explicit FBehaviorSetSource(const UInputBehaviorSet* InSet)
			: Set(InSet)
		{}

		virtual const UInputBehaviorSet* GetInputBehaviors() const override
		{
			return Set;
		}

	private:
		const UInputBehaviorSet* Set = nullptr;
	};
}

void UToolGroupManager::Initialize()
{
}

void UToolGroupManager::Shutdown()
{
	DisableAllToolGroups();
	RegisteredGroups.Empty();
	EnabledGroups.Empty();
}

bool UToolGroupManager::RegisterToolGroup(FName GroupName, TSharedPtr<const FToolGroup> ToolGroup)
{
	if (GroupName.IsNone() || !ToolGroup.IsValid())
	{
		return false;
	}

	if (RegisteredGroups.Contains(GroupName))
	{
		return false;
	}

	RegisteredGroups.Add(GroupName, ToolGroup);
	return true;
}

bool UToolGroupManager::HasToolGroup(FName GroupName) const
{
	return RegisteredGroups.Contains(GroupName);
}

bool UToolGroupManager::IsToolGroupEnabled(FName GroupName) const
{
	return EnabledGroups.Contains(GroupName);
}

TArray<FName> UToolGroupManager::GetEnabledToolGroupNames() const
{
	TArray<FName> Names;
	EnabledGroups.GetKeys(Names);
	return Names;
}

TOptional<FName> UToolGroupManager::FindEnabledGroupOwningActiveTool(EToolSide Side) const
{
	UInteractiveToolsContext* Context = GetOwningContext();
	if (!Context || !Context->HasActiveTool(Side))
	{
		return {};
	}

	const FString ActiveTool = Context->GetActiveToolName(Side);

	for (const TPair<FName, FEnabledToolGroupState>& Pair : EnabledGroups)
	{
		if (Pair.Value.RegisteredToolTypeIds.Contains(ActiveTool))
		{
			return Pair.Key;
		}
	}
	return {};
}

bool UToolGroupManager::HasAnyEnabledToolGroups()
{
	return EnabledGroups.Num() > 0;
}

bool UToolGroupManager::EnableToolGroup(FName GroupName)
{
	UInteractiveToolsContext* Context = GetOwningContext();
	if (!Context || !Context->ToolManager)
	{
		return false;
	}

	if (EnabledGroups.Contains(GroupName))
	{
		return true;
	}

	TSharedPtr<const FToolGroup>* Found = RegisteredGroups.Find(GroupName);
	if (!Found || !Found->IsValid())
	{
		return false;
	}

	const FToolGroup& ToolGroup = *Found->Get();
	UInteractiveToolManager* ToolManager = Context->ToolManager;

	FEnabledToolGroupState State;

	// Register tools for this group (record exact tool type ids for ownership + cleanup)
	RegisterGroupTools(ToolGroup, State, *ToolManager);

	// Acquire shared features into this context (ref-counted)
	AcquireGroupFeatures(ToolGroup, State, *Context);

	// Build & register per-context behaviors
	RegisterGroupBehaviors(ToolGroup, State, *Context);

	EnabledGroups.Add(GroupName, MoveTemp(State));
	return true;
}

bool UToolGroupManager::DisableToolGroup(FName GroupName)
{
	UInteractiveToolsContext* Context = GetOwningContext();
	if (!Context || !Context->ToolManager)
	{
		return false;
	}

	FEnabledToolGroupState* State = EnabledGroups.Find(GroupName);
	if (!State)
	{
		return false;
	}

	TSharedPtr<const FToolGroup>* Found = RegisteredGroups.Find(GroupName);
	if (!Found || !Found->IsValid())
	{
		// Group was enabled, but descriptor disappeared (shouldn't happen). Best-effort cleanup.
		UnregisterGroupBehaviors(*State, *Context);
		UnregisterGroupTools(*State, *Context->ToolManager);
		ReleaseAllFeaturesForGroup(*State, *Context);
		EnabledGroups.Remove(GroupName);
		return true;
	}

	const FToolGroup& Descriptor = *Found->Get();

	// If an active tool belongs to this group, cancel it
	if (Context->HasActiveTool(EToolSide::Mouse))
	{
		const FString ActiveTool = Context->GetActiveToolName(EToolSide::Mouse);
		if (State->RegisteredToolTypeIds.Contains(ActiveTool))
		{
			Context->EndTool(EToolSide::Mouse, EToolShutdownType::Cancel);
		}
	}
	
	// Unregister behaviors
	UnregisterGroupBehaviors(*State, *Context);

	// Unregister tools
	UnregisterGroupTools(*State, *Context->ToolManager);

	// Release shared features acquired for this group
	ReleaseAllFeaturesForGroup(*State, *Context);

	EnabledGroups.Remove(GroupName);
	return true;
}

void UToolGroupManager::DisableAllToolGroups()
{
	TArray<FName> Names;
	EnabledGroups.GetKeys(Names);
	for (FName Name : Names)
	{
		DisableToolGroup(Name);
	}
}
void UToolGroupManager::AcquireGroupFeatures(
	const FToolGroup& ToolGroup,
	FEnabledToolGroupState& State,
	UInteractiveToolsContext& Context)
{
	ToolGroup.EnumerateFeatures([&](const FToolGroup::FFeature& Feature)
	{
		FeatureRegistry.Acquire(Feature.Key, Context, Feature.Install, Feature.Uninstall);
		State.AcquiredFeatures.Add(Feature.Key);
		return true;
	});
}

void UToolGroupManager::ReleaseAllFeaturesForGroup(FEnabledToolGroupState& State, UInteractiveToolsContext& Context)
{
	for (FToolGroupFeatureKey Key : State.AcquiredFeatures)
	{
		FeatureRegistry.Release(Key, Context);
	}
	State.AcquiredFeatures.Reset();
}

void UToolGroupManager::RegisterGroupTools(const FToolGroup& ToolGroup, FEnabledToolGroupState& State, UInteractiveToolManager& ToolManager)
{
	ToolGroup.EnumerateTools([&](const FTool& Tool)
	{
		if (!Tool.BuilderClass)
		{
			return true;
		}

		// If/when namespacing is added later, compute ToolTypeId here and store that.
		const FString ToolTypeId = Tool.Name;

		UInteractiveToolBuilder* Builder = NewObject<UInteractiveToolBuilder>(&ToolManager, Tool.BuilderClass);
		if (!ensureMsgf(Builder, TEXT("Failed to allocate tool builder for ToolTypeId '%s' (%s)"), *ToolTypeId, *GetNameSafe(Tool.BuilderClass)))
		{
			return true; // continue enumeration but skip this tool
		}
		ToolManager.RegisterToolType(ToolTypeId, Builder);
		State.RegisteredToolTypeIds.Add(ToolTypeId);
		return true;
	});
}

void UToolGroupManager::UnregisterGroupTools(FEnabledToolGroupState& State, UInteractiveToolManager& ToolManager)
{
	for (const FString& ToolTypeId : State.RegisteredToolTypeIds)
	{
		ToolManager.UnregisterToolType(ToolTypeId);
	}
	State.RegisteredToolTypeIds.Reset();
}

void UToolGroupManager::RegisterGroupBehaviors(const FToolGroup& ToolGroup, FEnabledToolGroupState& State, UInteractiveToolsContext& Context)
{
	if (!Context.InputRouter)
	{
		return;
	}

	if (!IsValid(State.BehaviorSet))
	{
		// Outer on the manager so the behavior set's lifetime is tied to this context's manager instance.
		State.BehaviorSet = NewObject<UInputBehaviorSet>(this);
		ToolGroup.BuildInputBehaviors(Context, *State.BehaviorSet.Get());
	}

	if (!State.bRegisteredAsBehaviorSource)
	{
		State.BehaviorSource = MakeShared<FBehaviorSetSource>(State.BehaviorSet.Get());
		Context.InputRouter->RegisterSource(State.BehaviorSource.Get());
		State.bRegisteredAsBehaviorSource = true;
	}
}

void UToolGroupManager::UnregisterGroupBehaviors(FEnabledToolGroupState& State, UInteractiveToolsContext& Context)
{
	if (Context.InputRouter && State.bRegisteredAsBehaviorSource && State.BehaviorSource.IsValid())
	{
		Context.InputRouter->DeregisterSource(State.BehaviorSource.Get());
	}

	State.bRegisteredAsBehaviorSource = false;
	State.BehaviorSource.Reset();
	State.BehaviorSet = nullptr;
}
UInteractiveToolsContext* UToolGroupManager::GetOwningContext() const
{
	return Cast<UInteractiveToolsContext>(GetOuter());
}

void FToolGroupFeatureRegistry::Acquire(
	FToolGroupFeatureKey Key,
	UInteractiveToolsContext& Context,
	TFunction<void(UInteractiveToolsContext&)> Install,
	TFunction<void(UInteractiveToolsContext&)> Uninstall)
{
	FToolGroupFeatureEntry& Entry = Features.FindOrAdd(Key);

	// First acquisition installs and defines callbacks for this context.
	if (Entry.RefCount == 0)
	{
		Entry.Install = MoveTemp(Install);
		Entry.Uninstall = MoveTemp(Uninstall);

		if (Entry.Install)
		{
			Entry.Install(Context);
		}
	}
	else
	{
		// Re-acquire: feature already defined for this context; ignore callbacks and just ref-count.
		// (Immutable ToolGroup descriptors will naturally re-provide callbacks here.)
	}

	++Entry.RefCount;
}

void FToolGroupFeatureRegistry::Release(FToolGroupFeatureKey Key, UInteractiveToolsContext& Context)
{
	FToolGroupFeatureEntry* Entry = Features.Find(Key);
	if (!Entry)
	{
		return;
	}

	Entry->RefCount--;
	if (Entry->RefCount <= 0)
	{
		if (Entry->Uninstall)
		{
			Entry->Uninstall(Context);
		}
		Features.Remove(Key);
	}
}
