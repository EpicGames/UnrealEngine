// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolTargetManager.h"
#include "InteractiveToolsContext.h"
#include "ToolBuilderUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolTargetManager)

namespace ToolTargetManagerLocals
{
	// Given an array and a 1:1 array of bools, remove all entries that have the corresponding bool set to true.
	// Returns number of elements removed
	int32 RemoveByFlags(TArray<UObject*>& ObjectsInOut, const TArray<bool>& RemoveFlags)
	{
		if (!ensure(RemoveFlags.Num() == ObjectsInOut.Num()))
		{
			return 0;
		}

		int32 OutIndex = 0;
		int32 InIndex = 0;
		while (InIndex < ObjectsInOut.Num())
		{
			if (!RemoveFlags[InIndex])
			{
				ObjectsInOut[OutIndex] = ObjectsInOut[InIndex];
				++OutIndex;
			}
			++InIndex;
		}
		ObjectsInOut.SetNum(OutIndex);
		return InIndex - OutIndex;
	}
}

void UToolTargetManager::Initialize()
{
	bIsActive = true;
}

void UToolTargetManager::Shutdown()
{
	Factories.Empty();
	bIsActive = false;
}

void UToolTargetManager::AddTargetFactory(UToolTargetFactory* Factory)
{
	// If this type of factory has already been added, skip it.
	if (Factories.ContainsByPredicate(
		[Factory](UToolTargetFactory* ExistingFactory) { 
			return ExistingFactory->GetClass() == Factory->GetClass(); 
		}))
	{
		return;
	}

	Factories.Add(Factory);
}

void UToolTargetManager::RemoveTargetFactoriesByPredicate(TFunctionRef<bool(UToolTargetFactory*)> Predicate)
{
	Factories.RemoveAll(Predicate);
}

UToolTargetFactory* UToolTargetManager::FindFirstFactoryByPredicate(TFunctionRef<bool(UToolTargetFactory*)> Predicate)
{
	TObjectPtr<UToolTargetFactory>* Found = Factories.FindByPredicate(Predicate);
	return (Found) ? Found->Get() : nullptr;
}


bool UToolTargetManager::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetType) const
{
	if (InputFilterFunction && !InputFilterFunction(SourceObject))
	{
		return false;
	}
	for (UToolTargetFactory* Factory : Factories)
	{
		if (Factory->CanBuildTarget(SourceObject, TargetType))
		{
			return true;
		}
	}
	return false;
}

UToolTarget* UToolTargetManager::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetType)
{
	if (InputFilterFunction && !InputFilterFunction(SourceObject))
	{
		return nullptr;
	}
	for (UToolTargetFactory* Factory : Factories)
	{
		if (Factory->CanBuildTarget(SourceObject, TargetType))
		{
			UToolTarget* Result = Factory->BuildTarget(SourceObject, TargetType);
			if (Result != nullptr)
			{
				return Result;
			}
		}
	}
	return nullptr;
}

int32 UToolTargetManager::CountSelectedAndTargetable(const FToolBuilderState& SceneState, const FToolTargetTypeRequirements& TargetType) const
{
	return CountSelectedAndTargetableWithPredicate(SceneState, TargetType, 
		// We currently only ever get components out of our scene state, so it is ok to
		//  use the predicate that takes an actor component
		[](UActorComponent& Component) { return true; });
}


void UToolTargetManager::EnumerateSelectedAndTargetableComponents(const FToolBuilderState& SceneState,
	const FToolTargetTypeRequirements& TargetRequirements,
	TFunctionRef<void(UActorComponent*)> ComponentFunc) const
{
	// Gather input objects
	TArray<UObject*> InputObjects;
	InputObjects.Append(ToolBuilderUtil::FindAllComponents(SceneState, [this](UActorComponent* Component)
	{
		return Component && (!InputFilterFunction || InputFilterFunction(Component));
	}));

	TArray<bool> Used;
	for (TObjectPtr<UToolTargetFactory> Factory : Factories)
	{
		int32 NumTargets = Factory->CanBuildTargets(InputObjects, TargetRequirements, Used);
		if (NumTargets > 0)
		{
			for (int32 Index = 0; Index < Used.Num(); ++Index)
			{
				if (Used[Index] && ensure(InputObjects.IsValidIndex(Index)))
				{
					ComponentFunc(static_cast<UActorComponent*>(InputObjects[Index]));
				}
			}
			ensureMsgf(ToolTargetManagerLocals::RemoveByFlags(InputObjects, Used) > 0, 
				TEXT("Factory claimed it could build target(s) without using any objects."));

			if (InputObjects.IsEmpty())
			{
				break;
			}
		}
	}
}

int32 UToolTargetManager::CountSelectedAndTargetableWithPredicate(const FToolBuilderState& SceneState,
	const FToolTargetTypeRequirements& TargetRequirements,
	TFunctionRef<bool(UActorComponent&)> ComponentPred) const
{
	// Gather input objects
	TArray<UObject*> InputObjects;
	InputObjects.Append(ToolBuilderUtil::FindAllComponents(SceneState, [this, &ComponentPred](UActorComponent* Component)
	{
		return Component 
			&& (!InputFilterFunction || InputFilterFunction(Component)) 
			&& ComponentPred(*Component);
	}));

	int32 Count = 0;
	TArray<bool> Used;
	for (TObjectPtr<UToolTargetFactory> Factory : Factories)
	{
		int32 NumTargets = Factory->CanBuildTargets(InputObjects, TargetRequirements, Used);
		if (NumTargets > 0)
		{
			Count += NumTargets;
			ensureMsgf(ToolTargetManagerLocals::RemoveByFlags(InputObjects, Used) > 0,
				TEXT("Factory claimed it could build target(s) without using any objects."));
			if (InputObjects.IsEmpty())
			{
				break;
			}
		}
	}

	return Count;
}

UToolTarget* UToolTargetManager::BuildFirstSelectedTargetable(const FToolBuilderState& SceneState, const FToolTargetTypeRequirements& TargetType)
{
	// Gather input objects
	TArray<UObject*> InputObjects;
	InputObjects.Append(ToolBuilderUtil::FindAllComponents(SceneState, [this](UActorComponent* Component)
	{
		return Component && (!InputFilterFunction || InputFilterFunction(Component));
	}));

	TArray<bool> Used;
	for (TObjectPtr<UToolTargetFactory> Factory : Factories)
	{
		int32 NumTargets = Factory->CanBuildTargets(InputObjects, TargetType, Used);
		if (NumTargets > 0)
		{
			return Factory->BuildFirstTarget(InputObjects, TargetType, Used);
		}
	}
	return nullptr;
}

TArray<TObjectPtr<UToolTarget>> UToolTargetManager::BuildAllSelectedTargetable(const FToolBuilderState& SceneState,
	const FToolTargetTypeRequirements& TargetType)
{
	TArray<TObjectPtr<UToolTarget>> TargetsOut;

	// Gather input objects
	TArray<UObject*> InputObjects;
	InputObjects.Append(ToolBuilderUtil::FindAllComponents(SceneState, [this](UActorComponent* Component)
	{
		return Component && (!InputFilterFunction || InputFilterFunction(Component));
	}));

	TArray<bool> Used;
	for (TObjectPtr<UToolTargetFactory> Factory : Factories)
	{
		int32 NumTargets = Factory->CanBuildTargets(InputObjects, TargetType, Used);
		if (NumTargets > 0)
		{
			TargetsOut.Append(Factory->BuildTargets(InputObjects, TargetType, Used));
			ensureMsgf(ToolTargetManagerLocals::RemoveByFlags(InputObjects, Used) > 0,
				TEXT("Factory claimed it could build target(s) without using any objects."));

			if (InputObjects.IsEmpty())
			{
				break;
			}
		}
	}

	return TargetsOut;
}

