// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/PrimitiveComponentToolTarget.h"

#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrimitiveComponentToolTarget)

bool UPrimitiveComponentToolTarget::IsValid() const
{
	return Component.IsValid();
}

UPrimitiveComponent* UPrimitiveComponentToolTarget::GetOwnerComponent() const
{
	// Note that we don't just return Component.Get() because we want to call the virtual IsValid
	// for derived classes.
	return IsValid() ? Component.Get() : nullptr;
}

USceneComponent* UPrimitiveComponentToolTarget::GetOwnerSceneComponent() const
{
	return StaticCast<USceneComponent*>(GetOwnerComponent());
}


AActor* UPrimitiveComponentToolTarget::GetOwnerActor() const
{
	return IsValid() ? Component->GetOwner() : nullptr;
}

void UPrimitiveComponentToolTarget::SetOwnerVisibility(bool bVisible) const
{
	if (IsValid())
	{
		Component->SetVisibility(bVisible);
	}
}

FTransform UPrimitiveComponentToolTarget::GetWorldTransform() const
{
	return IsValid() ? Component->GetComponentTransform() : FTransform::Identity;
}

bool UPrimitiveComponentToolTarget::HitTestComponent(const FRay& WorldRay, FHitResult& OutHit) const
{
	FVector End = WorldRay.PointAt(HALF_WORLD_MAX);
	if (IsValid() && Component->LineTraceComponent(OutHit, WorldRay.Origin, End, FCollisionQueryParams(SCENE_QUERY_STAT(HitTest), true)))
	{
		return true;
	}
	return false;
}

void UPrimitiveComponentToolTarget::InitializeComponent(UPrimitiveComponent* ComponentIn)
{
	Component = ComponentIn;

	if (ensure(Component.IsValid()))
	{
#if WITH_EDITOR
		// See comment in OnObjectsReplaced
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UPrimitiveComponentToolTarget::OnObjectsReplaced);
#endif
	}
}

void UPrimitiveComponentToolTarget::BeginDestroy()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UPrimitiveComponentToolTarget::OnObjectsReplaced(const FCoreUObjectDelegates::FReplacementObjectMap& Map)
{
	// Components frequently get destroyed and recreated when they are part of blueprint actors that 
	// get modified. For the most part, we don't need to worry about supporting these cases, but keeping
	// a consistent reference here allows us to avoid getting into some bad states. For instance, we often
	// hide the source component and unhide at tool end, and if we lose the reference to the component
	// while the tool is running, we are unable to unhide it later. The user is unlikely to understand 
	// why their object disappeared in that case or know to fix it via the component visibility property.

	UObject* const * MappedObject = Map.Find(Component.Get(/* bEvenIfPendingKill = */ true));
	if (MappedObject)
	{
		Component = Cast<UPrimitiveComponent>(*MappedObject);
	}
}
#endif


// Factory

bool UPrimitiveComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(SourceObject);
	return Component 
		&& ::IsValid(Component)
		&& !Component->IsUnreachable() 
		&& Component->IsValidLowLevel() 
		&& Requirements.AreSatisfiedBy(UPrimitiveComponentToolTarget::StaticClass());
}

UToolTarget* UPrimitiveComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UPrimitiveComponentToolTarget* Target = NewObject<UPrimitiveComponentToolTarget>();
	Target->InitializeComponent(Cast<UPrimitiveComponent>(SourceObject));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));
	return Target;
}



