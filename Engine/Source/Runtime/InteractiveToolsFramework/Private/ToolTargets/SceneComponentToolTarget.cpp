// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/SceneComponentToolTarget.h"

#include "Components/SceneComponent.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneComponentToolTarget)

bool USceneComponentToolTarget::IsValid() const
{
	return Component.IsValid();
}

USceneComponent* USceneComponentToolTarget::GetOwnerSceneComponent() const
{
	// Note that we don't just return Component.Get() because we want to call the virtual IsValid
	// for derived classes.
	return IsValid() ? Component.Get() : nullptr;
}

AActor* USceneComponentToolTarget::GetOwnerActor() const
{
	return IsValid() ? Component->GetOwner() : nullptr;
}

void USceneComponentToolTarget::SetOwnerVisibility(bool bVisible) const
{
	if (IsValid())
	{
		Component->SetVisibility(bVisible);
	}
}

FTransform USceneComponentToolTarget::GetWorldTransform() const
{
	return IsValid() ? Component->GetComponentTransform() : FTransform::Identity;
}

void USceneComponentToolTarget::InitializeComponent(USceneComponent* ComponentIn)
{
	Component = ComponentIn;

	if (ensure(Component.IsValid()))
	{
#if WITH_EDITOR
		// See comment in OnObjectsReplaced
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &USceneComponentToolTarget::OnObjectsReplaced);
#endif
	}
}

void USceneComponentToolTarget::BeginDestroy()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif

	Super::BeginDestroy();
}

#if WITH_EDITOR
void USceneComponentToolTarget::OnObjectsReplaced(const FCoreUObjectDelegates::FReplacementObjectMap& Map)
{
	// Components frequently get destroyed and recreated when they are part of blueprint actors that 
	// get modified. For the most part, we don't need to worry about supporting these cases, but keeping
	// a consistent reference here allows us to avoid getting into some bad states. For instance, we often
	// hide the source component and unhide at tool end, and if we lose the reference to the component
	// while the tool is running, we are unable to unhide it later. The user is unlikely to understand 
	// why their object disappeared in that case or know to fix it via the component visibility property.

	UObject* const * MappedObject = Map.Find(Component.Get(/* bEvenIfPendingKill = */ true));
	if (MappedObject)
	{
		Component = Cast<USceneComponent>(*MappedObject);
	}
}
#endif


// Factory

bool USceneComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	USceneComponent* Component = Cast<USceneComponent>(SourceObject);
	return Component 
		&& ::IsValid(Component)
		&& !Component->IsUnreachable() 
		&& Component->IsValidLowLevel() 
		&& Requirements.AreSatisfiedBy(USceneComponentToolTarget::StaticClass());
}

UToolTarget* USceneComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	USceneComponentToolTarget* Target = NewObject<USceneComponentToolTarget>();
	Target->InitializeComponent(Cast<USceneComponent>(SourceObject));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));
	return Target;
}



