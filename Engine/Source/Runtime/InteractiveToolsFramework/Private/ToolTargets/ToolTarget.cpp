// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/ToolTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolTarget)

bool FToolTargetTypeRequirements::AreSatisfiedBy(UClass* Class) const
{
	// we have to support all the required interfaces
	for (const UClass* Interface : Interfaces)
	{
		if (!Class->ImplementsInterface(Interface))
		{
			return false;
		}
	}
	return true;
}

bool FToolTargetTypeRequirements::AreSatisfiedBy(UToolTarget* ToolTarget) const
{
	return !ToolTarget ? false : AreSatisfiedBy(ToolTarget->GetClass());
}

int32 UToolTargetFactory::CanBuildTargets(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WouldBeUsedOut)
{
	int32 Count = 0;
	WouldBeUsedOut.SetNum(InputObjects.Num());
	for (int32 ObjIndex = 0; ObjIndex < InputObjects.Num(); ++ObjIndex)
	{
		UObject* Object = InputObjects[ObjIndex];
		WouldBeUsedOut[ObjIndex] = CanBuildTarget(Object, TargetTypeInfo);
		Count += WouldBeUsedOut[ObjIndex];
	}
	return Count;
}

TArray<UToolTarget*> UToolTargetFactory::BuildTargets(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WasUsedOut)
{
	TArray<UToolTarget*> Output;
	WasUsedOut.SetNum(InputObjects.Num());
	for (int32 ObjIndex = 0; ObjIndex < InputObjects.Num(); ++ObjIndex)
	{
		UObject* Object = InputObjects[ObjIndex];
		WasUsedOut[ObjIndex] = CanBuildTarget(Object, TargetTypeInfo);
		if (WasUsedOut[ObjIndex])
		{
			Output.Add(BuildTarget(Object, TargetTypeInfo));
		}
	}

	return Output;
}

UToolTarget* UToolTargetFactory::BuildFirstTarget(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WasUsedOut)
{
	WasUsedOut.SetNumZeroed(InputObjects.Num());
	for (int32 ObjIndex = 0; ObjIndex < InputObjects.Num(); ++ObjIndex)
	{
		UObject* Object = InputObjects[ObjIndex];
		WasUsedOut[ObjIndex] = CanBuildTarget(Object, TargetTypeInfo);
		if (WasUsedOut[ObjIndex])
		{
			return BuildTarget(Object, TargetTypeInfo);
		}
	}

	return nullptr;
}
