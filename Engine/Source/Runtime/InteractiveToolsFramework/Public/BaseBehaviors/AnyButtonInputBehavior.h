// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputState.h"
#include "Math/Ray.h"
#include "Math/Vector2D.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AnyButtonInputBehavior.generated.h"

class UObject;

/**
 * UAnyButtonInputBehavior is a base behavior that provides a generic
 * interface to a TargetButton on a physical Input Device. You can subclass
 * UAnyButtonInputBehavior to write InputBehaviors that can work independent
 * of a particular device type or button, by using the UAnyButtonInputBehavior functions below.
 * 
 * The target device button is selected using the .ButtonNumber property, or you can
 * override the relevant GetXButtonState() function if you need more control.
 * 
 *  @todo spatial controllers
 *  @todo support tablet fingers
 *  @todo support gamepad?
 */
UCLASS(MinimalAPI)
class UAnyButtonInputBehavior : public UInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UAnyButtonInputBehavior();

	/** Return set of devices supported by this behavior */
	INTERACTIVETOOLSFRAMEWORK_API virtual EInputDevices GetSupportedDevices() override;

	/** @return true if Target Button has been pressed this frame */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsPressed(const FInputDeviceState& input);

	/** @return true if Target Button is currently held down */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsDown(const FInputDeviceState& input);

	/** @return true if Target Button was released this frame */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsReleased(const FInputDeviceState& input);

	/** @return true if Target Button was double clicked this frame */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsDoubleClicked(const FInputDeviceState& input);
	
	/** @return current 2D position of Target Device, or zero if device does not have 2D position */
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector2D GetClickPoint(const FInputDeviceState& input);

	/** @return current 3D world ray for Target Device position */
	INTERACTIVETOOLSFRAMEWORK_API virtual FRay GetWorldRay(const FInputDeviceState& input);
	
	/** @return current 3D world ray and optional 2D position for Target Device */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputDeviceRay GetDeviceRay(const FInputDeviceState& input);

	/** @return last-active supported Device */
	INTERACTIVETOOLSFRAMEWORK_API EInputDevices GetActiveDevice() const;


public:
	/** Configure the target Mouse button to be the left button */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUseLeftMouseButton();
	/** Configure the target Mouse button to be the middle button */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUseMiddleMouseButton();
	/** Configure the target Mouse button to be the right button */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUseRightMouseButton();
	/** Configure a custom target mouse button */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUseCustomMouseButton(TUniqueFunction<FDeviceButtonState(const FInputDeviceState& Input)>);


protected:
	/** Which device is currently active */
	EInputDevices ActiveDevice;

	/** Returns FDeviceButtonState for target mouse button */
	TUniqueFunction<FDeviceButtonState(const FInputDeviceState& Input)> GetMouseButtonStateFunc;

	/** Returns FDeviceButtonState for target active device button */
	INTERACTIVETOOLSFRAMEWORK_API FDeviceButtonState GetButtonStateFunc(const FInputDeviceState& Input);
};
