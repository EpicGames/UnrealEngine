// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InputState.h"

/**
 * IModifierToggleBehaviorTarget is an interface that InputBehaviors can use to notify
 * a target about modifier toggle states (like shift key being down, etc).
 * The meaning of the modifier ID is client-defined (generally provided to the InputBehavior in a registration step)
 */
class IModifierToggleBehaviorTarget
{
public:
	virtual ~IModifierToggleBehaviorTarget() = default;

	/**
	 * Notify target of current modifier state
	 * @param ModifierID client-defined integer that identifiers modifier
	 * @param bIsOn is modifier current on or off
	 */
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn)
	{
	}

	/**
	 * Notify Target that capture is ending
	 */
	virtual void OnForceEndCapture()
	{
	}
};

/**
 * IKeyInputBehaviorTarget is an interface that InputBehaviors can use to notify
 * a target about keyboard buttons toggling states, such as pressed or released.
 */
class IKeyInputBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IKeyInputBehaviorTarget() = default;

	virtual void OnKeyPressed(const FKey& KeyID)
	{
	}

	virtual void OnKeyReleased(const FKey& KeyID)
	{
	}

	virtual void OnForceEndCapture()
	{
	}
};


/**
 * Functions required to apply standard "Click" state machines to a target object.
 * See USingleClickBehavior for an example of this kind of state machine.
 */
class IClickBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IClickBehaviorTarget() = default;

	/**
	 * Test if target is hit by a click
	 * @param ClickPos device position/ray at click point
	 * @return hit information at this point
	 */
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) = 0;


	/**
	 * Notify Target that click ocurred
	 * @param ClickPos device position/ray at click point
	 */
	virtual void OnClicked(const FInputDeviceRay& ClickPos) = 0;
};




/**
 * Functions required to apply standard "Click-Drag" state machines to a target object.
 * See UClickDragBehavior for an example of this kind of state machine.
 */
class IClickDragBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IClickDragBehaviorTarget() = default;

	/**
	 * Test if target can begin click-drag interaction at this point
	 * @param PressPos device position/ray at click point
	 * @return hit information at this point
	 */
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) = 0;


	/**
	 * Notify Target that click press ocurred
	 * @param PressPos device position/ray at click point
	 */
	virtual void OnClickPress(const FInputDeviceRay& PressPos) = 0;

	/**
	 * Notify Target that input position has changed
	 * @param DragPos device position/ray at click point
	 */
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) = 0;

	/**
	 * Notify Target that click release occurred
	 * @param ReleasePos device position/ray at click point
	 */
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) = 0;

	/**
	 * Notify Target that click-drag sequence has been explicitly terminated (eg by escape key)
	 */
	virtual void OnTerminateDragSequence() = 0;
};


/**
 * Functions required to apply mouse wheel behavior
 */
class IMouseWheelBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IMouseWheelBehaviorTarget() = default;

	/**
	 * The result's bHit property determines whether the mouse wheel action will be captured.
	 * (Perhaps the mouse wheel only does something when mousing over some part of a mesh)
	 * 
	 * @param CurrentPos device position/ray at point where mouse wheel is engaged
	 * @return hit information at this point
	 */
	virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos) = 0;


	/**
	 *
	 * @param CurrentPos device position/ray at point where mouse wheel is engaged
	 */
	virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos) = 0;

	/**
	 *
	 * @param CurrentPos device position/ray at point where mouse wheel is engaged
	 */
	virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos) = 0;

};






/**
 * Target interface used by InputBehaviors that want to implement a multi-click sequence
 * (eg such as drawing a polygon with multiple clicks)
 */
class IClickSequenceBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IClickSequenceBehaviorTarget() = default;

	/**
	 * Notify Target device position has changed but a click sequence hasn't begun yet (eg for interactive previews)
	 * @param ClickPos device position/ray at click point
	 */
	virtual void OnBeginSequencePreview(const FInputDeviceRay& ClickPos) { }

	/**
	 * Test if target would like to begin sequence based on this click. Gets checked both 
	 * on mouse down and mouse up.
	 * @param ClickPos device position/ray at click point
	 * @return true if target wants to begin sequence
	 */
	virtual bool CanBeginClickSequence(const FInputDeviceRay& ClickPos) = 0;

	/**
	 * Notify Target about the first click in the sequence.
	 * @param ClickPos device position/ray at click point
	 */
	virtual void OnBeginClickSequence(const FInputDeviceRay& ClickPos) = 0;
	
	/**
	 * Notify Target device position has changed but a click hasn't ocurred yet (eg for interactive previews)
	 * @param ClickPos device position/ray at click point
	 */
	virtual void OnNextSequencePreview(const FInputDeviceRay& ClickPos) { }

	/**
	 * Notify Target about next click in sqeuence
	 * @param ClickPos device position/ray at click point
	 * @return false if Target wants to terminate sequence
	 */
	virtual bool OnNextSequenceClick(const FInputDeviceRay& ClickPos) = 0;

	/**
	 * Notify Target that click sequence has been explicitly terminated (eg by escape key, cancel tool, etc).
	 * Also called if sequence is terminated from querying target with RequestAbortClickSequence().
	 */
	virtual void OnTerminateClickSequence() = 0;

	/**
	 * Target overrides this and returns true if it wants to abort click sequence.
	 * Behavior checks every update and if this ever returns true, terminates sequence
	 */
	virtual bool RequestAbortClickSequence() { return false; }
};





/**
 * IHoverBehaviorTarget allows Behaviors to notify Tools/etc about
 * device event data in a generic way, without requiring that all Tools
 * know about the concept of Hovering.
 */
class IHoverBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IHoverBehaviorTarget() = default;

	/**
	 * Do hover hit-test
	 */
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) = 0;

	/**
	 * Initialize hover sequence at given position
	 */
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) = 0;

	/**
	 * Update active hover sequence with new input position
	 */
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) = 0;

	/**
	 * Terminate active hover sequence
	 */
	virtual void OnEndHover() = 0;
};


/**
 * ITwoAxisPropertyEditBehaviorTarget allows tools to
 * expose properties to drag-based property edit behaviors.
 */
class ITwoAxisPropertyEditBehaviorTarget
{
public:

	/* IPropertyInterface encapsulates a virtual 'property' (which may or may not map to a single member or property) */
	struct IPropertyInterface
	{
		virtual ~IPropertyInterface() = default;
		
		virtual FText GetName() = 0;								// Get a name for this virtual property (for display purposes).
		virtual float GetValue() = 0;
		virtual void SetValue(float NewValue) = 0;
		virtual float GetEditRate() { return 0.003f; }				// Scales how quickly this property is edited with respect to a fixed mouse movement.
		virtual float MutateDelta(float Delta) { return Delta; }	// Provides a hook to further augment the delta that is about to be applied to this property.
		virtual bool IsEnabled() { return false; }
	};
	
public:
	
	virtual ~ITwoAxisPropertyEditBehaviorTarget() = default;
	
	virtual IPropertyInterface* GetHorizontalProperty() = 0;
	virtual IPropertyInterface* GetVerticalProperty() = 0;
	
	virtual FKey GetCaptureKey() { return EKeys::B; }
	virtual void PostDragUpdated() {}
};
