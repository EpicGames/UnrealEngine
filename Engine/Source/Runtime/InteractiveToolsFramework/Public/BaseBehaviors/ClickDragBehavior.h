// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputState.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClickDragBehavior.generated.h"

class UObject;


/**
 * UClickDragInputBehavior implements a standard "button-click-drag"-style input behavior.
 * An IClickDragBehaviorTarget instance must be provided which is manipulated by this behavior.
 * 
 * The state machine works as follows:
 *    1) on input-device-button-press, call Target::CanBeginClickDragSequence to determine if capture should begin
 *    2) on input-device-move, call Target::OnClickDrag
 *    3) on input-device-button-release, call Target::OnClickRelease
 *    
 * If a ForceEndCapture occurs we call Target::OnTerminateDragSequence   
 */
UCLASS(MinimalAPI)
class UClickDragInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UClickDragInputBehavior();

	/**
	 * The modifier set for this behavior
	 */
	FInputBehaviorModifierStates Modifiers;

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of hit-test and on-clicked functions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IClickDragBehaviorTarget* Target);


	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;


	// UInputBehavior implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCapturePriority GetPriority() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;

	/**
	 * If true, then we will update Modifier states in UpdateCapture(). This defaults to false because
	 * in most cases during a capture you don't want this, eg in a brush interaction, if you have a shift-to-smooth modifier,
	 * you don't want to toggle this on/off during a sculpting stroke.
	 */
	UPROPERTY()
	bool bUpdateModifiersDuringDrag = false;

protected:
	/** Click Target object */
	IClickDragBehaviorTarget* Target;

	/** set to true if we are in a capture */
	bool bInClickDrag = false;

	/**
	 * Internal function that forwards click evens to Target::OnClickPress, you can customize behavior here
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickPressInternal(const FInputDeviceState& Input, EInputCaptureSide Side);

	/**
	 * Internal function that forwards click evens to Target::OnClickDrag, you can customize behavior here
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDragInternal(const FInputDeviceState& Input, const FInputCaptureData& Data);

	/**
	 * Internal function that forwards click evens to Target::OnClickRelease, you can customize behavior here
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickReleaseInternal(const FInputDeviceState& Input, const FInputCaptureData& Data);
};





/**
 * ULocalClickDragInputBehavior is an implementation of UClickDragInputBehavior that also implements IClickDragBehaviorTarget directly,
 * via a set of local lambda functions. To use/customize this class the client replaces the lambda functions with their own.
 * This avoids having to create a second IClickDragBehaviorTarget implementation for trivial use-cases.
 */
UCLASS(MinimalAPI)
class ULocalClickDragInputBehavior : public UClickDragInputBehavior, public IClickDragBehaviorTarget
{
	GENERATED_BODY()
protected:
	using UClickDragInputBehavior::Initialize;

public:
	/** Call this to initialize the class */
	virtual void Initialize()
	{
		this->Initialize(this);
	}

	/** lambda implementation of CanBeginClickDragSequence */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay& PressPos)> CanBeginClickDragFunc = [](const FInputDeviceRay&) { return FInputRayHit(); };

	/** lambda implementation of OnClickPress */
	TUniqueFunction<void(const FInputDeviceRay& PressPos)> OnClickPressFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnClickDrag */
	TUniqueFunction<void(const FInputDeviceRay& PressPos)> OnClickDragFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnClickRelease */
	TUniqueFunction<void(const FInputDeviceRay& ReleasePos)> OnClickReleaseFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnTerminateDragSequence */
	TUniqueFunction<void()> OnTerminateFunc = []() {};


public:
	//
	// IClickDragBehaviorTarget implementation
	//
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override
	{
		return CanBeginClickDragFunc(PressPos);
	}

	virtual void OnClickPress(const FInputDeviceRay& PressPos) override
	{
		OnClickPressFunc(PressPos);
	}

	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override
	{
		OnClickDragFunc(DragPos);
	}

	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override
	{
		OnClickReleaseFunc(ReleasePos);
	}

	virtual void OnTerminateDragSequence() override
	{
		OnTerminateFunc();
	}
};
