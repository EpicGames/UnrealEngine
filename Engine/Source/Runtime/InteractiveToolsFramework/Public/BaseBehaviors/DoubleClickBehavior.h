// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTargetInterfaces.h"
#include "CoreMinimal.h"
#include "InputState.h"
#include "SingleClickBehavior.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DoubleClickBehavior.generated.h"

class UObject;


/**
 * UDoubleClickInputBehavior implements a standard "button-click"-style input behavior.
 * The state machine works as follows:
 *    1) on input-device-button-double-click, hit-test the target. If hit, begin capture
 *    2) on input-device-button-release, hit-test the target. If hit, call Target::OnClicked(). If not hit, ignore click.
 *    
 * The second hit-test is required to allow the click to be "cancelled" by moving away
 * from the target. This is standard GUI behavior. You can disable this second hit test
 * using the .HitTestOnRelease property. This is strongly discouraged.
 *    
 * The hit-test and on-clicked functions are provided by a IClickBehaviorTarget instance.
 *
 * The expected sequence of mouse events for a double click is:
 *    a. MouseDown
 *    b. MouseUp
 *    c. MouseDoubleClick   <-- State machine starts here.
 *    d. MouseUp
 */
UCLASS(MinimalAPI)
class UDoubleClickInputBehavior : public USingleClickInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UDoubleClickInputBehavior();

	// UInputBehavior implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
};

/**
 * An implementation of UDoubleClickInputBehavior that also implements IClickBehaviorTarget directly, via a set 
 * of local lambda functions. To use/customize this class, the client replaces the lambda functions with their own.
 * This avoids having to create a separate IClickBehaviorTarget implementation for trivial use-cases.
 */
UCLASS(MinimalAPI)
class ULocalDoubleClickInputBehavior : public UDoubleClickInputBehavior, public IClickBehaviorTarget
{
	GENERATED_BODY()
protected:
	using UDoubleClickInputBehavior::Initialize;

public:
	/** Call this to initialize the class */
	virtual void Initialize()
	{
		this->Initialize(this);
	}

	/** lambda implementation of IsHitByClick */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay&)> IsHitByClickFunc = [](const FInputDeviceRay& ClickPos) { return FInputRayHit(); };

	/** lambda implementation of OnClicked */
	TUniqueFunction<void(const FInputDeviceRay&)> OnClickedFunc = [](const FInputDeviceRay& ClickPos) {};

	/** lambda implementation of OnUpdateModifierState */
	TUniqueFunction< void(int, bool) > OnUpdateModifierStateFunc = [](int ModifierID, bool bIsOn) {};

public:
	// IClickBehaviorTarget implementation

	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override
	{
		return IsHitByClickFunc(ClickPos);
	}

	virtual void OnClicked(const FInputDeviceRay& ClickPos) override
	{
		return OnClickedFunc(ClickPos);
	}

	// IModifierToggleBehaviorTarget implementation
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn)
	{
		return OnUpdateModifierStateFunc(ModifierID,bIsOn);
	}
};
