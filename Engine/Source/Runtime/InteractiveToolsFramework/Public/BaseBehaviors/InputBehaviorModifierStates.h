// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputState.h"
#include "BehaviorTargetInterfaces.h"

/**
 * FInputBehaviorModifierStates is an object that can be placed in an InputBehavior to allow
 * users of the behavior to request that they be notified about modifier keys/buttons/etc state.
 * 
 * We don't know ahead of time what might be used to determine modifier state, which input
 * devices might be used, etc. So the user has to register (ModifierID,ModifierTestFunction) pairs.
 * The behavior then calls UpdateModifiers() which will query each of the test functions and
 * notify the target object about the state of the modifier.
 */
class FInputBehaviorModifierStates
{
public:
	typedef TFunction<bool(const FInputDeviceState&)> FModifierTestFunction;

protected:
	/** List of modifier IDs that have been registered */
	TArray<int> ModifierIDs;
	/** The modifier test function for each ID */
	TMap<int, FModifierTestFunction> ModifierTests;
	/** The modifier states for each ID */
	TMap<int, bool> ModifierStates;

public:
	
	/**
	 * Register a modifier ID and an associated test function
	 * @param ModifierID the modifier ID
	 * @param ModifierTest the test function
	 */
	void RegisterModifier(int ModifierID, const FModifierTestFunction& ModifierTest)
	{
		check(ModifierIDs.Contains(ModifierID) == false);
		ModifierIDs.Add(ModifierID);
		ModifierTests.Add(ModifierID, ModifierTest);
		ModifierStates.Add(ModifierID, false);
	}


	/**
	 * @return true if any modifiers have been registered
	 */
	bool HasModifiers() const
	{
		return (ModifierIDs.Num() > 0);
	}

	/**
	 * @return The count of active modifiers 
	 */
	int32 GetActiveModifierCount() const
	{
		int32 Result = 0;
		for (const TPair<int, bool>& Pair : ModifierStates)
		{
			if (Pair.Value)
			{
				Result += 1;
			}
		}
		return Result;
	}

	/**
	 * Look up the current state of each registered modifier and pass to the target
	 * @param Input the current input device state
	 * @param ModifiersTarget the target that is interested in the modifier values
	 */
	void UpdateModifiers(const FInputDeviceState& Input, IModifierToggleBehaviorTarget* ModifiersTarget)
	{
		for (int ModifierID : ModifierIDs)
		{
			const bool bIsOn = ModifierTests[ModifierID](Input);
			ModifierStates[ModifierID] = bIsOn; 
			ModifiersTarget->OnUpdateModifierState(ModifierID, bIsOn);
		}
	}
	
	/**
	 * Look up the current state of each registered modifier and cache their values.
	 * @param Input the current input device state
	 */
	void CacheModifierState(const FInputDeviceState& Input)
	{
		for (int ModifierID : ModifierIDs)
		{
			const bool bIsOn = ModifierTests[ModifierID](Input);
			ModifierStates[ModifierID] = bIsOn;
		}
	}
};