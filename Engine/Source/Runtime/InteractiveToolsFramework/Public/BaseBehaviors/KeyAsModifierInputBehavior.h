// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputCoreTypes.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "KeyAsModifierInputBehavior.generated.h"

class IModifierToggleBehaviorTarget;
class UObject;


/**
 * UKeyAsModifierInputBehavior converts a specific key press/release into
 * a "Modifier" toggle via the IModifierToggleBehaviorTarget interface. It does
 * not capture the key press; rather, it updates the modifier its WantsCapture call.
 * This means that the modifier won't be updated if another behavior captures the
 * keyboard.
 */
UCLASS(MinimalAPI)
class UKeyAsModifierInputBehavior : public UInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UKeyAsModifierInputBehavior();

	virtual EInputDevices GetSupportedDevices() override
	{
		return EInputDevices::Keyboard;
	}

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of modifier-toggle behavior
	 * @param ModifierID integer ID that identifiers the modifier toggle
	 * @param ModifierKey the key that will be used as the modifier toggle
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IModifierToggleBehaviorTarget* Target, int ModifierID, const FKey& ModifierKey);

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of modifier-toggle behavior
	 * @param ModifierID integer ID that identifiers the modifier toggle
	 * @param ModifierCheckFunc the function that will be checked to set the the modifier.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void Initialize(IModifierToggleBehaviorTarget* TargetIn, int ModifierID, TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc);

	// UInputBehavior implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool WantsForceEndCapture() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;


protected:
	/** Modifier Target object */
	IModifierToggleBehaviorTarget* Target;

	/** Modifier set for this behavior, internally initialized with check on ModifierKey */
	FInputBehaviorModifierStates Modifiers;
};

/**
 * A version of UKeyAsModifierInputBehavior for mouse buttons
 */
UCLASS(MinimalAPI)
class UMouseButtonAsModifierInputBehavior : public UKeyAsModifierInputBehavior
{
	GENERATED_BODY()

	virtual EInputDevices GetSupportedDevices() override
	{
		return EInputDevices::Mouse;
	}
};
