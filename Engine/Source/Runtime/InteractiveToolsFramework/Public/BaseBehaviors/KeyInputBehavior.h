// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputCoreTypes.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "KeyInputBehavior.generated.h"

class IModifierToggleBehaviorTarget;
class UObject;


/**
 * UKeyInputBehavior provides a keyboard capturing behavior that works with single or multiple keys. Provided with a single key, the behavior triggers
 * OnKeyPressed and OnKeyReleased events upon seeing the target key pressed down and released for the first time, ignoring any other key presses.
 * When provided with multiple keys, the Behavior has variable behavior depending on whether bRequireAllKeys is set.
 * 
 * If true, the behavior sequence is as follows:
 * 
 *   1. Initiate capture when any of the target keys are pressed.
 *   2. Continue capture until all target keys are pressed simultaneously
 *   3. Upon seeing the last key to complete the full set of target keys, issue an OnKeyPressed for whichever key completed the requirement.
 *		3b. If any of the target keys are released after the full set was pressed, issue an OnKeyReleased for whichever key was released, then end Capture.
 *   4. If at any point all target keys are released after capture begins, end capture. 
 * 
 * If false, the behavior sequence is as follows:
 * 
 *   1. Initiate capture when any of the target keys are pressed.
 *   2. Continue capture while any of the target keys are still pressed.
 *   3. Issue an OnKeyPressed for any target key pressed during the capture period.
 *   4. Issue an OnKeyReleased for any target key released during the capture period.
 *   5. If at any point all target keys are released, end capture.
 * 
 */
UCLASS(MinimalAPI)
class UKeyInputBehavior : public UInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UKeyInputBehavior();

	virtual EInputDevices GetSupportedDevices() override
	{
		return EInputDevices::Keyboard;
	}

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of modifier-toggle behavior
	 * @param Key FKey that indicates the keyboard key for the behavior to watch for	 
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IKeyInputBehaviorTarget* Target, const FKey& Key);

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of modifier-toggle behavior
	 * @param Keys array of FKey that indicates the keyboard keys for the behavior to watch for	 
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IKeyInputBehaviorTarget* Target, const TArray<FKey>& Keys);

	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;

	// UInputBehavior implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool WantsForceEndCapture() override;

	/**
	 * @return Whether the given key is detected as pressed.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool IsKeyPressed(FKey InKey) const;

	/**
    * The modifier set for this behavior
    */
	FInputBehaviorModifierStates Modifiers;

	/**
	* If true, behavior requires all keys provided at initialization to be active at the same time in order to count as a "key down" event. If false, any key in the list will trigger the key down event.
	*/
	bool bRequireAllKeys = true;

	/**
	* If true, behavior will update modifiers also when not capturing. If false, modifiers will be updated only while capturing.
	*/
	bool bAlwaysUpdateModifiers = false;

protected:
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnKeyPressedInternal(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnKeyReleasedInternal(const FInputDeviceState& Input);

	/** Modifier Target object */
	IKeyInputBehaviorTarget* Target;

	TArray<FKey> TargetKeys;
	TArray<bool> KeyActivations;
	
private:
	void InitializeKeyActivations();
	bool IsTargetedKey(const FInputDeviceState& Input);
	bool UpdateActivations(const FInputDeviceState& Input, bool bEmitOnChange);
	bool IsAnyKeyPressed() const;
	bool AreAllKeysPressed() const;

	bool bAllKeysSeenPressed = false;
};
