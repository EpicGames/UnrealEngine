// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MouseHoverBehavior.generated.h"

class UObject;



/**
 * Trivial InputBehavior that forwards InputBehavior hover events to a Target object via
 * the IHoverBehaviorTarget interface.
 */
UCLASS(MinimalAPI)
class UMouseHoverBehavior : public UInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UMouseHoverBehavior();

	/**
	 * The modifier set for this behavior
	 */
	FInputBehaviorModifierStates Modifiers;

	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IHoverBehaviorTarget* Target);

	/**
	 * WantsHoverCapture() will only return capture request if this function returns true (or is null)
	 *
	 * Note that unlike click behaviors, hover behaviors continue to receive WantsCapture calls even when
	 * they hold capture (and end if they fail to request capture), so an active hover behavior will end
	 * if its HoverModifierCheckFunc returns false.
	 */
	TFunction<bool(const FInputDeviceState&)> HoverModifierCheckFunc = nullptr;

	// UInputBehavior hover implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual EInputDevices GetSupportedDevices() override;

	INTERACTIVETOOLSFRAMEWORK_API virtual bool WantsHoverEvents() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsHoverCapture(const FInputDeviceState& InputState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginHoverCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateHoverCapture(const FInputDeviceState& InputState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndHoverCapture() override;

protected:
	IHoverBehaviorTarget* Target;
};


/**
 * An implementation of UMouseHoverBehavior that also implements IHoverBehaviorTarget directly, via a set 
 * of local lambda functions. To use/customize this class, the client replaces the lambda functions with their own.
 * This avoids having to create a separate IHoverBehaviorTarget implementation for trivial use-cases.
 */
UCLASS(MinimalAPI)
class ULocalMouseHoverBehavior : public UMouseHoverBehavior, public IHoverBehaviorTarget
{
	GENERATED_BODY()
protected:
	using UMouseHoverBehavior::Initialize;

public:
	/** Call this to initialize the class */
	virtual void Initialize()
	{
		this->Initialize(this);
	}

	/** lambda implementation of BeginHoverSequenceHitTest */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay& PressPos)> BeginHitTestFunc = [](const FInputDeviceRay&) { return FInputRayHit(); };

	/** lambda implementation of OnBeginHover */
	TUniqueFunction<void(const FInputDeviceRay& PressPos)> OnBeginHoverFunc = [](const FInputDeviceRay&) {};

	/** lambda implementation of OnUpdateHover */
	TUniqueFunction<bool(const FInputDeviceRay& PressPos)> OnUpdateHoverFunc = [](const FInputDeviceRay&) { return false; };

	/** lambda implementation of OnEndHover */
	TUniqueFunction<void()> OnEndHoverFunc = []() {};

	/** lambda implementation of OnUpdateModifierState */
	TUniqueFunction< void(int, bool) > OnUpdateModifierStateFunc = [](int ModifierID, bool bIsOn) {};

public:
	// IHoverBehaviorTarget implementation
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override
	{
		return BeginHitTestFunc(PressPos);
	}

	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override
	{
		OnBeginHoverFunc(DevicePos);
	}

	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override
	{
		return OnUpdateHoverFunc(DevicePos);
	}

	virtual void OnEndHover() override
	{
		OnEndHoverFunc();
	}

	// IModifierToggleBehaviorTarget implementation
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn)
	{
		return OnUpdateModifierStateFunc(ModifierID, bIsOn);
	}
};
