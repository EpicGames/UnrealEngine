// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputBehaviorModifierStates.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MouseWheelBehavior.generated.h"

class IMouseWheelBehaviorTarget;
class UObject;
struct FInputDeviceState;

UCLASS(MinimalAPI)
class UMouseWheelInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UMouseWheelInputBehavior();

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of hit-test and on-clicked functions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IMouseWheelBehaviorTarget* Target);


	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;


	// UInputBehavior implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;

	/**
	 * The modifier set for this behavior
	 */
	FInputBehaviorModifierStates Modifiers;

protected:
	/**
	 *
	 */
	IMouseWheelBehaviorTarget* Target;

};
