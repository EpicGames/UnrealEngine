// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MouseWheelBehavior.h"
#include "MouseWheelModifierInputBehavior.generated.h"

#define UE_API INTERACTIVETOOLSFRAMEWORK_API

/**
 * 
 */
UCLASS(MinimalAPI)
class UMouseWheelModifierInputBehavior : public UMouseWheelInputBehavior
{
	GENERATED_BODY()

public:
	UE_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InputState) override;
};

#undef UE_API
