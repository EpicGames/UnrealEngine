// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "BehaviorTargetInterfaces.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MultiClickSequenceInputBehavior.generated.h"

class IClickSequenceBehaviorTarget;
class UObject;
struct FInputDeviceState;


/**
 * UMultiClickSequenceInputBehavior implements a generic multi-click-sequence input behavior.
 * For example this behavior could be used to implement a multi-click polygon-drawing interaction.
 *
 * The internal state machine works as follows:
 *   1) on input-device-button-press, check if target wants to begin sequence. If so, begin capture.
 *   2) on button *release*, check if target wants to continue or terminate sequence
 *       a) if terminate, release capture
 *       b) if continue, do nothing (capture continues between presses)
 *
 * The target will receive "preview" notifications (basically hover) during updates where there is
 * not a release. This can be used to (eg) update a rubber-band selection end point
 * 
 * @todo it may be better to implement this as multiple captures, and use hover callbacks to 
 * do the between-capture previews. holding capture across mouse release is not ideal.
 */
UCLASS(MinimalAPI)
class UMultiClickSequenceInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UMultiClickSequenceInputBehavior();

	/**
	 * The modifier set for this behavior
	 */
	FInputBehaviorModifierStates Modifiers;

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of hit-test and on-clicked functions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IClickSequenceBehaviorTarget* Target);


	/**
	 * The first click will only be accepted if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;

	/**
	 * WantsHoverCapture() will only return capture request if this function returns true (or is null)
	 *
	 * Note that unlike click behaviors, hover behaviors continue to receive WantsCapture calls even when
	 * they hold capture (and end if they fail to request capture), so an active hover behavior will end
	 * if its HoverModifierCheckFunc returns false.
	 */
	TFunction<bool(const FInputDeviceState&)> HoverModifierCheckFunc = nullptr;

	// UInputBehavior implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual bool WantsHoverEvents() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsHoverCapture(const FInputDeviceState& InputState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginHoverCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateHoverCapture(const FInputDeviceState& InputState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndHoverCapture() override;


public:

protected:
	enum class ESequenceState
	{
		NotStarted,
		WaitingForNextClick,
	};

	/** Click Target object */
	IClickSequenceBehaviorTarget* Target;

	ESequenceState State = ESequenceState::NotStarted;
};
