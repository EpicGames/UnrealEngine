// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "BehaviorTargetInterfaces.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputBehaviorModifierStates.h"
#include "InputState.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SingleClickBehavior.generated.h"

class UObject;


/**
 * USingleClickInputBehavior implements a standard "button-click"-style input behavior.
 * The state machine works as follows:
 *    1) on input-device-button-press, hit-test the target. If hit, begin capture
 *    2) on input-device-button-release, hit-test the target. If hit, call Target::OnClicked(). If not hit, ignore click.
 *    
 * The second hit-test is required to allow the click to be "cancelled" by moving away
 * from the target. This is standard GUI behavior. You can disable this second hit test
 * using the .HitTestOnRelease property. This is strongly discouraged.
 *    
 * The hit-test and on-clicked functions are provided by a IClickBehaviorTarget instance.
 */
UCLASS(MinimalAPI)
class USingleClickInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API USingleClickInputBehavior();

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of hit-test and on-clicked functions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IClickBehaviorTarget* Target);


	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;


	// UInputBehavior implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCapturePriority GetPriority() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;


public:
	/** Hit-test is repeated on release (standard behavior). If false, */
	UPROPERTY()
	bool HitTestOnRelease;

	/**
	 * The modifier set for this behavior
	 */
	FInputBehaviorModifierStates Modifiers;


protected:
	/** Click Target object */
	IClickBehaviorTarget* Target;

	/**
	 * Internal function that forwards click evens to Target::OnClicked, you can customize behavior here
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Clicked(const FInputDeviceState& Input, const FInputCaptureData& Data);
};

/**
 * An implementation of USingleClickInputBehavior that also implements IClickBehaviorTarget directly, via a set 
 * of local lambda functions. To use/customize this class, the client replaces the lambda functions with their own.
 * This avoids having to create a separate IClickBehaviorTarget implementation for trivial use-cases.
 */
UCLASS(MinimalAPI)
class ULocalSingleClickInputBehavior : public USingleClickInputBehavior, public IClickBehaviorTarget
{
	GENERATED_BODY()
protected:
	using USingleClickInputBehavior::Initialize;

public:
	/** Call this to initialize the class */
	virtual void Initialize()
	{
		this->Initialize(this);
	}

	/** lambda implementation of IsHitByClick */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay&)> IsHitByClickFunc = [](const FInputDeviceRay& ClickPos) { return FInputRayHit(); };

	/** lambda implementation of OnClicked */
	TUniqueFunction<void(const FInputDeviceRay&)> OnClickedFunc = [](const FInputDeviceRay& ClickPos) {};

	/** lambda implementation of OnUpdateModifierState */
	TUniqueFunction< void(int, bool) > OnUpdateModifierStateFunc = [](int ModifierID, bool bIsOn) {};

public:
	// IClickBehaviorTarget implementation

	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override
	{
		return IsHitByClickFunc(ClickPos);
	}

	virtual void OnClicked(const FInputDeviceRay& ClickPos) override
	{
		return OnClickedFunc(ClickPos);
	}

	// IModifierToggleBehaviorTarget implementation
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn)
	{
		return OnUpdateModifierStateFunc(ModifierID,bIsOn);
	}
};
