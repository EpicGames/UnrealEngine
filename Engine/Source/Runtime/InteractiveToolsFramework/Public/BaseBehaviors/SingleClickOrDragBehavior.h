// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "BehaviorTargetInterfaces.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputBehaviorModifierStates.h"
#include "Math/Ray.h"
#include "Math/Vector2D.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SingleClickOrDragBehavior.generated.h"

class IClickBehaviorTarget;
class IClickDragBehaviorTarget;
class UObject;
struct FInputDeviceState;


/**
 * USingleClickOrDragInputBehavior is a combination of a USingleClickBehavior and UClickDragBehavior,
 * and allows for the common UI interaction where a click-and-release does one action, but if the mouse 
 * is moved, then a drag interaction is started. For example click-to-select is often combined with
 * a drag-marquee-rectangle in this way. This can be directly implemented with a UClickDragBehavior but
 * requires the client to (eg) detect movement thresholds, etc. This class encapsulates all that state/logic.
 * 
 * The .ClickDistanceThreshold parameter determines how far the mouse must move (in whatever device units are in use)
 * to switch from a click to drag interaction
 * 
 * The .bBeginDragIfClickTargetNotHit parameter determines if the drag interaction will be immediately initiated 
 * if the initial 'click' mouse-down does not hit a valid clickable target. Defaults to true. 
 *
 * The hit-test and on-clicked functions are provided by a IClickBehaviorTarget instance, while an
 * IClickDragBehaviorTarget provides the can-click-drag/begin-drag/update-drag/end-drag functionality.
 */
UCLASS(MinimalAPI)
class USingleClickOrDragInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:

	/**
	* The modifier set for this behavior
	*/
	FInputBehaviorModifierStates Modifiers;


	/**
	 * Set the targets for Click and Drag interactions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IClickBehaviorTarget* ClickTarget, IClickDragBehaviorTarget* DragTarget);

	/**
	 * Change the Drag BehaviorTarget
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDragTarget(IClickDragBehaviorTarget* DragTarget);


	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;


	// UInputBehavior implementation
	
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCapturePriority GetPriority() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;



	/** If true (default), then if the click-mouse-down does not hit a valid click target (determined by IClickBehaviorTarget::IsHitByClick), then the Drag will be initiated */
	UPROPERTY()
	bool bBeginDragIfClickTargetNotHit = true;


	/** If the device moves more than this distance in 2D (pixel?) units, the interaction switches from click to drag */
	UPROPERTY()
	float ClickDistanceThreshold = 5.0;


protected:
	/** Click Target object */
	IClickBehaviorTarget* ClickTarget;

	/** Drag Target object */
	IClickDragBehaviorTarget* DragTarget;

	/** Click-down position, to determine if mouse has moved far enough to swap to drag interaction */
	FVector2D MouseDownPosition2D;
	FRay MouseDownRay;

	/** Device capture */
	EInputCaptureSide CaptureSide;

	/** set to true if we are in an active drag capture, eg after rejecting possible click */
	bool bInDrag = false;


protected:

	// flag used to communicate between WantsCapture and BeginCapture
	bool bImmediatelyBeginDragInBeginCapture = false;

	/**
	 * Internal function that forwards click evens to ClickTarget::OnClicked, you can customize behavior here
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickedInternal(const FInputDeviceState& Input, const FInputCaptureData& Data);

	/**
	* Internal function that forwards click evens to DragTarget::OnClickPress, you can customize behavior here
	*/
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDragPressInternal(const FInputDeviceState& Input, EInputCaptureSide Side);

	/**
	* Internal function that forwards click evens to DragTarget::OnClickDrag, you can customize behavior here
	*/
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDragInternal(const FInputDeviceState& Input, const FInputCaptureData& Data);

	/**
	* Internal function that forwards click evens to DragTarget::OnClickRelease, you can customize behavior here
	*/
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDragReleaseInternal(const FInputDeviceState& Input, const FInputCaptureData& Data);


};

