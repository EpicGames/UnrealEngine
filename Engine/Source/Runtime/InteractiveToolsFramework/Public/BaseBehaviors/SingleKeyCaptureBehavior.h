// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputCoreTypes.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SingleKeyCaptureBehavior.generated.h"

class IModifierToggleBehaviorTarget;
class UObject;


/**
 * USingleKeyCaptureBehavior captures a key press and routes it to target via
 * the IModifierToggleBehaviorTarget interface. If you want similar behavior
 * without actually capturing the key, you should use UKeyAsModifierInputBehavior.
 */
UCLASS(MinimalAPI)
class USingleKeyCaptureBehavior : public UInputBehavior
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API USingleKeyCaptureBehavior();

	virtual EInputDevices GetSupportedDevices() override
	{
		return EInputDevices::Keyboard;
	}

	/**
	 * Initialize this behavior with the given Target. Note that though it interacts through the
	 * IModifierToggleBehaviorTarget interface, the modifier here captures keyboard input, unlike
	 * the modifiers in UKeyAsMOdifierInputBehavior and other places.
	 * 
	 * @param Target implementor of IModifierToggleBehaviorTarget
	 * @param ModifierID integer ID that identifiers the modifier toggle
	 * @param ModifierKey the key that will be used as the modifier toggle
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IModifierToggleBehaviorTarget* Target, int ModifierID, const FKey& ModifierKey);

	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 * Intended to be used for alt/ctrl/cmd/shift modifiers on the main key
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;

	// UInputBehavior implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide Side) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& Data) override;


protected:
	/** Modifier Target object */
	IModifierToggleBehaviorTarget* Target;

	/** Key that is used as modifier */
	FKey ModifierKey;

	/** Modifier set for this behavior, internally initialized with check on ModifierKey */
	FInputBehaviorModifierStates Modifiers;

	/** The key that was pressed to activate capture */
	FKey PressedButton;
};
