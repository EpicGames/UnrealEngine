// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputBehavior.h"
#include "BehaviorTargetInterfaces.h"

#include "TwoAxisPropertyEditBehavior.generated.h"

class FCanvas;

class IToolsContextRenderAPI;

/**
 * A behavior that captures a keyboard hotkey to enter a property adjustment sub-mode while the key is pressed.
 * In this sub-mode, click-dragging the mouse will begin updating the properties specified by the behavior target.
 */
UCLASS(MinimalAPI)
class UTwoAxisPropertyEditInputBehavior : public UInputBehavior
{
	GENERATED_BODY()

public:
	
	INTERACTIVETOOLSFRAMEWORK_API void Initialize(ITwoAxisPropertyEditBehaviorTarget* InTarget);

	// Used to define the initial adjustment frame or to update the current reference point alternating between editing the 2 properties.
	INTERACTIVETOOLSFRAMEWORK_API void ResetOrigin(FVector2D InScreenPosition, bool bHorizontalAdjust = true, bool bResetStartOrigin = false);
	INTERACTIVETOOLSFRAMEWORK_API void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) const;
	bool IsEditing() const { return bInDrag; }

private:

	void OnDragStart(FVector2D InScreenPosition);
	void OnDragUpdate(FVector2D InScreenPosition);
	void OnDragEnd();
	void ResetDragState();

	// UInputBehavior implementation
	// Routes input to the device specific handler functions.
	virtual EInputDevices GetSupportedDevices() override { return EInputDevices::Mouse | EInputDevices::Keyboard; }
	virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InputState) override;
	virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide) override;
	virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData) override;
	virtual void ForceEndCapture(const FInputCaptureData& CaptureData) override;

	// Mouse input handling - In these functions we assume input is from mouse device.
	// This code is responsible for invoking OnDrag* functions at the proper times.
	FInputCaptureRequest WantsMouseCapture(const FInputDeviceState& InputState);
	FInputCaptureUpdate BeginMouseCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide);
	FInputCaptureUpdate UpdateMouseCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData);
	bool IsMousePressed(const FInputDeviceState& InputState) const;
	
	// Keyboard input handling - In these functions we assume input is from keyboard device.
	// This code is responsible for keeping bKeyPressed in the proper state.
	FInputCaptureRequest WantsKeyboardCapture(const FInputDeviceState& InputState);
	FInputCaptureUpdate BeginKeyboardCapture(const FInputDeviceState& InputState, EInputCaptureSide eSide);
	FInputCaptureUpdate UpdateKeyboardCapture(const FInputDeviceState& InputState, const FInputCaptureData& CaptureData);
	bool IsKeyboardPressed(const FInputDeviceState& InputState) const;
	
private:

	// These 2 bools reflect which input we are currently capturing.
	bool bInDrag = false;
	bool bKeyPressed = false;
	
	// Data that is only valid while in a drag (while bInDrag is true13).
	struct FDragState
	{
		float StartValue = 0.f;				// Cached initial values of the property being actively edited.
		FVector2D StartOrigin;				// The screen space coordinate of the brush when the drag started.
		FVector2D CurrentOrigin;			// The screen space coordinate of the origin of the current adjustment. Reset when changing between horizontal and vertical adjustment.
		bool bAdjustingHorizontally = true;
	} State;								// Only valid if bInDrag is true
	
	ITwoAxisPropertyEditBehaviorTarget* Target = nullptr;
};



/**
 * Variant of the base behavior which allows tools to use
 * lambda functions instead of explicitly defining a behavior target.
 */
UCLASS(MinimalAPI)
class ULocalTwoAxisPropertyEditInputBehavior : public UTwoAxisPropertyEditInputBehavior, public ITwoAxisPropertyEditBehaviorTarget
{
	GENERATED_BODY()

public:
	
	/* Lambda implementation of IPropertyInterface */
	struct FPropertyInterface : public IPropertyInterface
	{
		typedef TUniqueFunction<float(void)> FGetValueSignature;
		typedef TUniqueFunction<void(float)> FSetValueSignature;
		typedef TUniqueFunction<float(float)> FMutateDeltaSignature;

		FGetValueSignature GetValueFunc;
		FSetValueSignature SetValueFunc;
		FMutateDeltaSignature MutateDeltaFunc = [](float Delta){ return Delta; };

		FText Name;
		float EditRate = 0.002f;
		bool bEnabled = false;

		bool IsValid() const
		{
			/* Enabled properties must have all mandatory fields set, disabled properties are always valid. */
			if (bEnabled) { return !Name.IsEmpty() && GetValueFunc && SetValueFunc; }
			return true;
		}
		
	private:
		
		virtual FText GetName() override { return Name; }
		virtual float GetValue() override { return GetValueFunc(); }
		virtual void SetValue(float NewValue) override { SetValueFunc(NewValue); }
		virtual float GetEditRate() override { return EditRate; }
		virtual float MutateDelta(float Delta) override { return MutateDeltaFunc(Delta); }
		virtual bool IsEnabled() override { return bEnabled; }
	};
	
public:
	
	using UTwoAxisPropertyEditInputBehavior::Initialize;
	void Initialize()
	{
		UTwoAxisPropertyEditInputBehavior::Initialize(this);
	}

	FPropertyInterface HorizontalProperty;
	FPropertyInterface VerticalProperty;
	TUniqueFunction<FKey()> GetCaptureKeyFunc = [](){ return EKeys::B; };
	FSimpleMulticastDelegate OnDragUpdated;

private:

	// ITwoAxisPropertyEditBehaviorTarget implementation

	virtual IPropertyInterface* GetHorizontalProperty() override { return &HorizontalProperty; }
	virtual IPropertyInterface* GetVerticalProperty() override { return &VerticalProperty; }
	virtual FKey GetCaptureKey() override { return GetCaptureKeyFunc(); }
	virtual void PostDragUpdated() override { OnDragUpdated.Broadcast(); }
};
