// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Layout/Geometry.h"
#include "Input/Events.h"
#include "Rendering/DrawElements.h"

#include "WidgetBaseBehavior.generated.h"

/**
 * IWidgetBaseBehavior is an interface for tools that can modify / extend various step of a widget.
 *
 * If implementing this interface, please consider incrementing the LayerId if needed when overriding OnPaint.
 */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UWidgetBaseBehavior : public UInterface
{
	GENERATED_BODY()
};

class IWidgetBaseBehavior
{
	GENERATED_BODY()

public:
	// ~Begin SWidget "Proxy" Interface
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	// ~End SWidget "Proxy" Interface
};
