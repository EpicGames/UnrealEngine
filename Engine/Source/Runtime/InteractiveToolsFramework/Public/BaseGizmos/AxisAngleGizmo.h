// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoComponents.h"
#include "CoreMinimal.h"
#include "InputState.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "Math/MathFwd.h"
#include "Math/Ray.h"
#include "Math/Vector.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "AxisAngleGizmo.generated.h"

class IGizmoAxisSource;
class IGizmoClickTarget;
class IGizmoFloatParameterSource;
class IGizmoStateTarget;
class UClickDragInputBehavior;
class UObject;
struct FToolBuilderState;
namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}

UCLASS(MinimalAPI)
class UAxisAngleGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


/**
 *
 */
UCLASS(MinimalAPI)
class UAxisAngleGizmo : public UInteractiveGizmo, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * Helper that initializes AxisSource, AngleSource, HitTarget, and StateTarget for the common case 
	 *  of being used to control the rotation of a transform. Safe to use for reinitialization.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool InitializeAsRotateGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& InitializationParams,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);

	// UInteractiveGizmo overrides

	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;

	// IClickDragBehaviorTarget implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnTerminateDragSequence() override;

	// IHoverBehaviorTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndHover() override;

	// IModifierToggleBehaviorTarget implementation (inherited via IClickDragBehaviorTarget)
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

public:
	// The below properties can be manipulated for more fine-grained control, but typically it is sufficient
	// to use the initialization method above.

	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	UPROPERTY()
	TScriptInterface<IGizmoFloatParameterSource> AngleSource;

	UPROPERTY()
	TScriptInterface<IGizmoClickTarget> HitTarget;

	UPROPERTY()
	TScriptInterface<IGizmoStateTarget> StateTarget;

	/** The mouse click behavior of the gizmo is accessible so that it can be modified to use different mouse keys. */
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> MouseBehavior;

	/** 
	 * This gets checked to see if we should use the custom ray caster to get a destination point for the gizmo, rather
	 * than grabbing the intersection with the gizmo plane.
	 */
	TUniqueFunction<bool()> ShouldUseCustomDestinationFunc = []() {return false; };

	struct FCustomDestinationParams
	{
		// Right now we use the custom destination function for aligning to items in the scene, which
		// we just need the world ray for. If we want to use functions that use other inputs as the
		// basis for the destination, we would add those parameters here and would make sure that the
		// gizmo passes them in.
		const FRay* WorldRay = nullptr;
	};

	/**
	 * If ShouldUseCustomDestinationFunc() returns true, this function is used to get a destination point, and
	 * the output parameter is picked in such a way that the closest axis in the plane of the gizmo (the positive or
	 * negative x or y, if we're rotating around z, for instance) moves to the closest point in the plane to the
	 * destination point.
	 * Used, for instance, for aligning to items in the scene.
	 */
	TUniqueFunction<bool(const FCustomDestinationParams& WorldRay, FVector& OutputPoint)> CustomDestinationFunc =
		[](const FCustomDestinationParams& Params, FVector& OutputPoint) { return false; };

public:
	UPROPERTY()
	bool bInInteraction = false;

	UPROPERTY()
	FVector RotationOrigin;
	
	UPROPERTY()
	FVector RotationAxis;

	UPROPERTY()
	FVector RotationPlaneX;

	UPROPERTY()
	FVector RotationPlaneY;


	UPROPERTY()
	FVector InteractionStartPoint;

	UPROPERTY()
	FVector InteractionCurPoint;

	UPROPERTY()
	float InteractionStartAngle;

	UPROPERTY()
	float InteractionCurAngle;

protected:
	FVector LastHitPosition;
	float InitialTargetAngle;

	// The angle of the closest axis in the plane of the angle gizmo (will be 0, pi/2, pi, or 3pi/2).
	// Used for snapping the nearest axis to a ray cast point when using CustomDestinationRayCaster.
	float ClosestAxisStartAngle;
};

