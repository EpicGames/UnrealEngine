// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoComponents.h"
#include "CoreMinimal.h"
#include "InputState.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "Math/MathFwd.h"
#include "Math/Ray.h"
#include "Math/Vector.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "AxisPositionGizmo.generated.h"

class IGizmoAxisSource;
class IGizmoClickTarget;
class IGizmoFloatParameterSource;
class IGizmoStateTarget;
class UClickDragInputBehavior;
class UGizmoViewContext;
class UObject;
struct FToolBuilderState;
namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}

UCLASS(MinimalAPI)
class UAxisPositionGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


/**
 * UAxisPositionGizmo implements a gizmo interaction where 1D parameter value is manipulated
 * by dragging a point on a 3D line/axis in space. The 3D point is converted to the axis parameter at
 * the nearest point, giving us the 1D parameter value.
 *
 * As with other base gizmos, this class only implements the interaction. The visual aspect of the
 * gizmo, the axis, and the parameter storage are all provided externally.
 *
 * The axis direction+origin is provided by an IGizmoAxisSource. 
 *
 * The interaction target (ie the thing you have to click on to start the dragging interaction) is provided by an IGizmoClickTarget.
 *
 * The new 1D parameter value is sent to an IGizmoFloatParameterSource
 *
 * Internally a UClickDragInputBehavior is used to handle mouse input, configured in ::Setup()
 */
UCLASS(MinimalAPI)
class UAxisPositionGizmo : public UInteractiveGizmo, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * Helper that initializes AxisSource, ParameterSource, HitTarget, and StateTarget for the common case
	 *  of being used to control the translation of a transform. Safe to use for reinitialization.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool InitializeAsTranslateGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& InitializationParams,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);
	/**
	 * Helper that initializes AxisSource, ParameterSource, HitTarget, and StateTarget for the common case
	 *  of being used to control the scale of a transform. Safe to use for reinitialization.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool InitializeAsScaleGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& InitializationParams,
		bool bDisallowNegativeScaling,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);

	// UInteractiveGizmo overrides

	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;

	// IClickDragBehaviorTarget implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnTerminateDragSequence() override;

	// IHoverBehaviorTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndHover() override;


public:

	// The below properties can be manipulated for more fine-grained control, but typically it is sufficient
	// to use one of the initialization methods above.

	/** AxisSource provides the 3D line on which the interaction happens */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** The 3D line-nearest-point is converted to a 1D coordinate along the line, and the change in value is sent to this ParameterSource */
	UPROPERTY()
	TScriptInterface<IGizmoFloatParameterSource> ParameterSource;

	UE_DEPRECATED(5.5, "UAxisPositionGizmo does not use GizmoViewContext itself (it is used in gizmo components for hit testing).")
	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext;

	/** The HitTarget provides a hit-test against some 3D element (presumably a visual widget) that controls when interaction can start */
	UPROPERTY()
	TScriptInterface<IGizmoClickTarget> HitTarget;

	/** StateTarget is notified when interaction starts and ends, so that things like undo/redo can be handled externally. */
	UPROPERTY()
	TScriptInterface<IGizmoStateTarget> StateTarget;

	/** The mouse click behavior of the gizmo is accessible so that it can be modified to use different mouse keys. */
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> MouseBehavior;


public:
	/** If enabled, then the sign on the parameter delta is always "increasing" when moving away from the origin point, rather than just being a projection onto the axis */
	UPROPERTY()
	bool bEnableSignedAxis = false;

	/** 
	 * This gets checked to see if we should use the custom destination function to get a destination point for
	 * the gizmo, rather than grabbing the closest point on axis to ray.
	 */
	TUniqueFunction<bool()> ShouldUseCustomDestinationFunc = []() {return false; };

	struct FCustomDestinationParams
	{
		// Right now we use the custom destination function for aligning to items in the scene, which
		// we just need the world ray for. If we want to use functions that use other inputs as the
		// basis for the destination (for instance, just the line parameter), we would add those
		// parameters here and would make sure that the gizmo passes them in.
		const FRay* WorldRay = nullptr;
	};

	/**
	 * If ShouldUseCustomDestinationFunc() returns true, this function gets queried to get a destination point.
	 * The gizmo parameter will then be picked in such a way that the dragged location moves to the closest point
	 * on the axis to the destination point (optionally offset by the start click position relative the axis origin,
	 * if bCustomDestinationAlignsAxisOrigin is true, to align the axis origin itself). Can be used, for example, 
	 * to align to items in the scene.
	 */
	TUniqueFunction<bool(const FCustomDestinationParams& WorldRay, FVector& OutputPoint)> CustomDestinationFunc =
		[](const FCustomDestinationParams& Params, FVector& OutputPoint) { return false; };

	/**
	 * Only used when a custom destination is obtained from CustomDestinationFunc. When false, the custom destination
	 * is simply projected back to the axis to get the resulting parameter. This is useful when trying to align the
	 * dragged point on the gizmo to the destination point.
	 * If true, the same projection is performed, but the final parameter is adjusted to make the axis origin align
	 * to the destination instead, rather than the aligning the grabbed point (since the user probably grabbed the
	 * gizmo some distance away from the axis origin).
	 */
	bool bCustomDestinationAlignsAxisOrigin = true;

public:
	/** If true, we are in an active click+drag interaction, otherwise we are not */
	UPROPERTY()
	bool bInInteraction = false;


	//
	// The values below are used in the context of a single click-drag interaction, ie if bInInteraction = true
	// They otherwise should be considered uninitialized
	//

	UPROPERTY()
	FVector InteractionOrigin;

	UPROPERTY()
	FVector InteractionAxis;

	UPROPERTY()
	FVector InteractionStartPoint;

	UPROPERTY()
	FVector InteractionCurPoint;

	UPROPERTY()
	float InteractionStartParameter;

	UPROPERTY()
	float InteractionCurParameter;

	UPROPERTY()
	float ParameterSign = 1.0f;

protected:
	FVector LastHitPosition;
	float InitialTargetParameter;

	float InteractionStartAxisOriginParameterOffset = 0;
};

