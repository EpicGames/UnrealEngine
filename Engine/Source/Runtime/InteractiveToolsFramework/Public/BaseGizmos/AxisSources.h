// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoInterfaces.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "AxisSources.generated.h"

class USceneComponent;


/**
 * UGizmoConstantAxisSource is an IGizmoAxisSource implementation that
 * internally stores the Origin and Direction constants
 */
UCLASS(MinimalAPI)
class UGizmoConstantAxisSource : public UObject, public IGizmoAxisSource
{
	GENERATED_BODY()
public:
	virtual FVector GetOrigin() const
	{
		return Origin;
	}

	virtual FVector GetDirection() const
	{
		return Direction;
	}

	UPROPERTY()
	FVector Origin = FVector::ZeroVector;

	UPROPERTY()
	FVector Direction = FVector(0, 0, 1);
};





/**
 * UGizmoConstantFrameAxisSource is an IGizmoAxisSource implementation that
 * internally stores the Origin, Direction, and X/Y Tangent constants.
 */
UCLASS(MinimalAPI)
class UGizmoConstantFrameAxisSource : public UObject, public IGizmoAxisSource
{
	GENERATED_BODY()
public:
	virtual FVector GetOrigin() const
	{
		return Origin;
	}

	virtual FVector GetDirection() const
	{
		return Direction;
	}

	virtual bool HasTangentVectors() const override { return true; }

	virtual void GetTangentVectors(FVector& TangentXOut, FVector& TangentYOut) const override
	{
		TangentXOut = TangentX;
		TangentYOut = TangentY;
	}

	UPROPERTY()
	FVector Origin = FVector::ZeroVector;

	UPROPERTY()
	FVector Direction = FVector(0, 0, 1);

	UPROPERTY()
	FVector TangentX = FVector(1, 0, 0);

	UPROPERTY()
	FVector TangentY = FVector(0, 1, 0);
};



/**
 * UGizmoWorldAxisSource is an IGizmoAxisSource that provides one of the World axes
 * (ie X/Y/Z axes) based on an integer AxisIndex in range [0,2]. The Orgin is
 * internally stored as a FProperty.
 */
UCLASS(MinimalAPI)
class UGizmoWorldAxisSource : public UObject, public IGizmoAxisSource
{
	GENERATED_BODY()
public:
	virtual FVector GetOrigin() const
	{
		return Origin;
	}

	virtual FVector GetDirection() const
	{
		FVector Result(0, 0, 0);
		Result[FMath::Clamp(AxisIndex, 0, 2)] = 1.0;
		return Result;
	}

	UPROPERTY()
	FVector Origin = FVector::ZeroVector;

	/** Clamped internally to 0,1,2 */
	UPROPERTY()
	int AxisIndex = 2;

public:
	static UGizmoWorldAxisSource* Construct(
		const FVector& WorldOrigin,
		int WorldAxisIndex,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoWorldAxisSource* NewSource = NewObject<UGizmoWorldAxisSource>(Outer);
		NewSource->Origin = WorldOrigin;
		NewSource->AxisIndex = FMath::Clamp(WorldAxisIndex, 0, 2);
		return NewSource;
	}
};




/**
 * UGizmoComponentAxisSource is an IGizmoAxisSource implementation that provides one of the
 * X/Y/Z axes of a Component's local coordinate system, mapped to World, based on an integer AxisIndex in range [0,2].
 * The Axis Origin is the Component's transform origin. Tangent vectors are provided.
 */
UCLASS(MinimalAPI)
class UGizmoComponentAxisSource : public UObject, public IGizmoAxisSource
{
	GENERATED_BODY()
public:
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetOrigin() const;

	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetDirection() const;

	virtual bool HasTangentVectors() const { return true; }

	INTERACTIVETOOLSFRAMEWORK_API virtual void GetTangentVectors(FVector& TangentXOut, FVector& TangentYOut) const;

	UPROPERTY()
	TObjectPtr<USceneComponent> Component;

	/** Clamped internally to 0,1,2 */
	UPROPERTY()
	int AxisIndex = 2;

	/** If false, returns World axes */
	UPROPERTY()
	bool bLocalAxes = true;

public:
	static UGizmoComponentAxisSource* Construct(
		USceneComponent* ComponentIn,
		int LocalAxisIndex,
		bool bUseLocalAxes,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoComponentAxisSource* NewSource = NewObject<UGizmoComponentAxisSource>(Outer);
		NewSource->Component = ComponentIn;
		NewSource->AxisIndex = LocalAxisIndex;
		NewSource->bLocalAxes = bUseLocalAxes;
		return NewSource;
	}
};
