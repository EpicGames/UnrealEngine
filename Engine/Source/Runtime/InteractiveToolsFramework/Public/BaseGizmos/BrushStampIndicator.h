// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "BrushStampIndicator.generated.h"

class UPrimitiveComponent;
class UPreviewMesh;

UCLASS(MinimalAPI)
class UBrushStampIndicatorBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()
public:
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};



/*
 * UBrushStampIndicator is a simple 3D brush indicator. 
 */
UCLASS(Transient, MinimalAPI)
class UBrushStampIndicator : public UInteractiveGizmo
{
	GENERATED_BODY()

public:

	// UInteractiveGizmo interface/implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime) override;


	/**
	 * Update the Radius, Position, and Normal of the stamp indicator
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Update(float Radius, const FVector& Position, const FVector& Normal, float Falloff, float Strength);

	/**
	* Update the Radius, Transform and Falloff of the Stamp Indicator
	*/
	INTERACTIVETOOLSFRAMEWORK_API virtual void Update(float Radius, const FTransform& WorldTransform, float Falloff);

public:

	/** Controls whether Gizmo will draw visual elements. Does not currently affect AttachedComponent. */
	UPROPERTY()
	bool bVisible = true;

	UPROPERTY()
	float BrushRadius = 1.0f;

	UPROPERTY()
	float BrushFalloff = 0.5f;

	UPROPERTY()
	float BrushStrength = 1.0f;

	UPROPERTY()
	FVector BrushPosition = FVector::ZeroVector;

	UPROPERTY()
	FVector BrushNormal = FVector(0, 0, 1);;
	


	UPROPERTY()
	bool bDrawIndicatorLines = true;

	UPROPERTY()
	bool bDrawRadiusCircle = true;

	UPROPERTY()
	bool bScaleNormalByStrength = false;

	UPROPERTY()
	int SampleStepCount = 32;

	UPROPERTY()
	FLinearColor LineColor = FLinearColor(0.06f, 0.96f, 0.06f);

	UPROPERTY()
	float LineThickness = 2.0f;

	UPROPERTY()
	bool bDepthTested = false;



	UPROPERTY()
	bool bDrawSecondaryLines = true;

	UPROPERTY()
	float SecondaryLineThickness = 0.5f;

	UPROPERTY()
	FLinearColor SecondaryLineColor = FLinearColor(0.5f, 0.5f, 0.5f, 0.5f);


	/**
	 * Optional Component that will be transformed such that it tracks the Radius/Position/Normal
	 */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> AttachedComponent;

protected:
	UPrimitiveComponent* ScaleInitializedComponent = nullptr;		// we are just using this as a key, never calling any functions on it
	FVector InitialComponentScale;
};
