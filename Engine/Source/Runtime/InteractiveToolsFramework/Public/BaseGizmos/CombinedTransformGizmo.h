// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "InteractiveToolObjects.h"
#include "InteractiveToolChange.h"
#include "BaseGizmos/GizmoActor.h"
#include "BaseGizmos/TransformProxy.h"

#include "CombinedTransformGizmo.generated.h"

class UInteractiveGizmoManager;
class IGizmoAxisSource;
class IGizmoTransformSource;
class IGizmoStateTarget;
class UGizmoConstantFrameAxisSource;
class UGizmoComponentAxisSource;
class UGizmoTransformChangeStateTarget;
class UGizmoViewContext;
class FTransformGizmoTransformChange;
namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}
namespace UE::GizmoRenderingUtil
{
	class FSubGizmoTransformAdjuster;
}

/**
 * ACombinedTransformGizmoActor is an Actor type intended to be used with UCombinedTransformGizmo,
 * as the in-scene visual representation of the Gizmo.
 * 
 * FCombinedTransformGizmoActorFactory returns an instance of this Actor type (or a subclass), and based on
 * which Translate and Rotate UProperties are initialized, will associate those Components
 * with UInteractiveGizmo's that implement Axis Translation, Plane Translation, and Axis Rotation.
 * 
 * If a particular sub-Gizmo is not required, simply set that FProperty to null.
 * 
 * The static factory method ::ConstructDefault3AxisGizmo() creates and initializes an 
 * Actor suitable for use in a standard 3-axis Transformation Gizmo.
 */
UCLASS(Transient, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType, MinimalAPI)
class ACombinedTransformGizmoActor : public AGizmoActor
{
	GENERATED_BODY()
public:

	INTERACTIVETOOLSFRAMEWORK_API ACombinedTransformGizmoActor();

public:
	//
	// Translation Components
	//

	/** X Axis Translation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> TranslateX;

	/** Y Axis Translation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> TranslateY;

	/** Z Axis Translation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> TranslateZ;


	/** YZ Plane Translation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> TranslateYZ;

	/** XZ Plane Translation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> TranslateXZ;

	/** XY Plane Translation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> TranslateXY;

	//
	// Rotation Components
	//

	/** X Axis Rotation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> RotateX;

	/** Y Axis Rotation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> RotateY;

	/** Z Axis Rotation Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> RotateZ;

	/** Circle that gets drawn around the outside of the gizmo to make it look like a sphere */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> RotationSphere;

	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> FreeRotateHandle;

	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> FreeTranslateHandle;

	//
	// Scaling Components
	//

	/** Uniform Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> UniformScale;


	/** X Axis Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> AxisScaleX;

	/** Y Axis Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> AxisScaleY;

	/** Z Axis Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> AxisScaleZ;


	/** YZ Plane Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> PlaneScaleYZ;

	/** XZ Plane Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> PlaneScaleXZ;

	/** XY Plane Scale Component */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> PlaneScaleXY;



public:
	/**
	 * Replaces the component corresponding to the given element with a new component.
	 * 
	 * @param Element Element to replace, should be a single element (no combined flags), except that
	 *  RotateAllAxes will be interpreted to mean the rotation sphere component.
	 * @param NewComponent The component to replace with. If nullptr, no component is added (i.e. the
	 *  function just deletes the existing component). If not nullptr, then the component should have
	 *  this actor in its outer chain, i.e. it should have been created as NewObject<UYourComponentType>(ThisActor).
	 * @param SubGizmoToGizmo Transform from component to gizmo (i.e. the relative transform).
	 * @param ReplacedComponentOut Outputs a pointer to the replaced component, if there was one. Note 
	 *  that the component will already have had DestroyComponent() called on it.
	 * @return true if successful, which will be always as long as parameters are valid.
	 */
	bool ReplaceSubGizmoComponent(ETransformGizmoSubElements Element, UPrimitiveComponent* NewComponent, 
		const FTransform& SubGizmoToGizmo, UPrimitiveComponent** ReplacedComponentOut = nullptr);

	/**
	 * Create a new instance of ACombinedTransformGizmoActor and populate the various
	 * sub-components with standard GizmoXComponent instances suitable for a 3-axis transformer Gizmo
	 */
	static INTERACTIVETOOLSFRAMEWORK_API ACombinedTransformGizmoActor* ConstructDefault3AxisGizmo(
		UWorld* World, UGizmoViewContext* GizmoViewContext
	);

	/**
	 * Create a new instance of ACombinedTransformGizmoActor. Populate the sub-components 
	 * specified by Elements with standard GizmoXComponent instances suitable for a 3-axis transformer Gizmo
	 */
	static INTERACTIVETOOLSFRAMEWORK_API ACombinedTransformGizmoActor* ConstructCustom3AxisGizmo(
		UWorld* World, UGizmoViewContext* GizmoViewContext,
		ETransformGizmoSubElements Elements
	);

private:
	TArray<TWeakPtr<UE::GizmoRenderingUtil::FSubGizmoTransformAdjuster>> AdjustersThatMirrorOnlyInCombinedMode;
	friend class UCombinedTransformGizmo;
	
	// These store versions of the axis scale components that can be used when not using a combined gizmo,
	//  if they differ.
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> FullAxisScaleX;
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> FullAxisScaleY;
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> FullAxisScaleZ;
};





/**
 * FCombinedTransformGizmoActorFactory creates new instances of ACombinedTransformGizmoActor which
 * are used by UCombinedTransformGizmo to implement 3D transformation Gizmos. 
 * An instance of FCombinedTransformGizmoActorFactory is passed to UCombinedTransformGizmo
 * (by way of UCombinedTransformGizmoBuilder), which then calls CreateNewGizmoActor()
 * to spawn new Gizmo Actors.
 * 
 * By default CreateNewGizmoActor() returns a default Gizmo Actor suitable for
 * a three-axis transformation Gizmo, override this function to customize
 * the Actor sub-elements.
 */
class FCombinedTransformGizmoActorFactory
{
public:
	FCombinedTransformGizmoActorFactory(UGizmoViewContext* GizmoViewContextIn)
		: GizmoViewContext(GizmoViewContextIn)
	{
	}

	/** Only these members of the ACombinedTransformGizmoActor gizmo will be initialized */
	ETransformGizmoSubElements EnableElements =
		ETransformGizmoSubElements::TranslateAllAxes |
		ETransformGizmoSubElements::TranslateAllPlanes |
		ETransformGizmoSubElements::RotateAllAxes |
		ETransformGizmoSubElements::ScaleAllAxes |
		ETransformGizmoSubElements::ScaleAllPlanes | 
		ETransformGizmoSubElements::ScaleUniform;

	/**
	 * @param World the UWorld to create the new Actor in
	 * @return new ACombinedTransformGizmoActor instance with members initialized with Components suitable for a transformation Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual ACombinedTransformGizmoActor* CreateNewGizmoActor(UWorld* World) const;

protected:
	/**
	 * The default gizmos that we use need to have the current view information stored for them via
	 * the ITF context store so that they can figure out how big they are for hit testing, so this
	 * pointer needs to be set (and kept alive elsewhere) for the actor factory to work properly.
	 */
	UGizmoViewContext* GizmoViewContext = nullptr;
};






UCLASS(MinimalAPI)
class UCombinedTransformGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:

	/**
	 * strings identifing GizmoBuilders already registered with GizmoManager. These builders will be used
	 * to spawn the various sub-gizmos
	 */
	FString AxisPositionBuilderIdentifier;
	FString PlanePositionBuilderIdentifier;
	FString AxisAngleBuilderIdentifier;

	/**
	 * If set, this Actor Builder will be passed to UCombinedTransformGizmo instances.
	 * Otherwise new instances of the base FCombinedTransformGizmoActorFactory are created internally.
	 */
	TSharedPtr<FCombinedTransformGizmoActorFactory> GizmoActorBuilder;

	/**
	 * If set, this hover function will be passed to UCombinedTransformGizmo instances to use instead of the default.
	 * Hover is complicated for UCombinedTransformGizmo because all it knows about the different gizmo scene elements
	 * is that they are UPrimitiveComponent (coming from the ACombinedTransformGizmoActor). The default hover
	 * function implementation is to try casting to UGizmoBaseComponent and calling ::UpdateHoverState().
	 * If you are using different Components that do not subclass UGizmoBaseComponent, and you want hover to 
	 * work, you will need to provide a different hover update function.
	 */
	TFunction<void(UPrimitiveComponent*, bool)> UpdateHoverFunction;

	/**
	 * If set, this coord-system function will be passed to UCombinedTransformGizmo instances to use instead
	 * of the default UpdateCoordSystemFunction. By default the UCombinedTransformGizmo will query the external Context
	 * to ask whether it should be using world or local coordinate system. Then the default UpdateCoordSystemFunction
	 * will try casting to UGizmoBaseCmponent and passing that info on via UpdateWorldLocalState();
	 * If you are using different Components that do not subclass UGizmoBaseComponent, and you want the coord system
	 * to be configurable, you will need to provide a different update function.
	 */
	TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> UpdateCoordSystemFunction;


	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


/**
 * FToolContextOptionalToggle is used to store a boolean flag where the value of 
 * the boolean may either be set directly, or it may be set by querying some external context.
 * This struct does not directly do anything, it just wraps up the multiple flags/states
 * needed to provide such functionality
 */
struct FToolContextOptionalToggle
{
	bool bEnabledDirectly = false;
	bool bEnabledInContext = false;
	bool bInheritFromContext = false;

	FToolContextOptionalToggle() {}
	FToolContextOptionalToggle(bool bEnabled, bool bSetInheritFromContext)
	{
		bEnabledDirectly = bEnabled;
		bInheritFromContext = bSetInheritFromContext;
	}

	void UpdateContextValue(bool bNewValue) { bEnabledInContext = bNewValue; }
	bool InheritFromContext() const { return bInheritFromContext; }

	/** @return true if Toggle is currently set to Enabled/On, under the current configuration */
	bool IsEnabled() const { return (bInheritFromContext) ? bEnabledInContext : bEnabledDirectly; }
};


/**
 * UCombinedTransformGizmo provides standard Transformation Gizmo interactions,
 * applied to a UTransformProxy target object. By default the Gizmo will be
 * a standard XYZ translate/rotate Gizmo (axis and plane translation).
 * 
 * The in-scene representation of the Gizmo is a ACombinedTransformGizmoActor (or subclass).
 * This Actor has FProperty members for the various sub-widgets, each as a separate Component.
 * Any particular sub-widget of the Gizmo can be disabled by setting the respective
 * Actor Component to null. 
 * 
 * So, to create non-standard variants of the Transform Gizmo, set a new GizmoActorBuilder 
 * in the UCombinedTransformGizmoBuilder registered with the GizmoManager. Return
 * a suitably-configured GizmoActor and everything else will be handled automatically.
 * 
 */
UCLASS(MinimalAPI)
class UCombinedTransformGizmo : public UInteractiveGizmo
{
	GENERATED_BODY()

public:

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetWorld(UWorld* World);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetGizmoActorBuilder(TSharedPtr<FCombinedTransformGizmoActorFactory> Builder);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubGizmoBuilderIdentifiers(FString AxisPositionBuilderIdentifier, FString PlanePositionBuilderIdentifier, FString AxisAngleBuilderIdentifier);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpdateHoverFunction(TFunction<void(UPrimitiveComponent*, bool)> HoverFunction);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpdateCoordSystemFunction(TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> CoordSysFunction);
	
	/**
	 * Sets a given sub gizmo component to the given component. This is only valid to call after Setup(), but
	 *  can be before or after SetActiveTarget.
	 * 
	 * @param Element Element to replace, should be a single element (no combined flags), except that
	 *  RotateAllAxes will be interpreted to mean the rotation sphere component.
	 * @param NewComponent The component to replace with. If nullptr, no component is added (i.e. the
	 *   function just deletes the existing component). If not nullptr, then the component should have
	 *   the gizmo actor in its outer chain, e.g. NewObject<UYourComponentType>(ThisGizmo->GetGizmoActor()).
	 * @param SubGizmoToGizmo Transform from component to gizmo.
	 * @return true if successful.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool SetSubGizmoComponent(ETransformGizmoSubElements Element, UPrimitiveComponent* Component, 
		const FTransform& SubGizmoToGizmo);

	/**
	 * If used, binds alignment functions to the sub gizmos that they can use to align to geometry in the scene. 
	 * Specifically, translation and rotation gizmos will check ShouldAlignDestination() to see if they should
	 * use the custom ray caster (this allows the behavior to respond to modifier key presses, for instance),
	 * and then use DestinationAlignmentRayCaster() to find a point to align to.
	 * Subgizmos align to the point in different ways, usually by projecting onto the axis or plane that they
	 * operate in.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetWorldAlignmentFunctions(
		TUniqueFunction<bool()>&& ShouldAlignDestination,
		TUniqueFunction<bool(const FRay&, FVector&)>&& DestinationAlignmentRayCaster
		);

	/**
	 * These allow for the deltas of gizmo manipulations to be constrained or clamped in custom ways, for instance
	 *  to slow or stop the gizmo as the drag gets longer. The deltas constrained here are relative to drag start,
	 *  and note that a custom constraint stops default world grid delta snapping from being applied on that axis.
	 * Providing nullptr to any of these removes the custom constraint.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void SetCustomTranslationDeltaFunctions(
		TFunction<bool(double AxisDelta, double& SnappedDelta)> XAxis,
		TFunction<bool(double AxisDelta, double& SnappedDelta)> YAxis,
		TFunction<bool(double AxisDelta, double& SnappedDelta)> ZAxis);
	INTERACTIVETOOLSFRAMEWORK_API void SetCustomRotationDeltaFunctions(
		TFunction<bool(double AxisDelta, double& SnappedDelta)> XAxis,
		TFunction<bool(double AxisDelta, double& SnappedDelta)> YAxis,
		TFunction<bool(double AxisDelta, double& SnappedDelta)> ZAxis);
	INTERACTIVETOOLSFRAMEWORK_API void SetCustomScaleDeltaFunctions(
		TFunction<bool(double AxisDelta, double& SnappedDelta)> XAxis,
		TFunction<bool(double AxisDelta, double& SnappedDelta)> YAxis,
		TFunction<bool(double AxisDelta, double& SnappedDelta)> ZAxis);

	/**
	 * By default, non-uniform scaling handles appear (assuming they exist in the gizmo to begin with), 
	 * when CurrentCoordinateSystem == EToolContextCoordinateSystem::Local, since components can only be
	 * locally scaled. However, this can be changed to a custom check here, perhaps to hide them in extra
	 * conditions or to always show them (if the gizmo is not scaling a component).
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetIsNonUniformScaleAllowedFunction(
		TUniqueFunction<bool()>&& IsNonUniformScaleAllowed
	);

	/**
	 * Exposes the return value of the current IsNonUniformScaleAllowed function so that, for instance,
	 * numerical UI can react appropriately.
	 */
	virtual bool IsNonUniformScaleAllowed() const { return IsNonUniformScaleAllowedFunc(); }

	/**
	 * By default, the nonuniform scale components can scale negatively. However, they can be made to clamp
	 * to zero instead by passing true here. This is useful for using the gizmo to flatten geometry.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDisallowNegativeScaling(bool bDisallow);
	//~ TODO: Should the above affect uniform scaling too?

	// UInteractiveGizmo overrides
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime) override;

	/**
	 * Set the active target object for the Gizmo
	 * @param Target active target
	 * @param TransactionProvider optional IToolContextTransactionProvider implementation to use - by default uses GizmoManager
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider = nullptr);

	/**
	 * Clear the active target object for the Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearActiveTarget();

	/** The active target object for the Gizmo */
	UPROPERTY()
	TObjectPtr<UTransformProxy> ActiveTarget;

	/**
	 * @return the internal GizmoActor used by the Gizmo
	 */
	ACombinedTransformGizmoActor* GetGizmoActor() const { return GizmoActor.Get(); }

	/**
	 * @return current transform of Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API FTransform GetGizmoTransform() const;

	/**
	 * Repositions the gizmo without issuing undo/redo changes, triggering callbacks, 
	 * or moving any components. Useful for resetting the gizmo to a new location without
	 * it being viewed as a gizmo manipulation.
	 * @param bKeepGizmoUnscaled If true, the scale component of NewTransform is passed through to the target but gizmo scale is set to 1
	 */
	INTERACTIVETOOLSFRAMEWORK_API void ReinitializeGizmoTransform(const FTransform& NewTransform, bool bKeepGizmoUnscaled = true);

	/**
	 * Set a new position for the Gizmo. This is done via the same mechanisms as the sub-gizmos,
	 * so it generates the same Change/Modify() events, and hence works with Undo/Redo
	 * @param bKeepGizmoUnscaled If true, the scale component of NewTransform is passed through to the target but gizmo scale is set to 1
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNewGizmoTransform(const FTransform& NewTransform, bool bKeepGizmoUnscaled = true);

	/**
	 * Called at the start of a sequence of gizmo transform edits, for instance while dragging or
	 * manipulating the gizmo numerical UI.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginTransformEditSequence();

	/**
	 * Called at the end of a sequence of gizmo transform edits.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndTransformEditSequence();

	/**
	 * Updates the gizmo transform between Begin/EndTransformeditSequence calls.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void UpdateTransformDuringEditSequence(const FTransform& NewTransform, bool bKeepGizmoUnscaled = true);

	/**
	 * Explicitly set the child scale. Mainly useful to "reset" the child scale to (1,1,1) when re-using Gizmo across multiple transform actions.
	 * @warning does not generate change/modify events!!
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNewChildScale(const FVector& NewChildScale);

	/**
	 * Set visibility for this Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetVisibility(bool bVisible);

	/**
	 * @return true if Gizmo is visible
	 */
	virtual bool IsVisible()
	{
		return IsValid(GizmoActor) && !GizmoActor->IsHidden();
	}

	/**
	 * bSnapToWorldGrid controls whether any position snapping is applied, if possible, for Axis and Plane translations, via the ContextQueriesAPI
	 * Despite the name, this flag controls both world-space grid snapping and relative snapping
	 */
	UPROPERTY()
	bool bSnapToWorldGrid = true;

	/**
	 * Specify whether Relative snapping for Translations should be used in World frame mode. Relative snapping is always used in Local mode.
	 */
	FToolContextOptionalToggle RelativeTranslationSnapping = FToolContextOptionalToggle(true, true);

	/**
	 * Optional grid size which overrides the Context Grid
	 */
	UPROPERTY()
	bool bGridSizeIsExplicit = false;
	UPROPERTY()
	FVector ExplicitGridSize;

	/**
	 * Optional grid size which overrides the Context Rotation Grid
	 */
	UPROPERTY()
	bool bRotationGridSizeIsExplicit = false;
	UPROPERTY()
	FRotator ExplicitRotationGridSize;

	/**
	 * If true, then when using world frame, Axis and Plane rotation snap to the world grid via the ContextQueriesAPI (in RotationSnapFunction)
	 */
	UPROPERTY()
	bool bSnapToWorldRotGrid = true;

	/**
	 * If true, scaling snaps to the grid
	 */
	UPROPERTY()
	bool bSnapToScaleGrid = true;


	/**
	 * Whether to use the World/Local coordinate system provided by the context via the ContextyQueriesAPI.
	 */
	UPROPERTY()
	bool bUseContextCoordinateSystem = true;

	/**
	 * Current coordinate system in use. If bUseContextCoordinateSystem is true, this value will be updated internally every Tick()
	 * by quering the ContextyQueriesAPI, otherwise the default is Local and the client can change it as necessary
	 */
	UPROPERTY()
	EToolContextCoordinateSystem CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;




	/**
	 * Whether to use the Gizmo Mode provided by the context via the ContextyQueriesAPI.
	 */
	UPROPERTY()
	bool bUseContextGizmoMode = true;

	/**
	 * Current dynamic sub-widget visibility mode to use (eg Translate-Only, Scale-Only, Combined, etc)
	 * If bUseContextGizmoMode is true, this value will be updated internally every Tick()
	 * by quering the ContextyQueriesAPI, otherwise the default is Combined and the client can change it as necessary
	 */
	UPROPERTY()
	EToolContextTransformGizmoMode ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;

	/**
	 * Gets the elements that this gizmo was initialized with. Note that this may not account for individual
	 * element visibility- for instance the scaling component may not be visible if IsNonUniformScaleAllowed() is false.
	 */
	INTERACTIVETOOLSFRAMEWORK_API ETransformGizmoSubElements GetGizmoElements();


	/**
	 * The DisplaySpaceTransform is not used by the gizmo itself, but can be used by external adapters that might
	 * display gizmo values, to give values relative to this transform rather than relative to world origin and axes.
	 * 
	 * For example a numerical UI for a two-axis gizmo that is not in a world XY/YZ/XZ plane cannot use the global
	 * axes for setting the absolute position of the plane if it wants the gizmo to remain in that plane; instead, the
	 * DisplaySpaceTransform can give a space in which X and Y values keep the gizmo in the plane.
	 *
	 * Note that this is an optional feature, as it would require tools to keep this transform up to date if they
	 * want the UI to use it, so tools could just leave it unset.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void SetDisplaySpaceTransform(TOptional<FTransform> TransformIn);
	const TOptional<FTransform>& GetDisplaySpaceTransform() { return DisplaySpaceTransform; }

	/**
	 * Broadcast at the end of a SetDisplaySpaceTransform call that changes the display space transform.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDisplaySpaceTransformChanged, UCombinedTransformGizmo*, TOptional<FTransform>);
	FOnDisplaySpaceTransformChanged OnDisplaySpaceTransformChanged;

	/** 
	 * Broadcast at the end of a SetActiveTarget call. Using this, an adapter such as a numerical UI widget can 
	 * bind to the gizmo at construction and still be able to initialize using the transform proxy once that is set.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSetActiveTarget, UCombinedTransformGizmo*, UTransformProxy*);
	FOnSetActiveTarget OnSetActiveTarget;

	/** 
	 * Broadcast at the beginning of a ClearActiveTarget call, when the ActiveTarget (if present) is not yet 
	 * disconnected. Gives things a chance to unbind from it.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClearActiveTarget, UCombinedTransformGizmo*, UTransformProxy*);
	FOnClearActiveTarget OnAboutToClearActiveTarget;

	/** Broadcast at the end of a SetVisibility call if the visibility changes. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnVisibilityChanged, UCombinedTransformGizmo*, bool);
	FOnVisibilityChanged OnVisibilityChanged;

protected:
	TSharedPtr<FCombinedTransformGizmoActorFactory> GizmoActorBuilder;

	FString AxisPositionBuilderIdentifier;
	FString PlanePositionBuilderIdentifier;
	FString AxisAngleBuilderIdentifier;

	// This function is called on each active GizmoActor Component to update it's hover state.
	// If the Component is not a UGizmoBaseCmponent, the client needs to provide a different implementation
	// of this function via the ToolBuilder
	TFunction<void(UPrimitiveComponent*, bool)> UpdateHoverFunction;

	// This function is called on each active GizmoActor Component to update it's coordinate system (eg world/local).
	// If the Component is not a UGizmoBaseCmponent, the client needs to provide a different implementation
	// of this function via the ToolBuilder
	TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> UpdateCoordSystemFunction;

	/** List of current-active child components */
	UPROPERTY()
	TArray<TObjectPtr<UPrimitiveComponent>> ActiveComponents;

	/** list of currently-active child gizmos */
	UPROPERTY()
	TArray<TObjectPtr<UInteractiveGizmo>> ActiveGizmos;

	/** 
	 * FSubGizmoInfo stores the (UPrimitiveComponent,UInteractiveGizmo) pair for a sub-element of the widget.
	 * The ActiveComponents and ActiveGizmos arrays keep those items alive, so this is redundant information, but useful for filtering/etc
	 */
	struct FSubGizmoInfo
	{
		// note: either of these may be invalid
		TWeakObjectPtr<UPrimitiveComponent> Component;
		TWeakObjectPtr<UInteractiveGizmo> Gizmo;
	};
	TArray<FSubGizmoInfo> TranslationSubGizmos;
	TArray<FSubGizmoInfo> RotationSubGizmos;
	TArray<FSubGizmoInfo> UniformScaleSubGizmos;
	TArray<FSubGizmoInfo> NonUniformScaleSubGizmos;
private:
	TWeakObjectPtr<UInteractiveGizmo> AxisScaleXGizmo;
	TWeakObjectPtr<UInteractiveGizmo> AxisScaleYGizmo;
	TWeakObjectPtr<UInteractiveGizmo> AxisScaleZGizmo;

	TFunction<bool(double AxisDelta, double& SnappedDelta)> CustomTranslationDeltaConstraintFunctions[3];
	TFunction<bool(double AxisDelta, double& SnappedDelta)> CustomRotationDeltaConstraintFunctions[3];
	TFunction<bool(double AxisDelta, double& SnappedDelta)> CustomScaleDeltaConstraintFunctions[3];

protected:
	/** GizmoActors will be spawned in this World */
	UWorld* World;

	/** 
	 * Current active GizmoActor that was spawned by this Gizmo. Will be destroyed when
	 *  this gizmo is, and expected not to be destroyed otherwise. Still, needs to be
	 *  validated in case things go wrong and the world is destroyed before calling Shutdown
	 *  on the gizmo.
	 */
	UPROPERTY()
	TObjectPtr<ACombinedTransformGizmoActor> GizmoActor;

	//
	// Axis Sources
	//


	/** Axis that points towards camera, X/Y plane tangents aligned to right/up. Shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoConstantFrameAxisSource> CameraAxisSource;

	// internal function that updates CameraAxisSource by getting current view state from GizmoManager
	INTERACTIVETOOLSFRAMEWORK_API void UpdateCameraAxisSource();


	/** X-axis source is shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> AxisXSource;

	/** Y-axis source is shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> AxisYSource;

	/** Z-axis source is shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> AxisZSource;

	//
	// Scaling support. 
	// UE Components only support scaling in local coordinates, so we have to create separate sources for that.
	//

	/** Local X-axis source (ie 1,0,0) is shared across Scale Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> UnitAxisXSource;

	/** Y-axis source (ie 0,1,0) is shared across Scale Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> UnitAxisYSource;

	/** Z-axis source (ie 0,0,1) is shared across Scale Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> UnitAxisZSource;


	//
	// Other Gizmo Components
	//


	/** 
	 * State target is shared across gizmos, and created internally during SetActiveTarget(). 
	 * Several FChange providers are registered with this StateTarget, including the UCombinedTransformGizmo
	 * itself (IToolCommandChangeSource implementation above is called)
	 */
	UPROPERTY()
	TObjectPtr<UGizmoTransformChangeStateTarget> StateTarget;

	/**
	 * These are used to let the translation subgizmos use raycasts into the scene to align the gizmo with scene geometry.
	 * See comment for SetWorldAlignmentFunctions().
	 */
	TUniqueFunction<bool()> ShouldAlignDestination = []() { return false; };
	TUniqueFunction<bool(const FRay&, FVector&)> DestinationAlignmentRayCaster = [](const FRay&, FVector&) {return false; };

	TUniqueFunction<bool()> IsNonUniformScaleAllowedFunc = [this]() { return CurrentCoordinateSystem == EToolContextCoordinateSystem::Local; };

	bool bDisallowNegativeScaling = false;

	// See comment for SetDisplaySpaceTransform;
	TOptional<FTransform> DisplaySpaceTransform;

protected:

	using FTransformSubGizmoCommonParams = UE::GizmoUtil::FTransformSubGizmoCommonParams;
	using FTransformSubGizmoSharedState = UE::GizmoUtil::FTransformSubGizmoSharedState;

	/** @return a new instance of the standard axis-translation Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddAxisTranslationGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	/** @return a new instance of the standard plane-translation Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddPlaneTranslationGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	/** @return a new instance of the standard axis-rotation Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddAxisRotationGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	/** @return a new instance of the standard axis-scaling Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddAxisScaleGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	/** @return a new instance of the standard plane-scaling Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddPlaneScaleGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	/** @return a new instance of the standard plane-scaling Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddUniformScaleGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddFreeTranslationGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddFreeRotationGizmo(
		FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState);

	// Axis and Plane TransformSources use these function to execute snapping queries
	INTERACTIVETOOLSFRAMEWORK_API bool PositionSnapFunction(const FVector& WorldPosition, FVector& SnappedPositionOut) const;
	INTERACTIVETOOLSFRAMEWORK_API bool PositionAxisDeltaSnapFunction(double AxisDelta, double& SnappedDeltaOut, int AxisIndex) const;
	INTERACTIVETOOLSFRAMEWORK_API FQuat RotationSnapFunction(const FQuat& DeltaRotation) const;
	INTERACTIVETOOLSFRAMEWORK_API bool RotationAxisAngleSnapFunction(double AxisAngleDelta, double& SnappedAxisAngleDeltaOut, int AxisIndex) const;
	
private:
	// Used for scale delta snapping. Calls out to the non-axis endpoint by default
	bool ScaleAxisDeltaSnapFunction(double ScaleAxisDelta, double& SnappedAxisScaleDeltaOut, int AxisIndex) const;
protected:
	// currently not implemented because WorldGridSnapping currently has no affect on scale/scale-snapping
	INTERACTIVETOOLSFRAMEWORK_API bool ScaleSnapFunction(double DeltaScale) const;
	// Used for uniform scale delta snapping
	INTERACTIVETOOLSFRAMEWORK_API bool ScaleAxisDeltaSnapFunction(double ScaleAxisDelta, double & SnappedAxisScaleDeltaOut) const;

	UE_DEPRECATED(5.5, "Use FTransformSubGizmoCommonParams overload instead.")
	/** @return a new instance of the standard axis-translation Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddAxisTranslationGizmo(
		UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
		IGizmoAxisSource* AxisSource,
		IGizmoTransformSource* TransformSource,
		IGizmoStateTarget* StateTarget,
		int AxisIndex);
	UE_DEPRECATED(5.5, "Use FTransformSubGizmoCommonParams overload instead.")
	/** @return a new instance of the standard plane-translation Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddPlaneTranslationGizmo(
		UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
		IGizmoAxisSource* AxisSource,
		IGizmoTransformSource* TransformSource,
		IGizmoStateTarget* StateTarget,
		int XAxisIndex, int YAxisIndex);
	UE_DEPRECATED(5.5, "Use FTransformSubGizmoCommonParams overload instead.")
	/** @return a new instance of the standard axis-rotation Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddAxisRotationGizmo(
		UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
		IGizmoAxisSource* AxisSource,
		IGizmoTransformSource* TransformSource,
		IGizmoStateTarget* StateTarget);
	UE_DEPRECATED(5.5, "Use FTransformSubGizmoCommonParams overload instead.")
	/** @return a new instance of the standard axis-scaling Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddAxisScaleGizmo(
		UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
		IGizmoAxisSource* GizmoAxisSource, IGizmoAxisSource* ParameterAxisSource,
		IGizmoTransformSource* TransformSource,
		IGizmoStateTarget* StateTarget);
	UE_DEPRECATED(5.5, "Use FTransformSubGizmoCommonParams overload instead.")
	/** @return a new instance of the standard plane-scaling Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddPlaneScaleGizmo(
		UPrimitiveComponent* AxisComponent, USceneComponent* RootComponent,
		IGizmoAxisSource* GizmoAxisSource, IGizmoAxisSource* ParameterAxisSource,
		IGizmoTransformSource* TransformSource,
		IGizmoStateTarget* StateTarget);
	UE_DEPRECATED(5.5, "Use FTransformSubGizmoCommonParams overload instead.")
	/** @return a new instance of the standard plane-scaling Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddUniformScaleGizmo(
		UPrimitiveComponent* ScaleComponent, USceneComponent* RootComponent,
		IGizmoAxisSource* GizmoAxisSource, IGizmoAxisSource* ParameterAxisSource,
		IGizmoTransformSource* TransformSource,
		IGizmoStateTarget* StateTarget);

	// Useful for reinitializing components after SetActiveTarget, or for use by derived classes.
	TUniquePtr<FTransformSubGizmoSharedState> SubGizmoSharedState;
private:
	// Here to support subgizmo reinitialization after SetActiveTarget has been called. Private
	// instead of protected for now in case we change the approach here.
	IToolContextTransactionProvider* TransactionProviderAtLastSetActiveTarget = nullptr;

	EToolContextTransformGizmoMode PreviousActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
	void ApplyGizmoActiveMode();
};
