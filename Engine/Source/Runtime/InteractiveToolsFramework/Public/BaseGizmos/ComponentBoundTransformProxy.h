// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/TransformProxy.h"

#include "ComponentBoundTransformProxy.generated.h"

/**
 * A variant of a transform proxy whose transform is always bound to a particular component. This means 
 *  that if the component is moved as a result of its parents moving, the proxy will get that transform
 *  when queried. This makes the transform proxy very useful for being bound to sub gizmos that are moved
 *  by some parent gizmo. SetTransform will cause the proxy to set the transform on the bound component
 *  even if it is not part of its component set (to stay matched with the component) so AddComponent is
 *  not necessary for the bound component (nor is it likely to be used with this proxy, see below).
 * 
 * This class is mainly intended to be bound to a single component and used for its delegates. It can
 *  still be used for the multi-component movement functionality of a transform proxy, but it is worth
 *  noting in that case that if the bound component is moved by its parent, the other components won't
 *  necessarily be moved unless SetTransform() is called on the proxy (or they happen to be parented in
 *  the same subtree).
 */
UCLASS(Transient, MinimalAPI)
class UComponentBoundTransformProxy : public UTransformProxy
{
	GENERATED_BODY()
public:
	/**
	 * Make the proxy get its transform from the given component, and set the transform on this component
	 *  whenever SetTransform is called.
	 * 
	 * @param bStoreScaleSeparately If true, then the scale won't be obtained from the component, nor set on
	 *  it. Instead it will be stored separately internally. This is useful when using a gizmo to manipulate
	 *  scale, where you don't want the scale to be applied to the gizmo component itself.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BindToComponent(USceneComponent* Component, bool bStoreScaleSeparately);

	// UTransformProxy
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetTransform() const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransform(const FTransform& Transform) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSharedTransform() override;

protected:
	TWeakObjectPtr<USceneComponent> BoundComponent;
	bool bStoreScaleSeparately = true;
};