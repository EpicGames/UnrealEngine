// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolObjects.h"
#include "GizmoActor.generated.h"

class UGizmoArrowComponent;
class UGizmoRectangleComponent;
class UGizmoCircleComponent;
class UGizmoBoxComponent;
class UGizmoLineHandleComponent;
class UGizmoViewContext;


/**
 * AGizmoActor is a base class for Actors that would be created to represent Gizmos in the scene.
 * Currently this does not involve any special functionality, but a set of static functions
 * are provided to create default Components commonly used in Gizmos.
 */
UCLASS(Transient, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType, MinimalAPI)
class AGizmoActor : public AInternalToolFrameworkActor
{
	GENERATED_BODY()
public:
	INTERACTIVETOOLSFRAMEWORK_API AGizmoActor();

	/** Add standard arrow component to Actor, generally used for axis-translation */
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoArrowComponent* AddDefaultArrowComponent(
		UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
		const FLinearColor& Color, const FVector& LocalDirection, const float Length = 80.0f
	);
	/** Add standard rectangle component to Actor, generally used for plane-translation */
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoRectangleComponent* AddDefaultRectangleComponent(
		UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
		const FLinearColor& Color, const FVector& PlaneAxis1, const FVector& PlaneAxisx2
	);
	/** Add standard circle component to Actor, generally used for axis-rotation */
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoCircleComponent* AddDefaultCircleComponent(
		UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
		const FLinearColor& Color, const FVector& PlaneNormal,
		float Radius = 120.0f
	);

	/** Add standard 3D box component to Actor. By default the box is axis-aligned, centered at the specified Origin */
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoBoxComponent* AddDefaultBoxComponent(
		UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
		const FLinearColor& Color, const FVector& Origin, 
		const FVector& Dimensions = FVector(20.0f, 20.0f, 20.0f)
	);

	/** Add standard disk component to Actor, generally used for handles */
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoLineHandleComponent* AddDefaultLineHandleComponent(
		UWorld* World, AActor* Actor, UGizmoViewContext* GizmoViewContext,
		const FLinearColor& Color, const FVector& PlaneNormal, const FVector& LocalDirection,
		const float Length = 60.f, const bool bImageScale = false
	);
};
