// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"

#include "GizmoBaseComponent.generated.h"

class UGizmoViewContext;

UINTERFACE(MinimalAPI)
class UGizmoBaseComponentInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that allows a component to receive various gizmo-specific callbacks while
 * still inheriting from some class other than UGizmoBaseComponent.
 */
class IGizmoBaseComponentInterface
{
	GENERATED_BODY()
public:
	virtual void UpdateHoverState(bool bHoveringIn) {}

	virtual void UpdateWorldLocalState(bool bWorldIn) {}

	virtual void UpdateInteractingState(bool bInteractingIn) {}
};


/**
 * Base class for simple Components intended to be used as part of 3D Gizmos.
 * Contains common properties and utility functions.
 * This class does nothing by itself, use subclasses like UGizmoCircleComponent
 */
UCLASS(ClassGroup = Utility, HideCategories = (Physics, Collision, Mobile), MinimalAPI)
class UGizmoBaseComponent : public UPrimitiveComponent
	, public IGizmoBaseComponentInterface
{
	GENERATED_BODY()

public:
	UGizmoBaseComponent()
	{
		bUseEditorCompositing = false;
	}

	/**
	 * Currently this must be called if you change UProps on Base or subclass,
	 * to recreate render proxy which has a local copy of those settings
	 */
	void NotifyExternalPropertyUpdates()
	{
		MarkRenderStateDirty();
		UpdateBounds();
	}

public:
	UPROPERTY(EditAnywhere, Category = Options)
	FLinearColor Color = FLinearColor::Red;


	UPROPERTY(EditAnywhere, Category = Options)
	float HoverSizeMultiplier = 2.0f;


	UPROPERTY(EditAnywhere, Category = Options)
	float PixelHitDistanceThreshold = 7.0f;

	void SetGizmoViewContext(UGizmoViewContext* GizmoViewContextIn)
	{
		GizmoViewContext = GizmoViewContextIn;
		bIsViewDependent = (GizmoViewContext != nullptr);
	}

public:
	UFUNCTION()
	virtual void UpdateHoverState(bool bHoveringIn) override
	{
		if (bHoveringIn != bHovering)
		{
			bHovering = bHoveringIn;
		}
	}

	UFUNCTION()
	virtual void UpdateWorldLocalState(bool bWorldIn) override
	{
		if (bWorldIn != bWorld)
		{
			bWorld = bWorldIn;
		}
	}


protected:

	// hover state
	bool bHovering = false;

	// world/local coordinates state
	bool bWorld = false;

	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext = nullptr;

	// True when GizmoViewContext is not null. Here as a boolean so it
	// can be pointed to by the proxy to see what it should do.
	bool bIsViewDependent = false;
};
