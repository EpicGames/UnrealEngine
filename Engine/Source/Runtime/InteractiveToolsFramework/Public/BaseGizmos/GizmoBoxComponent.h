// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "GizmoBoxComponent.generated.h"

/**
 * Simple Component intended to be used as part of 3D Gizmos. 
 * Draws outline of 3D Box based on parameters.
 */
UCLASS(ClassGroup = Utility, HideCategories = (Physics, Collision, Lighting, Rendering, Mobile), MinimalAPI)
class UGizmoBoxComponent : public UGizmoBaseComponent
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	FVector Origin = FVector(0, 0, 0);

	UPROPERTY(EditAnywhere, Category = Options)
	FQuat Rotation = FQuat::Identity;

	UPROPERTY(EditAnywhere, Category = Options)
	FVector Dimensions = FVector(20.0f, 20.0f, 20.0f);

	UPROPERTY(EditAnywhere, Category = Options)
	float LineThickness = 2.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bRemoveHiddenLines = true;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bEnableAxisFlip = true;

private:
	//~ Begin UPrimitiveComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

};
