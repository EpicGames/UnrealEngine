// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "GizmoCircleComponent.generated.h"

/**
 * Simple Component intended to be used as part of 3D Gizmos.
 * Draws a 3D circle based on parameters.
 */
UCLASS(ClassGroup = Utility, HideCategories = (Physics, Collision, Mobile), MinimalAPI)
class UGizmoCircleComponent : public UGizmoBaseComponent
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	FVector Normal = FVector(0,0,1);

	UPROPERTY(EditAnywhere, Category = Options)
	float Radius = 100.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	float Thickness = 2.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	int NumSides = 64;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bViewAligned = false;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bDrawFullCircle = false;

	// If the circle was on a 3D sphere, then only the 'front' part of the circle can be hit, 
	// (in other words, if the ray would hit the sphere first, ignore the hit).
	// Dynamically disabled if the circle is parallel to the view plane (ie "fully visible")
	UPROPERTY(EditAnywhere, Category = Options)
	bool bOnlyAllowFrontFacingHits = true;

private:
	//~ Begin UPrimitiveComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.
};
