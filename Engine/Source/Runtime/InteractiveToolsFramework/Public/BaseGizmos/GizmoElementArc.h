// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementCircleBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementArc.generated.h"

class FPrimitiveDrawInterface;
class FMaterialRenderProxy;

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a thick arc based on parameters.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementArc : public UGizmoElementCircleBase
{
	GENERATED_BODY()

public:

	UGizmoElementArc();

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Inner circle radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInnerRadius(double InInnerRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetInnerRadius() const;

protected:

	// Arc inner radius.
	UPROPERTY()
	double InnerRadius = 5.0;
};
