// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementArrowHead.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "GizmoElementGroup.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"

#include "GizmoElementArrow.generated.h"

class UGizmoElementBox;
class UGizmoElementCone;
class UGizmoElementCylinder;
class UGizmoElementSphere;

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a 3D arrow based on parameters.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementArrow : public UGizmoElementGroupBase
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UGizmoElementArrow();

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Location of the base of arrow cylinder
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetBase(const FVector& InBase);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetBase() const;

	// Arrow direction.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDirection(const FVector& InDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetDirection() const;

	// Arrow side direction for box head.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSideDirection(const FVector& InSideDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetSideDirection() const;

	// Arrow body length.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetBodyLength(float InBodyLength);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetBodyLength() const;

	// Arrow body cylinder radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetBodyRadius(float InBodyRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetBodyRadius() const;

	// Arrow head length, used for both cone and cube head.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHeadLength(float InHeadLength);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetHeadLength() const;

	// Arrow head radius, if cone.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHeadRadius(float InHeadRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetHeadRadius() const;

	// Number of sides for cylinder and cone, if relevant.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumSides(int32 InNumSides);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumSides() const;

	// Whether to render and end cap on the base of the arrow head. Applicable when the ArrowHeadType is Cone.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEndCaps(bool InEndCaps);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEndCaps() const;

	// Head type cone or cube.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHeadType(EGizmoElementArrowHeadType InHeadType);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementArrowHeadType GetHeadType() const;

	// Whether to render the body element with dashing.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetIsDashed(bool bInDashing);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsDashed() const;

	// The dash parameters for the body element, if enabled. If GapLength is not specified, it defaults to half DashLength.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDashParameters(const float InDashLength = 10.0f, const TOptional<float>& InGapLength = TOptional<float>());
	INTERACTIVETOOLSFRAMEWORK_API virtual void GetDashParameters(float& OutDashLength, float& OutGapLength) const;

	// Pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPixelHitDistanceThreshold(float InPixelHitDistanceThreshold) override;

	// Minimum Pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetMinimumPixelHitDistanceThreshold(float InMinimumPixelHitDistanceThreshold) override;
	
	// Hit Mask Gizmo element used for adaptive pixel hit threshold
	INTERACTIVETOOLSFRAMEWORK_API void SetHitMask(const TWeakObjectPtr<UGizmoElementBase>& InHitMask);

protected:

	// Update arrow cylinder based on parameters
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateArrowBody();

	// Update arrow cone or box based on parameters
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateArrowHead();

	// Flag indicating body properties need to be updated prior to render
	bool bUpdateArrowBody = true;

	// Flag indicating head properties need to be updated prior to render
	bool bUpdateArrowHead = true;

	// Arrow cylinder body
	UPROPERTY()
	TObjectPtr<UGizmoElementCylinder> CylinderElement;

	// Arrow head
	UPROPERTY()
	TObjectPtr<UGizmoElementArrowHead> HeadElement;

	// Location of center of arrow base circle.
	UPROPERTY()
	FVector Base = FVector::ZeroVector;

	// Direction of arrow axis
	UPROPERTY()
	FVector Direction = FVector(0.0f, 0.0f, 1.0f);

	// Side direction for box head
	UPROPERTY()
	FVector SideDirection = FVector(0.0f, 1.0f, 0.0f);

	// Arrow body length
	UPROPERTY()
	float BodyLength = 1.5f;

	// Radius of arrow cylinder
	UPROPERTY()
	float BodyRadius = 0.5f;

	// Length of head, cone or box
	UPROPERTY()
	float HeadLength = 0.5f;

	// Radius of head cone
	UPROPERTY()
	float HeadRadius = 0.75f;

	// Number of sides for tessellating cone and cylinder
	UPROPERTY()
	int32 NumSides = 32;

	// Whether to render an end cap on the base of the arrow head.
	UPROPERTY()
	bool bEndCaps = false;

	// Head type
	UPROPERTY()
	EGizmoElementArrowHeadType HeadType = EGizmoElementArrowHeadType::Cone;

	// Hit mask
	UPROPERTY()
	TWeakObjectPtr<UGizmoElementBase> HitMask = nullptr;
};
