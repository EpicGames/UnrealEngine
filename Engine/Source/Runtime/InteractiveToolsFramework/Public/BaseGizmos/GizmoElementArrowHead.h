// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GizmoElementBase.h"
#include "GizmoElementGroup.h"

#include "GizmoElementArrowHead.generated.h"

class UGizmoElementBox;
class UGizmoElementCone;
class UGizmoElementSphere;

UENUM()
enum class EGizmoElementArrowHeadType
{
	None,
	Cone,
	Cube,
	Sphere
};

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a 3D arrowhead based on parameters.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementArrowHead : public UGizmoElementGroupBase
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UGizmoElementArrowHead();

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Location of center of the arrow head
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCenter(const FVector& InCenter);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetCenter() const;

	// Arrow direction.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDirection(const FVector& InDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetDirection() const;

	// Arrow side direction for box head.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSideDirection(const FVector& InSideDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetSideDirection() const;

	// Arrow head length, used for both cone and cube head.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetLength(float InLength);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetLength() const;

	// Arrow head radius, if cone or sphere.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetRadius(float InRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetRadius() const;

	// Number of sides for cylinder and cone, if relevant.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumSides(int32 InNumSides);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumSides() const;

	// Head type cone or cube.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetType(EGizmoElementArrowHeadType InHeadType);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementArrowHeadType GetType() const;

	// Pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPixelHitDistanceThreshold(float InPixelHitDistanceThreshold) override;

	// Hit Mask Gizmo element used for adaptive pixel hit threshold
	INTERACTIVETOOLSFRAMEWORK_API void SetHitMask(const TWeakObjectPtr<UGizmoElementBase>& InHitMask);

protected:

	// Update the appropriate shape based on parameters
	INTERACTIVETOOLSFRAMEWORK_API virtual void Update();

protected:

	// Flag indicating properties need to be updated prior to render
	bool bUpdate = true;

	// Cone head
	UPROPERTY()
	TObjectPtr<UGizmoElementCone> ConeElement;

	// Box head
	UPROPERTY()
	TObjectPtr<UGizmoElementBox> BoxElement;

	// Sphere head
	UPROPERTY()
	TObjectPtr<UGizmoElementSphere> SphereElement;

	// Location of center
	UPROPERTY()
	FVector Center = FVector::ZeroVector;

	// Direction of arrow axis
	UPROPERTY()
	FVector Direction = FVector(0.0f, 0.0f, 1.0f);

	// Side direction for box
	UPROPERTY()
	FVector SideDirection = FVector(0.0f, 1.0f, 0.0f);

	// Length of head, cone or box
	UPROPERTY()
	float Length = 0.5f;

	// Radius of head cone
	UPROPERTY()
	float Radius = 0.75f;

	// Number of sides for tessellating cone and cylinder
	UPROPERTY()
	int32 NumSides = 32;

	// Head type
	UPROPERTY()
	EGizmoElementArrowHeadType Type = EGizmoElementArrowHeadType::Cone;

	// Hit mask
	UPROPERTY()
	TWeakObjectPtr<UGizmoElementBase> HitMask = nullptr;
};

