// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementRenderState.h"
#include "BaseGizmos/GizmoElementShared.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "InputState.h"
#include "SceneView.h"
#include "ToolContextInterfaces.h"
#include "UObject/GCObject.h"
#include "GizmoElementBase.generated.h"

class UMaterialInterface;

/**
 * Base class for 2d and 3d primitive objects intended to be used as part of 3D Gizmos.
 * Contains common properties and utility functions.
 * This class does nothing by itself, use subclasses like UGizmoElementCylinder
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UGizmoElementBase : public UObject
{
	GENERATED_BODY()

	friend class FGizmoElementAccessor;
	
public:

	static constexpr float DefaultViewDependentAngleTol = 0.052f;			// ~3 degrees
	static constexpr float DefaultViewDependentAxialMaxCosAngleTol = 0.998f;	// Cos(DefaultViewDependentAngleTol)
	static constexpr float DefaultViewDependentPlanarMinCosAngleTol = 0.052f;	// Cos(HALF_PI - DefaultViewDependentAngleTol)

	static constexpr float DefaultViewAlignAngleTol = 0.052f;				// ~3 degrees
	static constexpr float DefaultViewAlignMaxCosAngleTol = 0.998f;			// Cos(DefaultViewAlignAngleTol)

	static constexpr uint32 DefaultPartIdentifier = 0;						// Default part ID, used for elements that are not associated with any gizmo part

	// 
	// Render traversal state structure used to maintain the current render state while rendering.
	// As the gizmo element hierarchy is traversed, current state is maintained and updated. 
	// Element state attribute inheritance works as follows:
	//
	// - Child element state that is not set inherits from parent state.
	// - Child element state that is set replaces the parent state, except in the case of overrides.
	// - Overrides: parent element state with override set to true replaces all child state regardless of whether the child state has been set.
	//
	struct FRenderTraversalState
	{
		// LocalToWorld transform 
		// Note: non-uniform scale is not supported and the X scale element will be used for uniform scaling.
		FTransform LocalToWorldTransform;
		
		// The potentially flattened scale of this state, if this view is orthographic. Otherwise, a unit vector.
		// When rendering gizmos, `FScaleMatrix(FlattenScale)` can be applied to a rendering matrix so that gizmos never clip into the near orthographic view plane, even at extreme distances.
		FVector FlattenScale = FVector::OneVector;

		// Pixel to world scale. Also accounts for DPI scale, @see IToolsContextRenderAPI::ViewCameraState.
		double PixelToWorldScale = 1.0;

		// The screen space depth of the reference point used to compute PixelToWorldScale. Defaults to -1.0 (unset). Should not be used if negative.
		double DepthAtPixelToWorldReferencePoint = -1.0;

		// Interact state, if not equal to none, overrides the element's interact state 
		EGizmoElementInteractionState InteractionState = EGizmoElementInteractionState::None;

		// Current state used for rendering meshes
		FGizmoElementMeshRenderStateAttributes MeshRenderState;

		// Current state used for rendering lines
		FGizmoElementLineRenderStateAttributes LineRenderState;

		// Initialize state
		INTERACTIVETOOLSFRAMEWORK_API void Initialize(const UE::GizmoRenderingUtil::ISceneViewInterface& InView, const FTransform& InTransform, const double InDPIScale = 1.0);

		// Initialize state
		UE_DEPRECATED(5.8, "Use Initialize with ISceneViewInterface instead.")
		INTERACTIVETOOLSFRAMEWORK_API void Initialize(const FSceneView* InSceneView, FTransform InTransform, const double InDPIScale = 1.0);

		// Returns the mesh material based on the current interaction state.
		const UMaterialInterface* GetCurrentMaterial()
		{
			return MeshRenderState.GetMaterial(InteractionState);
		}

		const UMaterialInterface* GetCurrentMaterial() const
		{
			return MeshRenderState.GetMaterial(InteractionState);
		}

		// Returns the mesh vertex color based on the current interaction state.
		FLinearColor GetCurrentVertexColor()
		{
			return MeshRenderState.GetVertexColor(InteractionState);
		}

		// Returns the mesh vertex color based on the current interaction state.
		FLinearColor GetCurrentVertexColor() const
		{
			return MeshRenderState.GetVertexColor(InteractionState);
		}

		// Returns the line color based on the current interaction state.
		FLinearColor GetCurrentLineColor()
		{
			return LineRenderState.GetLineColor(InteractionState);
		}

		// Returns the line color based on the current interaction state.
		FLinearColor GetCurrentLineColor() const
		{
			return LineRenderState.GetLineColor(InteractionState);
		}
		
		// Returns the flatten scale if the LocalToWorldTransform is safe to flatten. Otherwise, returns an unscaled matrix.
		INTERACTIVETOOLSFRAMEWORK_API FScaleMatrix SafeFlattenMatrix() const;
		
		// Returns the flatten scale if the provided LocalToWorldTransform is safe to flatten. Otherwise, returns an unscaled matrix.
		// Useful if the transform matrix is altered elsewhere, and you need to ensure proper scaling for rendering.
		INTERACTIVETOOLSFRAMEWORK_API FScaleMatrix SafeFlattenMatrix(const FTransform& InLocalToWorldTransform) const;
	};

	struct FLineTraceTraversalState
	{
		// LocalToWorld transform 
		// Note: non-uniform scale is not supported and the X scale element will be used for uniform scaling.
		FTransform LocalToWorldTransform;

		// Pixel to world scale
		double PixelToWorldScale = 1.0;

		// View context is perspective projection
		bool bIsPerspectiveProjection = true;

		// The criteria that determines the singular element when multiple are hit.
		EGizmoElementHitSortType HitSortType = EGizmoElementHitSortType::Closest;

		// Initialize state
		INTERACTIVETOOLSFRAMEWORK_API void Initialize(const UGizmoViewContext* InGizmoViewContext, FTransform InTransform);
	};

	/** Contains information about a line trace, in addition to FInputRayHit. */
	struct FLineTraceOutput
	{
		/** Client-defined hit priority. Higher priorities can optionally be used to prefer one hit over another */
		int32 HitPriority = 0;

		/** True if the ray hit the actual surface, rather than within a proximity (specified by PixelHitDistanceThreshold in GizmoElements). */
		bool bIsSurfaceHit = false;
	};

public:

	// Render enabled visible element.
	virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) PURE_VIRTUAL(UGizmoElementBase::Render);

	// Do screen space drawing.
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState);
	
	// Line trace enabled hittable element.
	virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) PURE_VIRTUAL(UGizmoElementBase::LineTrace, return FInputRayHit(););

	// Line trace enabled hittable element.
	UE_DEPRECATED(5.7, "Use the version of LineTrace that outputs FLineTraceOutput instead.")
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection);

	// Set/get the visible bit in element state.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetVisibleState(bool bVisible);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetVisibleState() const;

	// Set/get the hittable bit in element state.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHittableState(bool bHittable);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetHittableState() const;

	// Set/get the hit priority. When hits overlap, higher priority elements will be hit first. If two elements have the same priority, the one that was closest to the hit location will be returned.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHitPriority(const int32 InHitPriority);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetHitPriority() const;

	// Element enabled flag. Render and LineTrace only occur when bEnabled is true.
	// This flag is useful for turning on and off an element globally while retaining its specific settings.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabled(bool bInEnabled);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabled() const;

	// Whether element is enabled for perspective projections.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledForPerspectiveProjection(bool bInEnabledForPerspectiveProjection);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForPerspectiveProjection();

	// Whether element is enabled for orthographic projections.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledInOrthographicProjection(bool bInEnabledForOrthographicProjection);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledInOrthographicProjection();

	// Whether element is enabled when element state is default.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledForDefaultState(bool bInEnabledForDefaultState);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForDefaultState();

	// Whether element is enabled when element state is hovering.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledForHoveringState(bool bInEnabledForHoveringState);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForHoveringState();

	// Whether element is enabled when element state is interacting.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledForInteractingState(bool bInEnabledForInteractingState);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForInteractingState();

	// Whether element is enabled when element state is selected.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledForSelectedState(bool bInEnabledForSelectedState);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForSelectedState();

	// Whether element is enabled when element state is subdued.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEnabledForSubduedState(bool bInEnabledForSubduedState);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForSubduedState();

	// Sets the custom visibility function. Note that this is called in addition to built-in logic rather than as a replacement.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetIsVisibleFunction(
		TUniqueFunction<bool(
			const FSceneView* InView, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState InCurrentInteractionState, 
			const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter)>&& InIsVisibleFunction);

	// Sets the custom hittability function. Note that this is called in addition to built-in logic rather than as a replacement.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetIsHittableFunction(
		TUniqueFunction<bool(const UGizmoViewContext* InViewContext, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter)>&& InIsHittableFunction);

	// For an element hierarchy representing multiple parts of a single gizmo, the part identifier establishes 
	// a correspondence between a gizmo part and the elements that represent that part. The recognized
	// part identifier values should be defined in the gizmo. Gizmo part identifiers must be greater than or 
	// equal to one. Identifier 0 is reserved for the default ID which should be assigned to elements
	// that do not correspond to any gizmo part, such as non-hittable decorative elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartIdentifier(uint32 InPartId);
	INTERACTIVETOOLSFRAMEWORK_API virtual uint32 GetPartIdentifier();
	INTERACTIVETOOLSFRAMEWORK_API virtual uint32 GetPartIdentifier() const;

	// Sets the part identifier for this element and all of its children whose current ID is 0/Default.
	// If bInOverrideSet is true, the PartId will be set even if it's currently 0. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartIdentifier(uint32 InPartId, const bool bInOverrideUnsetChildren, const bool bInOverrideSet = true);

	// Returns the element associated with the specified part id, or nullptr if not found.
	INTERACTIVETOOLSFRAMEWORK_API virtual UGizmoElementBase* FindPartElement(const uint32 InPartId);

	// Returns the element associated with the specified part id, or nullptr if not found.
	INTERACTIVETOOLSFRAMEWORK_API virtual const UGizmoElementBase* FindPartElement(const uint32 InPartId) const;

	// Get the (top-level) sub-elements, if any.
	INTERACTIVETOOLSFRAMEWORK_API virtual TConstArrayView<TObjectPtr<UGizmoElementBase>> GetSubElements() const;

	// Object type bitmask indicating whether this object is visible or hittable or both
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetElementState(EGizmoElementState InElementState);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementState GetElementState() const;

	// Object interaction state - None, Hovering, Interacting, Selected or Subdued
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetElementInteractionState(EGizmoElementInteractionState InInteractionState);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementInteractionState GetElementInteractionState() const;

	// Update element's visibility state if element is associated with the specified gizmo part, return true if part was found.
	UE_DEPRECATED(5.7, "Use UpdatePartVisibleState with bInAllowMultipleElements instead.")
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartVisibleState(bool bVisible, uint32 InPartIdentifier);

	// Update element's visibility state if element is associated with the specified gizmo part, return true if part was found.
	// Setting bAllowMultipleElements to true will allow multiple elements with the same Part Id to be updated.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartVisibleState(bool bVisible, uint32 InPartIdentifier, bool bInAllowMultipleElements);

	// Get element's visible state for element associated with the specified gizmo part, if part id was found.
	INTERACTIVETOOLSFRAMEWORK_API virtual TOptional<bool> GetPartVisibleState(uint32 InPartIdentifier) const;

	// Update element's hittable state if element is associated with the specified gizmo part, return true if part id was found.
	UE_DEPRECATED(5.7, "Use UpdatePartHittableState with bInAllowMultipleElements instead.")
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartHittableState(bool bHittable, uint32 InPartIdentifier);

	// Update element's hittable state if element is associated with the specified gizmo part, return true if part id was found.
	// Setting bAllowMultipleElements to true will allow multiple elements with the same Part Id to be updated.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartHittableState(bool bHittable, uint32 InPartIdentifier, bool bInAllowMultipleElements);

	// Get element's hittable state for element associated with the specified gizmo part, if part id was found.
	INTERACTIVETOOLSFRAMEWORK_API virtual TOptional<bool> GetPartHittableState(uint32 InPartIdentifier) const;

	// Update element's interaction state if element is associated with the specified gizmo part, return true if part id was found.
	UE_DEPRECATED(5.7, "Use UpdatePartInteractionState with bInAllowMultipleElements instead.")
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartInteractionState(EGizmoElementInteractionState InInteractionState, uint32 InPartIdentifier);

	// Update element's interaction state if element is associated with the specified gizmo part, return true if part id was found.
	// Setting bAllowMultipleElements to true will allow multiple elements with the same Part Id to be updated.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartInteractionState(EGizmoElementInteractionState InInteractionState, uint32 InPartIdentifier, bool bInAllowMultipleElements);

	// Get element's interaction state for element associated with the specified gizmo part, if part id was found.
	INTERACTIVETOOLSFRAMEWORK_API virtual TOptional<EGizmoElementInteractionState> GetPartInteractionState(uint32 InPartIdentifier) const;

	// View-dependent type - None, Axis or Plane. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewDependentType(EGizmoElementViewDependentType ViewDependentType);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementViewDependentType GetViewDependentType() const;

	// View-dependent angle tolerance in radians 
	//   For Axis, object is culled when angle between view dependent axis and view direction is less than tolerance angle.
	//   For Planar, cos of angle between view dependent axis (plane normal) and view direction. 
	// When the view direction is within this tolerance from the plane or axis, this object will be culled.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewDependentAngleTol(float InMaxAngleTol);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetViewDependentAngleTol() const;

	// View-dependent axis or plane normal, based on the view-dependent type.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewDependentAxis(FVector InAxis);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetViewDependentAxis() const;

	// View align type: None, PointEye, PointOnly, or Axial.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewAlignType(EGizmoElementViewAlignType InViewAlignType);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementViewAlignType GetViewAlignType() const;

	// View align axis. 
	// PointEye, PointScreen and PointOnly rotate this axis to align with view up.
	// Axial rotates about this axis.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewAlignAxis(FVector InAxis);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetViewAlignAxis() const;

	// View align normal.
	// PointEye rotates the normal to align with camera view direction.
	// PointScreen rotates the normal to align with screen forward direction.
	// Axial rotates the normal around the axis to align as closely as possible with the view direction.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewAlignNormal(FVector InAxis);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetViewAlignNormal() const;

	// View-align angle tolerance in radians.
	// Viewer alignment will not occur when the viewing angle is within this angle of view align axis.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewAlignAxialAngleTol(float InMaxAngleTol);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetViewAlignAxialAngleTol() const;

	// Additional view-space offset to apply to the element, affecting both visuals and the hit geometry. Greater values are further away from the camera.
	// An example use case: A large sphere and view-facing rectangle are both at the origin, but the rectangle should draw on top of the sphere.
	// The sphere would otherwise overlap the rectangle.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetViewDepthOffset(float InViewDepthOffset);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetViewDepthOffset() const;

	// Pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPixelHitDistanceThreshold(float InPixelHitDistanceThreshold);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetPixelHitDistanceThreshold() const;

	// Minimum pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing the "exact" element (useful for pixel-thin elements where an "exact" hit would be otherwise challenging to hit).
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetMinimumPixelHitDistanceThreshold(const float InMinimumPixelHitDistanceThreshold);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetMinimumPixelHitDistanceThreshold() const;

	//
	// Methods for managing render state attributes: Material, HoverMaterial, InteractMaterial, SelectMaterial, SubdueMaterial, VertexColor 
	// 
	// State inheritance works as follows: 
	// - Gizmo element state that is not set inherits from the corresponding state in the current render traversal.
	// - Gizmo element state that is set replaces the corresponding state in the current render traversal, except in the case of overrides.
	// - Gizmo element state that is set to override, will override any corresponding state in children.
	//

	// Set mesh render state material attribute. 
	//  @param InMaterial - material to be set
	//  @param InOverridesChildState - when true, this material will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetMaterial(TWeakObjectPtr<UMaterialInterface> InMaterial, bool InOverridesChildState = false);

	// Get mesh render state material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual const UMaterialInterface* GetMaterial() const;

	// Get mesh render state material attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesMaterialOverrideChildState() const;

	// Clear mesh render state material attribute. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearMaterial();

	// Set mesh render state hover material attribute. 
	//  @param InHoverMaterial - hover material to be set
	//  @param InOverridesChildState - when true, this hover material will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHoverMaterial(TWeakObjectPtr<UMaterialInterface> InHoverMaterial, bool InOverridesChildState = false);

	// Get mesh render state hover material attribute's value.
	INTERACTIVETOOLSFRAMEWORK_API virtual const UMaterialInterface* GetHoverMaterial() const;

	// Get mesh render state hover material attribute's override setting.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesHoverMaterialOverrideChildState() const;

	// Clear mesh render state hover material attribute. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearHoverMaterial();

	// Set mesh render state interact material attribute. 
	//  @param InInteractMaterial - interact material to be set
	//  @param InOverridesChildState - when true, this interact material will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInteractMaterial(TWeakObjectPtr<UMaterialInterface> InInteractMaterial, bool InOverridesChildState = false);

	// Get mesh render state interact material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual const UMaterialInterface* GetInteractMaterial() const;

	// Get mesh render state interact material attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesInteractMaterialOverrideChildState() const;

	// Clear mesh render interact state material attribute. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearInteractMaterial();

	// Set mesh render state Select material attribute. 
	//  @param InSelectMaterial - Select material to be set
	//  @param InOverridesChildState - when true, this Select material will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSelectMaterial(TWeakObjectPtr<UMaterialInterface> InSelectMaterial, bool InOverridesChildState = false);

	// Get mesh render state Select material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual const UMaterialInterface* GetSelectMaterial() const;

	// Get mesh render state Select material attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesSelectMaterialOverrideChildState() const;

	// Clear mesh render Select state material attribute. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSelectMaterial();

	// Set mesh render state Subdue material attribute. 
	//  @param InSubdueMaterial - Subdue material to be set
	//  @param InOverridesChildState - when true, this Subdue material will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubdueMaterial(TWeakObjectPtr<UMaterialInterface> InSubdueMaterial, bool InOverridesChildState = false);

	// Get mesh render state Subdue material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual const UMaterialInterface* GetSubdueMaterial() const;

	// Get mesh render state Subdue material attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesSubdueMaterialOverrideChildState() const;

	// Clear mesh render Subdue state material attribute. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSubdueMaterial();

	// Set mesh render state vertex color attribute. 
	//  @param InVertexColor - vertex color to be set
	//  @param InOverridesChildState - when true, this vertex color will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetVertexColor(FLinearColor InVertexColor, bool InOverridesChildState = false);

	// Get mesh render state vertex color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetVertexColor() const;

	// Returns true, if mesh render state vertex color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasVertexColor() const;

	// Get mesh render state vertex color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesVertexColorOverrideChildState() const;

	// Clear mesh render state vertex color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearVertexColor();

	// Set mesh render state vertex color attribute. 
	//  @param InVertexColor - vertex color to be set
	//  @param InOverridesChildState - when true, this vertex color will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHoverVertexColor(FLinearColor InVertexColor, bool InOverridesChildState = false);

	// Get mesh render state vertex color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetHoverVertexColor() const;

	// Returns true, if mesh render state vertex color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasHoverVertexColor() const;

	// Get mesh render state vertex color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesHoverVertexColorOverrideChildState() const;

	// Clear mesh render state vertex color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearHoverVertexColor();

	// Set mesh render state vertex color attribute. 
	//  @param InVertexColor - vertex color to be set
	//  @param InOverridesChildState - when true, this vertex color will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInteractVertexColor(FLinearColor InVertexColor, bool InOverridesChildState = false);

	// Get mesh render state vertex color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetInteractVertexColor() const;

	// Returns true, if mesh render state vertex color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasInteractVertexColor() const;

	// Get mesh render state vertex color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesInteractVertexColorOverrideChildState() const;

	// Clear mesh render state vertex color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearInteractVertexColor();

	// Set mesh render state vertex color attribute. 
	//  @param InVertexColor - vertex color to be set
	//  @param InOverridesChildState - when true, this vertex color will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSelectVertexColor(FLinearColor InVertexColor, bool InOverridesChildState = false);

	// Get mesh render state vertex color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetSelectVertexColor() const;

	// Returns true, if mesh render state vertex color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasSelectVertexColor() const;

	// Get mesh render state vertex color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesSelectVertexColorOverrideChildState() const;

	// Clear mesh render state vertex color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSelectVertexColor();

	// Set mesh render state vertex color attribute. 
	//  @param InVertexColor - vertex color to be set
	//  @param InOverridesChildState - when true, this vertex color will override the material of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubdueVertexColor(FLinearColor InVertexColor, bool InOverridesChildState = false);

	// Get mesh render state vertex color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetSubdueVertexColor() const;

	// Returns true, if mesh render state vertex color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasSubdueVertexColor() const;

	// Get mesh render state vertex color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesSubdueVertexColorOverrideChildState() const;

	// Clear mesh render state vertex color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSubdueVertexColor();

protected:

	// Whether element is enabled. Render and LineTrace only occur when bEnabled is true.
	UPROPERTY()
	bool bEnabled = true;

	// Whether element is enabled for perspective projection
	UPROPERTY()
	bool bEnabledForPerspectiveProjection = true;

	// Whether element is enabled for orthographic projection
	UPROPERTY()
	bool bEnabledForOrthographicProjection = true;

	// Whether element is enabled when element state is default
	UPROPERTY()
	bool bEnabledForDefaultState = true;

	// Whether element is enabled when element state is hovering
	UPROPERTY()
	bool bEnabledForHoveringState = true;

	// Whether element is enabled when element state is interacting
	UPROPERTY()
	bool bEnabledForInteractingState = true;

	// Whether element is enabled when element state is selected
	UPROPERTY()
	bool bEnabledForSelectedState = true;

	// Whether element is enabled when element state is subdued
	UPROPERTY()
	bool bEnabledForSubduedState = true;

	// An optional custom function to determine visibility of the element. Note that this is called in addition to the built-in checks, not in place of it.
	TUniqueFunction<bool(
		const FSceneView* InView, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState InCurrentInteractionState, 
		const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter)> IsVisibleFunction = nullptr;

	// An optional custom function to determine hittability of the element. Note that this is called in addition to the built-in checks, not in place of it.
	TUniqueFunction<bool(const UGizmoViewContext* InViewContext, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter)> IsHittableFunction = nullptr;

	// Part identifier
	UPROPERTY()
	uint32 PartIdentifier = DefaultPartIdentifier;

	// Mesh render state attributes for this element
	UPROPERTY()
	FGizmoElementMeshRenderStateAttributes MeshRenderAttributes;

	// Element state - indicates whether object is visible or hittable
	UPROPERTY()
	EGizmoElementState ElementState = EGizmoElementState::VisibleAndHittable;

	// Current element interaction state - None, Hovering, Interacting, Selected or Subdued
	UPROPERTY()
	EGizmoElementInteractionState ElementInteractionState = EGizmoElementInteractionState::None;

	// View-dependent type - None, Axis or Plane. 
	UPROPERTY()
	EGizmoElementViewDependentType ViewDependentType = EGizmoElementViewDependentType::None;

	// View-dependent axis or plane normal, based on the view-dependent type.
	UPROPERTY()
	FVector ViewDependentAxis = FVector::UpVector;

	// View-dependent angle tolerance based on :
	//   For Axis, minimum radians between view dependent axis and view direction.
	//   For Planar, minimum radians between view dependent axis and the plane where axis is its normal.
	// When the angle between the view direction and the axis/plane is less than this tolerance, this object should be culled.
	UPROPERTY()
	float ViewDependentAngleTol = DefaultViewDependentAngleTol;

	// Axial view alignment minimum cos angle tolerance, computed based on ViewDependentAngleTol. 
	// When the cos of the angle between the view direction and the axis is less than this value, this object should not be culled.
	UPROPERTY()
	float ViewDependentAxialMaxCosAngleTol = DefaultViewDependentAxialMaxCosAngleTol;

	// Planar view alignment minimum cos angle tolerance, computed based on ViewDependentAngleTol. 
	// When the cos of the angle between the view direction and the axis is greater than this value, this object should not be culled.
	UPROPERTY()
	float ViewDependentPlanarMinCosAngleTol = DefaultViewDependentPlanarMinCosAngleTol;

	// View align type: None, PointEye, or PointWorld.
	// PointEye rotates this axis to align with the view up axis.
	// PointWorld rotates this axis to align with the world up axis.
	// Axial rotates around this axis to align the normal as closely as possible to the view direction.
	UPROPERTY()
	EGizmoElementViewAlignType ViewAlignType = EGizmoElementViewAlignType::None;

	// View align axis. 
	UPROPERTY()
	FVector ViewAlignAxis = FVector::UpVector;

	// View align normal.
	// PointEye and PointWorld both rotate the normal to align with the view direction.
	// Axial rotates the normal to align as closely as possible with view direction.
	UPROPERTY()
	FVector ViewAlignNormal = -FVector::ForwardVector;

	// Axial view alignment angle tolerance in radians, based on angle between align normal and view direction. 
	// When angle between the view align normal and the view direction is greater than this angle, the align rotation will be computed.
	UPROPERTY()
	float ViewAlignAxialAngleTol = DefaultViewAlignAngleTol;

	// Axial view alignment minimum cos angle tolerance, computed based on ViewAlignAxialAngleTol. 
	// When the cos of the angle between the view direction and the align normal is less than this value, the align rotation will be computed.
	UPROPERTY()
	float ViewAlignAxialMaxCosAngleTol = DefaultViewAlignMaxCosAngleTol;

	// Pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing.
	UPROPERTY()
	float PixelHitDistanceThreshold = 7.0f;

	// Minimum pixel hit distance threshold, element will be scaled enough to add this threshold when line-tracing the "exact" element (useful for pixel-thin elements where an "exact" hit would be otherwise challenging to hit).
	UPROPERTY()
	float MinimumPixelHitDistanceThreshold = 1.0f;

	// When hits overlap, higher priority elements will be hit first. If two elements have the same priority, the one that was closest to the hit location will be returned.
	UPROPERTY()
	int32 HitPriority = 0;

private:

	// Additional view-space offset to apply to the element, affecting both visuals and the hit geometry.
	UPROPERTY()
	float ViewDepthOffset = 0.0;

protected:

	// Iterate over each top-level element (sub-element), excluding the element itself.
	INTERACTIVETOOLSFRAMEWORK_API void ForEachSubElement(const TFunctionRef<void(UGizmoElementBase*)>& InFunc);

	// Iterate over each element (sub-element) in the element tree, excluding the element itself.
	INTERACTIVETOOLSFRAMEWORK_API void ForEachSubElementRecursive(const TFunctionRef<void(UGizmoElementBase*)>& InFunc) const;

	// Iterate over each sub-element with the given Part Id in the element tree, excluding the element itself.
	INTERACTIVETOOLSFRAMEWORK_API void ForEachSubElementRecursive(const TFunctionRef<void(UGizmoElementBase*)>& InFunc, const uint32 InPartId) const;

	// Get all sub-elements for the element tree, if any. Returns true if any sub-elements were found.
	INTERACTIVETOOLSFRAMEWORK_API bool GetSubElementsRecursive(TArray<TObjectPtr<UGizmoElementBase>>& OutElements) const;

	// Get all sub-elements with the given Part Id for the element tree, if any. Returns true if any sub-elements were found.
	INTERACTIVETOOLSFRAMEWORK_API bool GetSubElementsRecursive(TArray<TObjectPtr<UGizmoElementBase>>& OutElements, const uint32 InPartId) const;

	// Return whether element is currently visible.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsVisible(const FSceneView* View, EViewInteractionState InCurrentViewInteractionState, EGizmoElementInteractionState InCurrentInteractionState, 
		const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const;

	// Return whether element is currently visible.
	UE_DEPRECATED(5.8, "Use IsVisible that accepts a ViewInteractionState instead.")
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsVisible(const FSceneView* View, EGizmoElementInteractionState InCurrentInteractionState, 
		const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const;

	// Return whether element is currently hittable.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsHittable(const UGizmoViewContext* ViewContext, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const;

	// Returns whether object is visible in input FSceneView based on view-dependent visibility settings.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetViewDependentVisibility(const FSceneView* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const;

	// Returns whether object is visible in input gizmo view context based on view-dependent visibility settings
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetViewDependentVisibility(const UGizmoViewContext* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const;

	// Returns whether object is visible based on view-dependent visibility settings.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetViewDependentVisibility(const FVector& InViewLocation, const FVector& InViewDirection, bool bInPerspectiveView, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter) const;

	// Returns whether element is enabled for given interaction state.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForInteractionState(EGizmoElementInteractionState InInteractionState) const;

	// Returns whether element is enabled for given view projection type.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEnabledForViewProjection(bool bIsPerspectiveProjection) const;

	// Return whether this element has a view alignment rotation based on input FSceneView and view-dependent alignment settings.
	// @param OutAlignRot the rotation to align this element in local space, should be prepended to the local-to-world transform.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetViewAlignRot(const FSceneView* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter, FQuat& OutAlignRot) const;

	// Return whether this element has a view alignment rotation based on input gizmo view context and view-dependent alignment settings.
	// @param OutAlignRot the rotation to align this element in local space, should be prepended to the local-to-world transform.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetViewAlignRot(const UGizmoViewContext* View, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter, FQuat& OutAlignRot) const;

	// Return whether this element has a view alignment rotation based on input view parameters and view-dependent alignment settings.
	// @param OutAlignRot the rotation to align this element in local space, should be prepended to the local-to-world transform.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetViewAlignRot(const FVector& InViewLocation, const FVector& InViewDirection, const FVector& InViewUp, bool bInPerspectiveView, const FTransform& InLocalToWorldTransform, const FVector& InLocalCenter, FQuat& OutAlignRot) const;

	INTERACTIVETOOLSFRAMEWORK_API virtual void ApplyViewDepthOffset(const FSceneView* InView, const double& InViewDepthOffset, FTransform& InOutTransform) const;
	
	INTERACTIVETOOLSFRAMEWORK_API virtual void ApplyViewDepthOffset(const FVector& InViewLocation, const FVector& InViewDirection, const bool bInIsPerspectiveView, FTransform& InOutTransform) const;

	INTERACTIVETOOLSFRAMEWORK_API virtual void ApplyViewDepthOffset(const FVector& InViewLocation, const FVector& InViewDirection, const bool bInIsPerspectiveView, const double& InViewDepthOffset, FTransform& InOutTransform) const;

	// Get mesh render state material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetMaterial();

	// Get mesh render state hover material attribute's value.
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetHoverMaterial();

	// Get mesh render state interact material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetInteractMaterial();

	// Get mesh render state Select material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetSelectMaterial();

	// Get mesh render state Subdue material attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetSubdueMaterial();

	// Update render state during render traversal, determines the current render state for this element 
	// @param RenderAPI - tools render context
	// @param InLocalCenter - local element center position
	// @param InOutRenderState - render state's local to world transform will be updated with translation to center and view-dependent alignment rotation, if applicable.
	// @return view dependent visibility, true if this element is visible in the current view. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdateRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalCenter, FRenderTraversalState& InOutRenderState);

	// Update render state during render traversal, determines the current render state for this element
	// Same parameters as UpdateRenderState above plus two output parameters:
	// @param bOutHasAlignRot - whether alignment rotation was applied to output state
	// @param OutAlignRot - alignment rotation applied to output state, if applicable.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdateRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalCenter, FRenderTraversalState& InOutRenderState, bool& bOutHasAlignRot, FQuat& OutAlignRot);

	// Update line trace state during line trace traversal, determines the current state for this element 
	// @param ViewContext - current gizmo view context
	// @param InLocalCenter - local element center position, this will update InOutLineTraceState's LocalToWorldTransform with a translation to LocalCenter
	// @param InOutLineTraceState - line trace state's local to world transform will be updated with translation to center and view-dependent alignment rotation, if applicable.
	// @return view dependent visibility, true if this element is visible in the current view. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdateLineTraceState(const UGizmoViewContext* ViewContext, const FVector& InLocalCenter, FLineTraceTraversalState& InOutLineTraceState);

	// Update line trace state during line trace traversal, determines the current state for this element 
	// Same parameters as UpdateLineTraceState above plus two output parameters:
	// @param bOutHasAlignRot - whether alignment rotation was applied to output state
	// @param OutAlignRot - alignment rotation applied to output state, if applicable.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdateLineTraceState(const UGizmoViewContext* ViewContext, const FVector& InLocalCenter, FLineTraceTraversalState& InOutRenderState, bool& bOutHasAlignRot, FQuat& OutAlignRot);

	// Initialize ray hit for this element, injecting the Part Identifier, Hit Priority, etc.
	[[nodiscard]] INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit MakeRayHit(const double InHitDepth, FLineTraceOutput& OutLineTraceOutput);

protected:

	// Helper method to verify scale is uniform. If it is non-uniform, a one-time warning log is issued.
	// Returns true if scale is uniform.
	INTERACTIVETOOLSFRAMEWORK_API bool VerifyUniformScale(const FVector& Scale) const;

	// Helper method for view alignment.
	// Returns rotation between source and target input coordinate spaces.
	INTERACTIVETOOLSFRAMEWORK_API FQuat GetAlignRotBetweenCoordSpaces(FVector SourceForward, FVector SourceRight, FVector SourceUp, FVector TargetForward, FVector TargetRight, FVector TargetUp) const;
};

