// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementBox.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a solid 3D box based on parameters.
 */

UCLASS(Transient, MinimalAPI)
class UGizmoElementBox : public UGizmoElementBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Location of center of box's base.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCenter(const FVector& InCenter);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetCenter() const;

	// Box axis up direction.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpDirection(const FVector& InUpDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetUpDirection() const;

	// Box axis side direction.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSideDirection(const FVector& InSideDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetSideDirection() const;

	// Box dimensions.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDimensions(const FVector& InDimensions);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetDimensions() const;

protected:

	// Location of center of box
	UPROPERTY()
	FVector Center = FVector::ZeroVector;

	// Box dimensions
	UPROPERTY()
	FVector Dimensions = FVector(1.0f, 1.0f, 1.0f);

	// Box up axis, corresponds to Z dimension
	UPROPERTY()
	FVector UpDirection = FVector::UpVector;

	// Box side axis, corresponds to Y dimension
	UPROPERTY()
	FVector SideDirection = FVector::RightVector;
};

