// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementCircleBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementCircle.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a filled or line circle based on parameters.
 * 
 * The circle element does not yet support partial circles.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementCircle : public UGizmoElementCircleBase
{
	GENERATED_BODY()

public:
	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Draw mesh
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDrawMesh(bool InDrawMesh);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetDrawMesh() const;

	// Draw line
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDrawLine(bool InDrawLine);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetDrawLine() const;

	// Hit mesh
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHitMesh(bool InHitMesh);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetHitMesh() const;

	// Hit line
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHitLine(bool InHitLine);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetHitLine() const;

protected:

	// Whether to render solid circle.
	UPROPERTY()
	bool bDrawMesh = true;

	// Whether to render line circle.
	UPROPERTY()
	bool bDrawLine = false;

	// Whether to perform hit test on mesh.
	UPROPERTY()
	bool bHitMesh = true;

	// Whether to perform hit test on line.
	UPROPERTY()
	bool bHitLine = false;
};
