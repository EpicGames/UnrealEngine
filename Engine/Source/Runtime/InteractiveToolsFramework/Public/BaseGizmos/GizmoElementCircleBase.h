// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementLineBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementCircleBase.generated.h"

class FPrimitiveDrawInterface;
class FMaterialRenderProxy;

/**
 * Abstract base object for circle, torus and arc.
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UGizmoElementCircleBase : public UGizmoElementLineBase
{
	GENERATED_BODY()

	friend class FGizmoElementAccessor;

public:

	// CircleBase center.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCenter(const FVector& InCenter);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetCenter() const;

	// Axis0 (or the Bitangent axis) of the plane in which the circle lies, must be perpendicular to the normal and tangent.  
	// Start and end angles for partial circles are relative to this axis.
	UE_DEPRECATED(5.8, "Use SetAxisBitangent instead")
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetAxis0(const FVector& InAxis0);

	// Bitangent axis of the plane in which the circle lies, must be perpendicular to Axis1/Tangent.
	// Start and end angles for partial circles are relative to this axis.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetAxisBitangent(const FVector& InAxisBitangent);

	UE_DEPRECATED(5.8, "Use GetAxisBitangent instead")
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetAxis0() const;

	// Bitangent axis of the plane in which the circle lies, must be perpendicular to Axis1/Tangent.
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetAxisBitangent() const;

	// Axis1 (usually the "Side" axis) of the plane in which circle lies, must be perpendicular to Axis0.
	UE_DEPRECATED(5.8, "Use SetAxisTangent instead")
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetAxis1(const FVector& InAxis1);

	// Tangent axis of the plane in which the circle lies, must be perpendicular to the normal and bitangent.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetAxisTangent(const FVector& InAxisTangent);

	UE_DEPRECATED(5.8, "Use GetAxisTangent instead")
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetAxis1() const;

	// Tangent axis of the plane in which the circle lies, must be perpendicular to the normal and bitangent.
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetAxisTangent() const;

	// Circle radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetRadius(double InRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetRadius() const;

	// Multiplier applied to the circle radius when hovering
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHoverRadiusMultiplier(double InHoverRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetHoverRadiusMultiplier() const;

	// Multiplier applied to the circle radius when interacting
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInteractRadiusMultiplier(double InInteractRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetInteractRadiusMultiplier() const;

	// Multiplier applied to the circle radius when selected
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSelectRadiusMultiplier(double InSelectRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetSelectRadiusMultiplier() const;

	// Multiplier applied to the circle radius when subdued
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubdueRadiusMultiplier(double InSubdueRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetSubdueRadiusMultiplier() const;

	// Number of segments for rendering circle.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumSegments(int32 InNumSegments);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumSegments() const;

	// When Partial, renders partial arc based on angle.
	// When PartialViewDependent, same as Partial but renders full arc when arc normal aligns with view direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartialType(EGizmoElementPartialType InPartial);
	INTERACTIVETOOLSFRAMEWORK_API virtual EGizmoElementPartialType GetPartialType() const;

	// Start of arc angle of partial torus in radians, relative to Axis0.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartialStartAngle(double InPartialAngle);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetPartialStartAngle() const;

	// Start of arc angle of partial torus in radians, relative to Axis1.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartialEndAngle(double InPartialAngle);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetPartialEndAngle() const;

	// If partial type is PartialViewDependent, when the cosine of angle between the normal and view direction 
	// is within this tolerance, the arc will be rendered as full rather than partial
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartialViewDependentMaxCosTol(double InPartialViewDependentMaxCosTol);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetPartialViewDependentMaxCosTol() const;

protected:

	// CircleBase center.
	UPROPERTY()
	FVector Center = FVector::ZeroVector;

	// Axis0 of plane in which circle lies, must perpendicular to normal. 
	// Start and end angles for partial circles are relative to this axis.
	UPROPERTY()
	FVector Axis0 = FVector::RightVector;

	// Axis1 of plane in which circle lies, must perpendicular to Axis0.
	UPROPERTY()
	FVector Axis1 = FVector::ForwardVector;

	// Radius of main circle, some derived elements have inner radius (e.g. torus and arc)
	UPROPERTY()
	double Radius = 100.0;

	// Optional radius multipliers to apply when hovering, interacting, selected or subdued.
	UPROPERTY()
	FGizmoPerStateValueDouble RadiusMultiplier;

	// Number of segments for rendering arc.
	UPROPERTY()
	int32 NumSegments = 64;

	// True when the arc is not full.
	UPROPERTY()
	EGizmoElementPartialType PartialType = EGizmoElementPartialType::None;

	// Start angle to render for partial torus
	UPROPERTY()
	double PartialStartAngle = 0.0;

	// End angle to render for partial torus
	UPROPERTY()
	double PartialEndAngle = UE_DOUBLE_TWO_PI;

	// For PartialViewDependent, max cosine of angle between the normal and view direction 
	// Within this tolerance, the arc will be rendered as full rather than partial
	UPROPERTY()
	double PartialViewDependentMaxCosTol = 0.96f;

protected:

	// Returns the current radius based on the current interaction state.
	INTERACTIVETOOLSFRAMEWORK_API double GetCurrentRadius() const;

	// Returns whether element should be partial based on current view
	INTERACTIVETOOLSFRAMEWORK_API bool IsPartial(const FSceneView* View, const FVector& InWorldCenter, const FVector& InWorldNormal);

	// Returns whether element should be partial based on current view
	INTERACTIVETOOLSFRAMEWORK_API bool IsPartial(const UGizmoViewContext* ViewContext, const FVector& InWorldCenter, const FVector& InWorldNormal);

	// Returns whether element should be partial based on current view
	INTERACTIVETOOLSFRAMEWORK_API bool IsPartial(const FVector& InWorldCenter, const FVector& InWorldNormal,
		const FVector& InViewLocation, const FVector& InViewDirection,
		const bool bIsPerspectiveProjection);

};
