// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementCone.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a solid 3D cone based on parameters.
 */

UCLASS(Transient, MinimalAPI)
class UGizmoElementCone : public UGizmoElementBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Cone tip location.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetOrigin(const FVector& InBase);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetOrigin() const;

	// Cone axis direction pointing from tip toward base of cone.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDirection(const FVector& InDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetDirection() const;

	// Cone height.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHeight(float InHeight);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetHeight() const;

	// Cone radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetRadius(float InRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetRadius() const;

	// Number of sides for tessellating cone.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumSides(int32 InNumSides);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumSides() const;

	// Whether to render and end cap on the base of the cone.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEndCaps(bool InEndCaps);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEndCaps() const;

protected:

	// Cone tip location.
	UPROPERTY()
	FVector Origin = FVector::ZeroVector;

	// Cone axis direction pointing from tip toward base of cone.
	UPROPERTY()
	FVector Direction = FVector(0.0f, 0.0f, -1.0f);

	// Cone height.
	UPROPERTY()
	float Height = 1.0f;

	// Cone radius.
	UPROPERTY()
	float Radius = 0.5f;

	// Number of sides for tessellating cone.
	UPROPERTY()
	int32 NumSides = 32;
	
	// Whether to render and end cap on the base of the cone.
	UPROPERTY()
	bool bEndCaps = false;
};
