// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementCylinder.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a solid 3D cylinder based on parameters.
 */

UCLASS(Transient, MinimalAPI)
class UGizmoElementCylinder : public UGizmoElementBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Location of center of cylinder's base circle.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetBase(const FVector& InBase);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetBase() const;

	// Cylinder axis direction.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDirection(const FVector& InDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetDirection() const;

	// Cylinder height.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHeight(float InHeight);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetHeight() const;

	// Cylinder radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetRadius(float InRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetRadius() const;

	// Number of sides for tessellating cylinder.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumSides(int32 InNumSides);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumSides() const;

	// Whether to render with dashing.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetIsDashed(bool bInDashing);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetIsDashed() const;

	// The dash parameter, if enabled. If GapLength is not specified, it defaults to half DashLength.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDashParameters(const float InDashLength = 10.0f, const TOptional<float>& InGapLength = TOptional<float>());
	INTERACTIVETOOLSFRAMEWORK_API virtual void GetDashParameters(float& OutDashLength, float& OutGapLength) const;

	// Whether the radius, dash length and spacing (if applicable) are in screen space
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetScreenSpace(bool bInScreenSpace);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetScreenSpace() const;

protected:

	// Location of center of cylinder's base circle.
	UPROPERTY()
	FVector Base = FVector::ZeroVector;

	// Cylinder axis direction.
	UPROPERTY()
	FVector Direction = FVector(0.0f, 0.0f, 1.0f);

	// Cylinder height.
	UPROPERTY()
	float Height = 1.0f;

	// Cylinder radius.
	UPROPERTY()
	float Radius = 0.5f;

	// Number of sides for tessellating cylinder.
	UPROPERTY()
	int32 NumSides = 32;

	// Whether to render with dashing.
	UPROPERTY()
	bool bIsDashed = false;

	// The Dash Length, if Dashing is enabled.
	UPROPERTY()
	float DashLength = 10.0f;

	// The Dash Gap Length, if Dashing is enabled.
	UPROPERTY()
	float DashGapLength = 5.0f;

	// Whether the radius, dash length and spacing (if applicable) are in screen space
	UPROPERTY(Getter = "GetScreenSpace", Setter = "SetScreenSpace")
	bool bScreenSpace = false;
};
