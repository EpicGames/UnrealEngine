// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BaseGizmos/GizmoElementLineBase.h"
#include "InputState.h"
#include "GizmoElementGroup.generated.h"

/**
 * Simple group object intended to be used as part of 3D Gizmos.
 * Contains multiple gizmo objects.
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UGizmoElementGroupBase : public UGizmoElementLineBase
{
	GENERATED_BODY()

	friend class FGizmoElementAccessor;

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;

	// Sets the part identifier for this element and all of its children whose current ID is 0/Default.
	using UGizmoElementBase::SetPartIdentifier;

	// Sets the part identifier for this element and all of its children whose current ID is 0/Default.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPartIdentifier(uint32 InPartId, const bool bInOverrideUnsetChildren, const bool bInOverrideSet = true) override;

	// Returns the element associated with the specified part id, or nullptr if not found.
	INTERACTIVETOOLSFRAMEWORK_API virtual UGizmoElementBase* FindPartElement(const uint32 InPartId) override;

	// Returns the element associated with the specified part id, or nullptr if not found.
	INTERACTIVETOOLSFRAMEWORK_API virtual const UGizmoElementBase* FindPartElement(const uint32 InPartId) const override;

	// Get the (top-level) sub-elements, if any.
	INTERACTIVETOOLSFRAMEWORK_API virtual TConstArrayView<TObjectPtr<UGizmoElementBase>> GetSubElements() const override;

	// Update group and contained elements' visibility state for elements in specified gizmo parts, return true if part was found.
	// Setting bAllowMultipleElements to true will allow multiple elements with the same Part Id to be updated.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartVisibleState(bool bVisible, uint32 InPartIdentifier, bool bInAllowMultipleElements = false) override;

	// Get element's visible state for element associated with the specified gizmo part, if part was found.
	INTERACTIVETOOLSFRAMEWORK_API virtual TOptional<bool> GetPartVisibleState(uint32 InPartIdentifier) const override;

	// Update group and contained elements' hittable state for elements in specified gizmo parts, return true if part was found.
	// Setting bAllowMultipleElements to true will allow multiple elements with the same Part Id to be updated.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartHittableState(bool bHittable, uint32 InPartIdentifier, bool bInAllowMultipleElements = false) override;

	// Get element's hittable state for element associated with the specified gizmo part, if part was found.
	INTERACTIVETOOLSFRAMEWORK_API virtual TOptional<bool> GetPartHittableState(uint32 InPartIdentifier) const override;

	// Update group and contained elements' interaction state for elements in specified gizmo parts, return true if part was found.
	// Setting bAllowMultipleElements to true will allow multiple elements with the same Part Id to be updated.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdatePartInteractionState(EGizmoElementInteractionState InInteractionState, uint32 InPartIdentifier, bool bInAllowMultipleElements = false) override;

	// Get element's interaction state for element associated with the specified gizmo part, if part was found.
	INTERACTIVETOOLSFRAMEWORK_API virtual TOptional<EGizmoElementInteractionState> GetPartInteractionState(uint32 InPartIdentifier) const override;

	// When true, maintains view-dependent constant scale for this gizmo object hierarchy
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetConstantScale(bool InConstantScale);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetConstantScale() const;

	// Set whether this group should be treated as a hit owner and its part identifier returned when any of its sub-elements are hit.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHitOwner(bool bInHitOwner);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetHitOwner() const;

protected:

	// Add object to group.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Add(UGizmoElementBase* InElement);

	// Remove object from group, if it exists
	INTERACTIVETOOLSFRAMEWORK_API virtual void Remove(UGizmoElementBase* InElement);

protected:

	// When true, maintains view-dependent constant scale for this gizmo object hierarchy
	UPROPERTY()
	bool bConstantScale = false;

	// When true, this group is treated as a single element such that when LineTrace is called, if any of its sub-elements is hit, 
	// this group will be returned as the owner of the hit. This should be used when a group of elements should be treated as a single handle.
	UPROPERTY()
	bool bHitOwner = false;
		
	// Gizmo elements within this group
	UPROPERTY()
	TArray<TObjectPtr<UGizmoElementBase>> Elements;

	// Updates input transform's scale component to have uniform scale and applies constant scale if bConstantScale is true
	INTERACTIVETOOLSFRAMEWORK_API virtual void ApplyUniformConstantScaleToTransform(double PixelToWorldScale, FTransform& InOutLocalToWorldTransform) const;

	// Updates input transform's scale component to have uniform scale and applies constant scale if bConstantScale or bInForceApply is true
	INTERACTIVETOOLSFRAMEWORK_API virtual void ApplyUniformConstantScaleToTransform(double PixelToWorldScale, FTransform& InOutLocalToWorldTransform, const bool bInForceApply) const;
};

/**
 * Simple group object intended to be used as part of 3D Gizmos.
 * Contains multiple gizmo objects.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementGroup : public UGizmoElementGroupBase
{
	GENERATED_BODY()

public:

	// Add object to group.
	using UGizmoElementGroupBase::Add;

	// Remove object from group, if it exists
	using UGizmoElementGroupBase::Remove;
};
