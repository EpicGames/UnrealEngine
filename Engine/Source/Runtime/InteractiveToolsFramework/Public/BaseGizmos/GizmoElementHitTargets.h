// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/HitTargets.h"
#include "BaseGizmos/TransformProxy.h"
#include "GizmoElementHitTargets.generated.h"

namespace UE::GizmoRenderingUtil
{
	class IViewBasedTransformAdjuster;
}

class UGizmoElementBase;

/**
 * UGizmoElementHitTarget is an IGizmoClickTarget implementation that
 * hit-tests any object derived from UGizmoElementBase
 * This hit target should be used for hitting a whole gizmo element hierarchy.
 * Use UGizmoElementHitMultiTarget, for hit targets that support hitting parts within a gizmo element hierarchy.
 */
UCLASS(MinimalAPI)
class UGizmoElementHitTarget : public UObject, public IGizmoClickTarget
{
	GENERATED_BODY()
public:

	/**
	 * Gizmo element.
	 */
	UPROPERTY()
	TObjectPtr<UGizmoElementBase> GizmoElement;

	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext;

	UPROPERTY()
	TObjectPtr<UTransformProxy> GizmoTransformProxy;

	/**
	 * If set, this condition is checked before performing the hit test. This gives a way
	 * to disable the hit test without hiding the component. This is useful, for instance,
	 * in a repositionable transform gizmo in world-coordinate mode, where the rotation
	 * components need to be hittable for movement, but not for repositioning.
	 */
	TFunction<bool(const FInputDeviceRay&)> Condition = nullptr;

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHoverState(bool bHovering) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateInteractingState(bool bInteracting) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSelectedState(bool bInSelected) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSubdueState(bool bSubdued) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransformAdjuster(const TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster>& InAdjuster) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> GetTransformAdjuster() const override;

public:
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoElementHitTarget* Construct(
		UGizmoElementBase* InGizmoElement,
		UGizmoViewContext* InGizmoViewContext,
		UObject* Outer = (UObject*)GetTransientPackage());

protected:
	/**
	 * An optional transform adjuster to apply when testing hit geometry.
	 * Note that while the name indicates it's view-based, it may also be used to apply any transform.
	 */
	TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> TransformAdjuster = nullptr;
};

/**
 * UGizmoElementHitMultiTarget is an IGizmoClickMultiTarget implementation that
 * hit-tests any object derived from UGizmoElementBase. This implementation is used for 
 * HitTargets which support hitting multiple parts within a gizmo element hierarchy. 
 *
 * For a gizmo with multiple parts, the part identifier establishes a correspondence between a gizmo part 
 * and the elements representing that part within the hit target. The valid part identifiers should 
 * be defined in the gizmo. Identifier 0 is reserved for the default ID which should be assigned to 
 * elements that do not correspond to any gizmo part, such as non-hittable decorative elements.
 */
UCLASS(MinimalAPI)
class UGizmoElementHitMultiTarget : public UObject, public IGizmoClickMultiTarget
{
	GENERATED_BODY()
public:

	/**
	 * Gizmo element.
	 */
	UPROPERTY()
	TObjectPtr<UGizmoElementBase> GizmoElement;

	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext;

	UPROPERTY()
	TObjectPtr<UTransformProxy> GizmoTransformProxy;

	/**
	 * If set, this condition is checked before performing the hit test. This gives a way
	 * to disable the hit test without hiding the component. This is useful, for instance,
	 * in a repositionable transform gizmo in world-coordinate mode, where the rotation
	 * components need to be hittable for movement, but not for repositioning.
	 */
	TFunction<bool(const FInputDeviceRay&)> Condition = nullptr;

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHoverState(bool bHovering, uint32 PartIdentifier) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHoverState(bool bHovering, uint32 PartIdentifier, bool bInAllowMultipleElements);

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateInteractingState(bool bInteracting, uint32 PartIdentifier) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateInteractingState(bool bInteracting, uint32 PartIdentifier, bool bInAllowMultipleElements);

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSelectedState(bool bInSelected, uint32 PartIdentifier) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSelectedState(bool bInSelected, uint32 PartIdentifier, bool bInAllowMultipleElements);

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSubdueState(bool bInSubdued, uint32 PartIdentifier) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSubdueState(bool bInSubdued, uint32 PartIdentifier, bool bInAllowMultipleElements);

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHittableState(bool bHittable, uint32 PartIdentifier) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHittableState(bool bHittable, uint32 PartIdentifier, bool bInAllowMultipleElements);

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransformAdjuster(const TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster>& InAdjuster) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> GetTransformAdjuster() const override;

public:
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoElementHitMultiTarget* Construct(
		UGizmoElementBase* InGizmoElement,
		UGizmoViewContext* InGizmoViewContext,
		UObject* Outer = (UObject*)GetTransientPackage());

protected:
	/**
	 * An optional transform adjuster to apply when testing hit geometry.
	 * Note that while the name indicates it's view-based, it may also be used to apply any transform.
	 */
	TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> TransformAdjuster = nullptr;
};
