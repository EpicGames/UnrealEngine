// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "BaseGizmos/GizmoElementRenderState.h"
#include "BaseGizmos/GizmoElementShared.h"
#include "ToolContextInterfaces.h"
#include "GizmoElementLineBase.generated.h"

/**
 * Base class for 2d and 3d primitive objects which support line drawing,
 * intended to be used as part of 3D Gizmos.
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UGizmoElementLineBase : public UGizmoElementBase
{
	GENERATED_BODY()
public:

	// Get line thickness for based on current element interaction state and view
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetCurrentLineThickness(bool bPerspectiveView, float InViewFOV) const;

	// Line thickness when rendering lines, 0.0 is valid and will render thinnest line 
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetLineThickness(float InLineThickness);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetLineThickness() const;

	// Multiplier applied to line thickness when hovering
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHoverLineThicknessMultiplier(float InHoverLineThicknessMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetHoverLineThicknessMultiplier() const;

	// Multiplier applied to line thickness when interacting
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInteractLineThicknessMultiplier(float InInteractLineThicknessMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetInteractLineThicknessMultiplier() const;

	// Multiplier applied to line thickness when selected
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSelectLineThicknessMultiplier(float InSelectLineThicknessMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetSelectLineThicknessMultiplier() const;

	// Multiplier applied to line thickness when subdued
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubdueLineThicknessMultiplier(float InSubdueLineThicknessMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetSubdueLineThicknessMultiplier() const;

	// Whether line thickness is in screen space 
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetScreenSpaceLine(bool bInScreenSpaceLine);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetScreenSpaceLine() const;

	//
	// Methods for managing line state attributes: LineColor, HoverLineColor, InteractLineColor, SelectLineColor, SubdueLineColor
	//
	// State inheritance works as follows:
	// - Gizmo element state that is not set inherits from the corresponding state in the current render traversal.
	// - Gizmo element state that is set replaces the corresponding state in the current render traversal, except in the case of overrides.
	// - Gizmo element state that is set to override, will override any corresponding state in children.
	//

	// Set line render state line color attribute. 
	//  @param InMaterial - line color to be set
	//  @param InOverridesChildState - when true, this line color will override the line color of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetLineColor(FLinearColor InColor, bool InOverridesChildState = false);

	// Get line render state line color attribute's value.
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetLineColor() const;

	// Returns true if line render state line color attribute has been set.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasLineColor() const;

	// Get line render state line color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesLineColorOverrideChildState() const;

	// Clear line render state line color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearLineColor();

	// Set line render state hover line color attribute. 
	//  @param InMaterial - hover line color to be set
	//  @param InOverridesChildState - when true, this hover line color will override the line color of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHoverLineColor(FLinearColor InColor, bool InOverridesChildState = false);

	// Get line render state hover line color attribute's value.
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetHoverLineColor() const;

	// Returns true if line render state hover line color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasHoverLineColor() const;

	// Get line render state hover line color attribute's override setting.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesHoverLineColorOverrideChildState() const;

	// Clear line render state hover line color attribute. 
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearHoverLineColor();

	// Set line render state interact line color attribute. 
	//  @param InMaterial - interact line color to be set
	//  @param InOverridesChildState - when true, this interact line color will override the line color of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInteractLineColor(FLinearColor InColor, bool InOverridesChildState = false);

	// Get line render state interact line color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetInteractLineColor() const;

	// Returns true if line render state interact line color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasInteractLineColor() const;

	// Get line render state interact line color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesInteractLineColorOverrideChildState() const;

	// Clear line render state interact line color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearInteractLineColor();

	// Set line render state Select line color attribute. 
	//  @param InColor - Select line color to be set
	//  @param InOverridesChildState - when true, this Select line color will override the line color of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSelectLineColor(FLinearColor InColor, bool InOverridesChildState = false);

	// Get line render state interact line color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetSelectLineColor() const;

	// Returns true if line render state interact line color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasSelectLineColor() const;

	// Get line render state interact line color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesSelectLineColorOverrideChildState() const;

	// Clear line render state interact line color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSelectLineColor();

	// Set line render state subdue line color attribute. 
	//  @param InColor - subdue line color to be set
	//  @param InOverridesChildState - when true, this subdue line color will override the line color of all child elements.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubdueLineColor(FLinearColor InColor, bool InOverridesChildState = false);

	// Get line render state interact line color attribute's value. 
	INTERACTIVETOOLSFRAMEWORK_API virtual FLinearColor GetSubdueLineColor() const;

	// Returns true if line render state interact line color attribute has been set. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasSubdueLineColor() const;

	// Get line render state interact line color attribute's override setting. 
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DoesSubdueLineColorOverrideChildState() const;

	// Clear line render state interact line color attribute.
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSubdueLineColor();

protected:

	// Line render state attributes for this element
	UPROPERTY()
	FGizmoElementLineRenderStateAttributes LineRenderAttributes;

	// Line thickness when rendering lines, must be >= 0.0, value of 0.0 will render thinnest line 
	UPROPERTY()
	float LineThickness = 0.0;

	// Whether line thickness is in screen space
	UPROPERTY()
	bool bScreenSpaceLine = true;

	// Multiplier applied to line thickness when hovering
	UPROPERTY(EditAnywhere, Category = Options)
	float HoverLineThicknessMultiplier = 2.0f;

	// Multiplier applied to line thickness when interacting
	UPROPERTY(EditAnywhere, Category = Options)
	float InteractLineThicknessMultiplier = 2.0f;

	// Multiplier applied to line thickness when selected
	UPROPERTY(EditAnywhere, Category = Options)
	float SelectLineThicknessMultiplier = 2.0f;

	// Multiplier applied to line thickness when subdued
	UPROPERTY(EditAnywhere, Category = Options)
	float SubdueLineThicknessMultiplier = 1.0f;

	// Update render state during render traversal, determines the current render state for this element
	// @return view dependent visibility, true if this element is visible in the current view.
	INTERACTIVETOOLSFRAMEWORK_API virtual bool UpdateRenderState(IToolsContextRenderAPI* RenderAPI, const FVector& InLocalOrigin, FRenderTraversalState& InOutRenderState) override;
};
