// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementLineBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementLineStrip.generated.h"

class FPrimitiveDrawInterface;
class FMaterialRenderProxy;

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a rectangle based on parameters.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementLineStrip : public UGizmoElementLineBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// The vertices of the line strip
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetVertices(const TArrayView<const FVector>& InVertices);
	INTERACTIVETOOLSFRAMEWORK_API virtual const TArray<FVector>& GetVertices() const;

	// Base Location of the line strip
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetBase(FVector InBase);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetBase() const;

	// Up direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpDirection(const FVector& InUpDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetUpDirection() const;

	// Side direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSideDirection(const FVector& InSideDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetSideDirection() const;

	// Defines if the vertices should be treated as a connected strip or separate line segments
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDrawLineStrip(bool InDrawLineStrip);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetDrawLineStrip() const;

protected:

	void ComputeProjectedVertices(const FTransform& InLocalToWorldTransform, bool bIncludeBase);

	// The vertices of the line strip
	UPROPERTY()
	TArray<FVector> Vertices;

	// Base Location of the line strip
	UPROPERTY()
	FVector Base = FVector::ZeroVector;

	// Up direction
	UPROPERTY()
	FVector UpDirection = FVector(0.0f, 0.0f, 1.0f);

	// Side direction
	UPROPERTY()
	FVector SideDirection = FVector(0.0f, 1.0f, 0.0f);

	// Defines if the vertices should be treated as a connected strip or separate line segments
	UPROPERTY()
	bool bDrawLineStrip = true;

	TArray<FVector> ProjectedVertices;
};
