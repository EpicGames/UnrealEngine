// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementLineBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementRectangle.generated.h"

class FPrimitiveDrawInterface;
class FMaterialRenderProxy;

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a rectangle based on parameters.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementRectangle : public UGizmoElementLineBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Location of rectangle center
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCenter(FVector InCenter);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetCenter() const;

	// Width
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetWidth(float InWidth);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetWidth() const;

	// Height
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHeight(float InHeight);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetHeight() const;

	// Up direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpDirection(const FVector& InUpDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetUpDirection() const;

	// Side direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSideDirection(const FVector& InSideDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetSideDirection() const;

	// Draw mesh
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDrawMesh(bool InDrawMesh);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetDrawMesh() const;

	// Draw line
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDrawLine(bool InDrawLine);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetDrawLine() const;

	// Hit mesh
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHitMesh(bool InHitMesh);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetHitMesh() const;

	// Hit line
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHitLine(bool InHitLine);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetHitLine() const;

protected:

	// Location of rectangle center
	UPROPERTY()
	FVector Center = FVector::ZeroVector;

	// Width
	UPROPERTY()
	float Width = 1.0f;

	// Height
	UPROPERTY()
	float Height = 1.0f;

	// Up direction
	UPROPERTY()
	FVector UpDirection = FVector(0.0f, 0.0f, 1.0f);

	// Side direction
	UPROPERTY()
	FVector SideDirection = FVector(0.0f, 1.0f, 0.0f);

	UPROPERTY()
	bool bDrawMesh = true;

	UPROPERTY()
	bool bDrawLine = false;

	UPROPERTY()
	bool bHitMesh = true;

	UPROPERTY()
	bool bHitLine = false;
};
