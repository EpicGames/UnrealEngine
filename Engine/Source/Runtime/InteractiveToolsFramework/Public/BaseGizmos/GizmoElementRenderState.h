// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "BaseGizmos/GizmoElementShared.h"
#include "Materials/MaterialInterface.h"
#include "GizmoElementRenderState.generated.h"


//
// Structs used to store gizmo element render state.
//

//
// Color state attribute struct 
//
// Stores color state value, whether state is set and whether it overrides child state. 
//
USTRUCT()
struct FGizmoElementColorAttribute
{
	GENERATED_BODY()

	static constexpr FLinearColor DefaultColor = FLinearColor(1.0f, 1.0f, 1.0f);

	// Color value
	UPROPERTY()
	FLinearColor Value = DefaultColor;

	// True if value has been set.
	UPROPERTY()
	bool bHasValue = false;

	// True if this attribute overrides child state attributes
	UPROPERTY()
	bool bOverridesChildState = false;

	// Return the color value. This returns the default color value if no value has been set.
	INTERACTIVETOOLSFRAMEWORK_API FLinearColor GetColor() const;

	// Set color value and override flag
	INTERACTIVETOOLSFRAMEWORK_API void SetColor(FLinearColor InColor, bool InOverridesChildState = false);

	// Reset attribute to default values.
	INTERACTIVETOOLSFRAMEWORK_API void Reset();

	// Update this state based on a child state attribute.
	INTERACTIVETOOLSFRAMEWORK_API void UpdateState(const FGizmoElementColorAttribute& InChildColorAttribute);
};

//
// Material state attribute struct
//
// Stores material state value, whether state is set and whether it overrides child state.
//
USTRUCT()
struct FGizmoElementMaterialAttribute 
{
	GENERATED_BODY()

	// Material value
	UPROPERTY()
	TWeakObjectPtr<UMaterialInterface> Value = nullptr;

	// True if this attribute overrides child state attributes 
	UPROPERTY()
	bool bOverridesChildState = false;

	// Return the material value.
	INTERACTIVETOOLSFRAMEWORK_API const UMaterialInterface* GetMaterial() const;

	// Return the material value.
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetMaterial();

	// Set material value and override flag
	INTERACTIVETOOLSFRAMEWORK_API void SetMaterial(TWeakObjectPtr<UMaterialInterface> InColor, bool InOverridesChildState = false);

	// Reset attribute to default values.
	INTERACTIVETOOLSFRAMEWORK_API void Reset();

	// Update this state based on a child state attribute.
	INTERACTIVETOOLSFRAMEWORK_API void UpdateState(const FGizmoElementMaterialAttribute& InChildMaterialAttribute);
};

//
// Mesh render state structure.
//
USTRUCT()
struct FGizmoElementMeshRenderStateAttributes
{
	GENERATED_BODY()

	// Default material
	UPROPERTY()
	FGizmoElementMaterialAttribute Material;

	// Hover material
	UPROPERTY()
	FGizmoElementMaterialAttribute HoverMaterial;

	// Interact material
	UPROPERTY()
	FGizmoElementMaterialAttribute InteractMaterial;

	// Select material
	UPROPERTY()
	FGizmoElementMaterialAttribute SelectMaterial;

	// Subdue material
	UPROPERTY()
	FGizmoElementMaterialAttribute SubdueMaterial;

	// Mesh vertex color
	UPROPERTY()
	FGizmoElementColorAttribute VertexColor;

	// Hover mesh vertex color
	UPROPERTY()
	FGizmoElementColorAttribute HoverVertexColor;

	// Interact mesh vertex color
	UPROPERTY()
	FGizmoElementColorAttribute InteractVertexColor;

	// Select mesh vertex color
	UPROPERTY()
	FGizmoElementColorAttribute SelectVertexColor;

	// Subdue mesh vertex color
	UPROPERTY()
	FGizmoElementColorAttribute SubdueVertexColor;

	// Returns the material corresponding to the input interaction state
	INTERACTIVETOOLSFRAMEWORK_API const UMaterialInterface* GetMaterial(EGizmoElementInteractionState InteractionState);

	// Returns the material corresponding to the input interaction state
	INTERACTIVETOOLSFRAMEWORK_API const UMaterialInterface* GetMaterial(EGizmoElementInteractionState InteractionState) const;

	// Returns mesh vertex color
	INTERACTIVETOOLSFRAMEWORK_API FLinearColor GetVertexColor(EGizmoElementInteractionState InteractionState);

	// Returns mesh vertex color
	INTERACTIVETOOLSFRAMEWORK_API FLinearColor GetVertexColor(EGizmoElementInteractionState InteractionState) const;

	// Update this mesh render state based on a child mesh render state attribute.
	INTERACTIVETOOLSFRAMEWORK_API void Update(FGizmoElementMeshRenderStateAttributes& InChildAttributes);

private:
	// Returns the material corresponding to the input interaction state
	const UMaterialInterface* GetMaterialInternal(EGizmoElementInteractionState InteractionState) const;

	// Returns mesh vertex color
	FLinearColor GetVertexColorInternal(EGizmoElementInteractionState InteractionState) const;
};

//
// Line render state structure.
//
USTRUCT()
struct FGizmoElementLineRenderStateAttributes
{
	GENERATED_BODY()

	// Line color
	UPROPERTY()
	FGizmoElementColorAttribute LineColor;

	// Hover line color
	UPROPERTY()
	FGizmoElementColorAttribute HoverLineColor;

	// Interact line color
	UPROPERTY()
	FGizmoElementColorAttribute InteractLineColor;

	// Select line color
	UPROPERTY()
	FGizmoElementColorAttribute SelectLineColor;

	// Subdue line color
	UPROPERTY()
	FGizmoElementColorAttribute SubdueLineColor;

	// Returns the line color corresponding to the input interaction state
	INTERACTIVETOOLSFRAMEWORK_API FLinearColor GetLineColor(EGizmoElementInteractionState InteractionState);

	// Returns the line color corresponding to the input interaction state
	INTERACTIVETOOLSFRAMEWORK_API FLinearColor GetLineColor(EGizmoElementInteractionState InteractionState) const;

	// Update this line render state based on a child mesh render state attribute.
	INTERACTIVETOOLSFRAMEWORK_API void Update(FGizmoElementLineRenderStateAttributes& InChildAttributes);

private:
	// Returns the line color corresponding to the input interaction state, shared between the public const and non-const versions
	FLinearColor GetLineColorInternal(EGizmoElementInteractionState InteractionState) const;
};

