// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "FrameTypes.h"

#include "GizmoElementShared.generated.h"

namespace UE::InteractiveToolsFramework
{
	/**
	 * Construct a FFrame3 for the given plane parameters, to remove axis ambiguity and invalid transforms.
	 * Normal = X, Up = Z, Side = Y
	 */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	Geometry::TFrame3<RealType> MakePlaneFrame(
		const Math::TVector<RealType>& InOrigin,
		const Math::TVector<RealType>& InNormal,
		const Math::TVector<RealType>& InUp,
		const Math::TVector<RealType>& InSide)
	{
		// The order is really important here
		return Geometry::TFrame3<RealType>(InOrigin, InNormal, InSide, InUp);
	}

	/** Decompose a plane frame into its origin, normal, up, and side vectors. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void BreakPlaneFrame(
		const Geometry::TFrame3<RealType>& InFrame,
		Math::TVector<RealType>& OutOrigin,
		Math::TVector<RealType>& OutNormal,
		Math::TVector<RealType>& OutUp,
		Math::TVector<RealType>& OutSide)
	{
		// The order is really important here
		OutOrigin = InFrame.Origin;
		OutNormal = InFrame.X();
		OutUp = InFrame.Z();
		OutSide = InFrame.Y();
	}

	/** Decompose a plane frame into its normal, up, and side vectors. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void BreakPlaneFrame(
		const Geometry::TFrame3<RealType>& InFrame,
		Math::TVector<RealType>& OutNormal,
		Math::TVector<RealType>& OutUp,
		Math::TVector<RealType>& OutSide)
	{
		Math::TVector<RealType> UnusedOrigin;
		BreakPlaneFrame<RealType>(InFrame, UnusedOrigin, OutNormal, OutUp, OutSide);
	}

	/** Decompose a plane quaternion into its normal, up, and side vectors. */
	template <typename RealType UE_REQUIRES(std::is_floating_point_v<RealType>)>
	void BreakPlaneQuaternion(
		const FQuat& InQuaternion,
		Math::TVector<RealType>& OutNormal,
		Math::TVector<RealType>& OutUp,
		Math::TVector<RealType>& OutSide)
	{
		// The order is really important here
		OutNormal = InQuaternion.GetAxisX();
		OutUp = InQuaternion.GetAxisZ();
		OutSide = InQuaternion.GetAxisY();
	}
}

//
// Visible/hittable state of gizmo element
//
UENUM()
enum class EGizmoElementState : uint8
{
	None		= 0x00,
	Visible		= 1<<1,
	Hittable	= 1<<2,
	VisibleAndHittable = Visible | Hittable
};

ENUM_CLASS_FLAGS(EGizmoElementState)

//
// Interaction state of gizmo element
//
UENUM()
enum class EGizmoElementInteractionState
{
	None,
	Hovering,
	Interacting,
	Selected,		// When an element is "Selected", it is considered to be the target of Interaction via indirect manipulation (not hit tested).
	Subdued,		// When one or more other elements considered part of a set is active, but this one is not (ie. X Axis is Interacting, Y and Z are Subdued).

	Max
};

//
// View dependent type: automatically cull gizmo element based on view.
//
//   Axis  - Cull object when angle between axis and view direction is within a given tolerance
//   Plane - Cull object when angle between plane normal and view direction is perpendicular within a given tolerance
//
UENUM()
enum class EGizmoElementViewDependentType
{
	None,
	Axis,
	Plane
};

//
// View align type: automatically align gizmo element towards a view.
// 
//   PointOnly   - Align object forward axis to view direction only, useful for symmetrical objects such as a circle
//   PointEye    - Align object forward axis to -camera view direction (camera pos - object center), align object up axis to scene view up
//   PointScreen - Align object forward axis to scene view forward direction (view up ^ view right), align object up axis to scene view up
//   Axial		- Rotate object around up axis, minimizing angle between forward axis and view direction
//
UENUM()
enum class EGizmoElementViewAlignType
{
	None,
	PointOnly,
	PointEye,
	PointScreen,
	Axial
};


//
// Partial type: render partial element for those elements which support it.
// 
//   Partial				- Render partial element.
//   PartialViewDependent   - Render partial unless view direction aligns with an axis or normal specified by the element type.
//
UENUM()
enum class EGizmoElementPartialType
{
	None,
	Partial,
	PartialViewDependent
};

// Can be used to specify what visuals should be created for a given element, when multiple are available, ie. Fill and Line
UENUM()
enum class EGizmoElementDrawType : uint8
{
	None = 0x00,

	Fill = 1 << 1,
	Line = 1 << 2,

	FillAndLine = Fill | Line
};
ENUM_CLASS_FLAGS(EGizmoElementDrawType)

// Used to specify the sort criteria when hit testing against multiple elements
UENUM(Flags)
enum class EGizmoElementHitSortType : uint8
{
	Closest = 1 << 0,	// Sort hits by the closest to ray origin (default)
	Priority = 1 << 1,	// Sort hits by hit priority (see UGizmoElementBase::HitPriority)
	Surface = 1 << 2,	// Sort hits by whether they are surface hits (not proximity hits)

	PriorityThenSurfaceThenClosest = Priority | Surface | Closest, // Sort by all three listed criteria, in order
};
ENUM_CLASS_FLAGS(EGizmoElementHitSortType)

/**
 * Used to store per-state (double) values for gizmo elements.
 * ie. line thickness multipliers.
 */
USTRUCT(MinimalAPI, meta = (DisplayName = "Per-State Value (Double)"))
struct FGizmoPerStateValueDouble
{
	GENERATED_BODY()

	using FValueType = double;

	/**
	 * Default value, used when the Interaction State is "None".
	 * Optional to allow explicit un-setting, implying inheritance or some other value source.
	 */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<double> Default;

	/** Value used when hovering. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<double> Hover;

	/** Value used when interacting. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<double> Interact;

	/** Value used when selected. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<double> Select;

	/** Value used when subdued. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<double> Subdue;

	/** Get the value for the given Interaction State, or Default if not set. */
	INTERACTIVETOOLSFRAMEWORK_API double GetValueForState(const EGizmoElementInteractionState InState) const;

	/** Get the Default value if set, otherwise "ValueDefault" */
	double GetDefaultValue() const { return Default.Get(ValueDefault); }

	/** Get the Hover value, or Default if not set. */
	double GetHoverValue() const { return Hover.Get(GetDefaultValue()); }

	/** Get the Interact value, or Default if not set. */
	double GetInteractValue() const { return Interact.Get(GetDefaultValue()); }

	/** Get the Select value, or Default if not set. */
	double GetSelectValue() const { return Select.Get(GetDefaultValue()); }

	/** Get the Subdue value, or Default if not set. */
	double GetSubdueValue() const { return Subdue.Get(GetDefaultValue()); }

	friend bool operator==(const FGizmoPerStateValueDouble& InLeft, const FGizmoPerStateValueDouble& InRight)
	{
		return InLeft.Default == InRight.Default
			&& InLeft.Hover == InRight.Hover
			&& InLeft.Interact == InRight.Interact
			&& InLeft.Select == InRight.Select
			&& InLeft.Subdue == InRight.Subdue;
	}

	friend bool operator!=(const FGizmoPerStateValueDouble& InLeft, const FGizmoPerStateValueDouble& InRight)
	{
		return !(InLeft == InRight);
	}

private:
	static constexpr double ValueDefault = 1.0;
};

/**
 * Used to store per-state (Linear Color) values for gizmo elements.
 * ie. vertex color.
 */
USTRUCT(MinimalAPI, meta = (DisplayName = "Per-State Value (LinearColor)"))
struct FGizmoPerStateValueLinearColor
{
	GENERATED_BODY()

	using FValueType = FLinearColor;

	/**
	 * Default value, used when the Interaction State is "None".
	 * Optional to allow explicit un-setting, implying inheritance or some other value source.
	 */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FLinearColor> Default;

	/** Value used when hovering. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FLinearColor> Hover;

	/** Value used when interacting. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FLinearColor> Interact;

	/** Value used when selected. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FLinearColor> Select;

	/** Value used when subdued. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TOptional<FLinearColor> Subdue;

	/** Get the value for the given Interaction State, or Default if not set. */
	INTERACTIVETOOLSFRAMEWORK_API const FLinearColor& GetValueForState(const EGizmoElementInteractionState InState) const;

	/** Get the Default value if set, otherwise FLinearColor::Transparent. */
	const FLinearColor& GetDefaultValue() const { return Default.Get(FLinearColor::Transparent); }

	/** Get the Hover value, or Default if not set. */
	const FLinearColor& GetHoverValue() const { return Hover.Get(GetDefaultValue()); }

	/** Get the Interact value, or Default if not set. */
	const FLinearColor& GetInteractValue() const { return Interact.Get(GetDefaultValue()); }

	/** Get the Select value, or Default if not set. */
	const FLinearColor& GetSelectValue() const { return Select.Get(GetDefaultValue()); }

	/** Get the Subdue value, or Default if not set. */
	const FLinearColor& GetSubdueValue() const { return Subdue.Get(GetDefaultValue()); }

	friend bool operator==(const FGizmoPerStateValueLinearColor& InLeft, const FGizmoPerStateValueLinearColor& InRight)
	{
		return InLeft.Default == InRight.Default
			&& InLeft.Hover == InRight.Hover
			&& InLeft.Interact == InRight.Interact
			&& InLeft.Select == InRight.Select
			&& InLeft.Subdue == InRight.Subdue;
	}

	friend bool operator!=(const FGizmoPerStateValueLinearColor& Lhs, const FGizmoPerStateValueLinearColor& RHS)
	{
		return !(Lhs == RHS);
	}
};
