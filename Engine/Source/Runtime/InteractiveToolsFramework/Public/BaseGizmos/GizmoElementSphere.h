// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementSphere.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a solid 3D sphere based on parameters.
 */

UCLASS(Transient, MinimalAPI)
class UGizmoElementSphere : public UGizmoElementBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Location of center of sphere
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCenter(const FVector& InCenter);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetCenter() const;

	// Sphere radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetRadius(float InRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetRadius() const;

	// Number of sides for tessellating sphere.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumSides(int32 InNumSides);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumSides() const;

protected:

	// Location of center of sphere
	UPROPERTY()
	FVector Center = FVector::ZeroVector;

	// Sphere radius.
	UPROPERTY()
	float Radius = 0.5f;

	// Number of sides for tessellating sphere.
	UPROPERTY()
	int32 NumSides = 32;
};

