// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementCircleBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementTorus.generated.h"

class FPrimitiveDrawInterface;
class FMaterialRenderProxy;

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a torus based on parameters.
 * 
 * Note: the LineTrace method does not perform a true ray-torus intersection!
 * See comment above LineTrace method below for details of how this intersection is approximated.
 */
UCLASS(Transient, MinimalAPI)
class UGizmoElementTorus : public UGizmoElementCircleBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;

	// LineTrace approximates ray-torus intersection by intersecting the ray with the plane in which the torus 
	// lies, then determining a hit point closest to the linear circle defined by torus center and torus outer radius.
	// If the torus lies at a glancing angle, the ray-torus intersection is performed against cylinders approximating
	// the shape of the torus.
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// Inner circles radius.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInnerRadius(double InInnerRadius);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetInnerRadius() const;

	// Multiplier applied to the inner radius when hovering
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetHoverInnerRadiusMultiplier(double InHoverInnerRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetHoverInnerRadiusMultiplier() const;

	// Multiplier applied to the inner radius when interacting
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetInteractInnerRadiusMultiplier(double InInteractInnerRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetInteractInnerRadiusMultiplier() const;

	// Multiplier applied to the inner radius when selected
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSelectInnerRadiusMultiplier(double InSelectInnerRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetSelectInnerRadiusMultiplier() const;

	// Multiplier applied to the inner radius when subdued
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSubdueInnerRadiusMultiplier(double InSubdueInnerRadiusMultiplier);
	INTERACTIVETOOLSFRAMEWORK_API virtual double GetSubdueInnerRadiusMultiplier() const;

	// Number of inner slices for rendering torus.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetNumInnerSlices(int32 InNumInnerSlices);
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 GetNumInnerSlices() const;

	// If partial, renders end caps when true.
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetEndCaps(bool InEndCaps);
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetEndCaps() const;

protected:

	// Torus inner radius.
	UPROPERTY()
	double InnerRadius = 5.0;

	// Optional inner radius multipliers to apply when hovering, interacting, selected or subdued.
	UPROPERTY()
	FGizmoPerStateValueDouble InnerRadiusMultiplier;

	// Number of slices to render in each torus segment.
	UPROPERTY()
	int32 NumInnerSlices = 8;

	// Whether to render end caps on a partial torus.
	UPROPERTY()
	bool bEndCaps = false;

protected:

	// Returns the current inner radius based on the current interaction state.
	INTERACTIVETOOLSFRAMEWORK_API double GetCurrentInnerRadius() const;

	INTERACTIVETOOLSFRAMEWORK_API bool IsGlancingAngle(const UGizmoViewContext* View, const FVector& InWorldCenter, const FVector& InWorldNormal);

};
