// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoElementBase.h"
#include "InputState.h"
#include "UObject/ObjectMacros.h"
#include "GizmoElementTriangleList.generated.h"

/**
 * Simple object intended to be used as part of 3D Gizmos.
 * Draws a solid 3D sphere based on parameters.
 */

UCLASS(Transient, MinimalAPI)
class UGizmoElementTriangleList : public UGizmoElementBase
{
	GENERATED_BODY()

public:

	//~ Begin UGizmoElementBase Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI, const FRenderTraversalState& RenderState) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit LineTrace(const UGizmoViewContext* ViewContext, const FLineTraceTraversalState& LineTraceState, const FVector& RayOrigin, const FVector& RayDirection, FLineTraceOutput& OutLineTraceOutput) override;
	//~ End UGizmoElementBase Interface.

	// The vertices of the triangle list, each tuple of 3 forming a triangle
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetVertices(const TArrayView<const FVector>& InVertices);
	INTERACTIVETOOLSFRAMEWORK_API virtual const TArray<FVector>& GetVertices() const;

	// Base location the triangles
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetBase(FVector InBase);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetBase() const;

	// Up direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpDirection(const FVector& InUpDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetUpDirection() const;

	// Side direction
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetSideDirection(const FVector& InSideDirection);
	INTERACTIVETOOLSFRAMEWORK_API virtual FVector GetSideDirection() const;

protected:

	void ComputeProjectedVertices(const FTransform& InLocalToWorldTransform, bool bIncludeBase);

	static bool Intersect(const FRay& Ray, const FVector& A, const FVector& B, const FVector& C, double Tolerance, double& Distance);

	// The vertices of the triangle list, each tuple of 3 forming a triangle
	UPROPERTY()
	TArray<FVector> Vertices;

	// Base location the triangles
	UPROPERTY()
	FVector Base = FVector::ZeroVector;

	// Up direction
	UPROPERTY()
	FVector UpDirection = FVector(0.0f, 0.0f, 1.0f);

	// Side direction
	UPROPERTY()
	FVector SideDirection = FVector(0.0f, 1.0f, 0.0f);

	TArray<FVector> ProjectedVertices;
};

