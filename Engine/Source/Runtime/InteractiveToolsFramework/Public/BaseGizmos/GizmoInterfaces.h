// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputState.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "ToolContextInterfaces.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GizmoInterfaces.generated.h"

namespace UE::GizmoRenderingUtil
{
	class IViewBasedTransformAdjuster;
}

class IToolsContextRenderAPI;
class UObject;
struct FFrame;


//
// UInterfaces for the various UObjects used in the Standard Gizmo Library.
// 



UINTERFACE(MinimalAPI)
class UGizmoTransformSource : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoTransformSource is an interface which is used to Get/Set an FTransform.
 */
class IGizmoTransformSource
{
	GENERATED_BODY()
public:
	UFUNCTION()
	virtual FTransform GetTransform() const = 0;

	UFUNCTION()
	virtual void SetTransform(const FTransform& NewTransform) = 0;
};





UINTERFACE(MinimalAPI)
class UGizmoAxisSource : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoAxisSource is an interface which is used to get information about a 3D Axis.
 * At minimum this includes a 3D Direction Vector and Origin Point.
 * Optionally the implementation may provide two Tangent Vectors which are
 * assumed to be mutually-orthogonal and perpendicular to the Axis Direction
 * (ie that's the normal and the 3 vectors form a coordinate frame).
 */
class IGizmoAxisSource
{
	GENERATED_BODY()
public:
	/** @return Origin Point of axis */
	UFUNCTION()
	virtual FVector GetOrigin() const = 0;

	/** @return Direction Vector of axis */
	UFUNCTION()
	virtual FVector GetDirection() const = 0;

	/** @return true if this AxisSource has tangent vectors orthogonal to the Direction vector */
	UFUNCTION()
	virtual bool HasTangentVectors() const { return false; }

	/** Get the two tangent vectors that are orthogonal to the Direction vector. 
	 * @warning Only valid if HasTangentVectors() returns true
	 */
	UFUNCTION()
	virtual void GetTangentVectors(FVector& TangentXOut, FVector& TangentYOut) const { }


	/**
	 * Utility function that always returns a 3D coordinate system (ie plane normal and perpendicular axes).
	 * Internally calls GetTangentVectors() if available, otherwise constructs arbitrary mutually perpendicular vectors. 
	 */
	INTERACTIVETOOLSFRAMEWORK_API void GetAxisFrame(
		FVector& PlaneNormalOut, FVector& PlaneAxis1Out, FVector& PlaneAxis2Out) const;
};






UINTERFACE(MinimalAPI)
class UGizmoClickTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoClickTarget is an interface used to provide a ray-object hit test.
 */
class IGizmoClickTarget
{
	GENERATED_BODY()
public:
	/**
	 * @return FInputRayHit indicating whether or not the target object was hit by the device-ray at ClickPos
	 */
	//UFUNCTION()    // FInputDeviceRay is not USTRUCT because FRay is not USTRUCT
	virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const = 0;

	/*
	 * Updates the hover state indicating whether the input device is currently hovering over the Standard gizmo.
	 * This should be be set to false once interaction with the gizmo commences.
	 */
	UFUNCTION()
	virtual void UpdateHoverState(bool bHovering) = 0;

	/*
	 * Updates the interacting state indicating when interaction with the Standard gizmo is actively occurring, 
	 * typically upon the input device clicking and dragging the Standard gizmo.
	 */
	UFUNCTION()
	virtual void UpdateInteractingState(bool bInteracting) = 0;

	/*
	 * Updates the selected state, indicating when the gizmo is currently selected,
	 * typically upon the input device clicking the Standard gizmo. 
	 */
	UFUNCTION()
	virtual void UpdateSelectedState(bool bSelected) = 0;

	/*
	 * Updates the subdued state, indicating when the element should have a subdued/background appearance, 
	 * typically upon interacting with one of a set of gizmos representing 3 axis - the two not interacted with would be subdued. 
	 */
	UFUNCTION()
	virtual void UpdateSubdueState(bool bSubdued) = 0;

	/** Sets the optional transform adjuster to apply to the tested geometry. */
	virtual void SetTransformAdjuster(const TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster>& InAdjuster) { /** Empty default implementation. */ }

	/** Get the (optional) transform adjuster applied to this Target. */
	virtual TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> GetTransformAdjuster() const { return nullptr; /** No adjuster by default. */ }
};

UINTERFACE(MinimalAPI)
class UGizmoClickMultiTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoClickMultiTarget is an interface used to provide a ray-object hit test against a target which
 * supports hitting parts of the target.
 *
 * For a gizmo with multiple parts, the part identifier establishes a correspondence between a gizmo part 
 * and the elements representing that part within the hit target. The valid part identifiers should 
 * be defined in the gizmo. Identifier 0 is reserved for the default ID which should be assigned to 
 * elements that do not correspond to any gizmo part, such as non-hittable decorative elements.
 */
class IGizmoClickMultiTarget
{
	GENERATED_BODY()
public:
	/**
	 * @return FInputRayHit indicating whether or not the target object was hit by the device-ray at ClickPos
	 *         The ray hit contains client-defined ID, HitOwner and HitObject which are used to identify the hit part.
	 */
	 //UFUNCTION()    // FInputDeviceRay is not USTRUCT because FRay is not USTRUCT
	virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const = 0;

	/*
	 * Updates the hover state of the specified gizmo part, indicating whether the input device is currently hovering 
	 * over the Standard gizmo.
	 */
	UFUNCTION()
	virtual void UpdateHoverState(bool bHovering, uint32 InPartIdentifier) = 0;

	/*
	 * Updates the interacting state of the specified gizmo part, indicating when interaction with the 
	 * Standard gizmo is actively occurring, typically upon the input device clicking and dragging the Standard gizmo. 
	 */
	UFUNCTION()
	virtual void UpdateInteractingState(bool bInteracting, uint32 InPartIdentifier) = 0;

	/*
	 * Updates the selected state of the specified gizmo part, indicating when the gizmo part is currently selected,
	 * typically upon the input device clicking the Standard gizmo.
	 */
	UFUNCTION()
	virtual void UpdateSelectedState(bool bSelected, uint32 InPartIdentifier) = 0;

	/*
	 * Updates the subdued state of the specified gizmo part, indicating when the element should have a subdued/background appearance, 
	 * typically upon interacting with one of a set of gizmos representing 3 axis - the two not interacted with would be subdued.
	 */
	UFUNCTION()
	virtual void UpdateSubdueState(bool bSubdued, uint32 InPartIdentifier) = 0;

	/*
	 * Updates the hittable state of the specified gizmo part. 
	 */
	UFUNCTION()
	virtual void UpdateHittableState(bool bHittable, uint32 InPartIdentifier) = 0;

	/** Sets the optional transform adjuster to apply to the tested geometry. */
	virtual void SetTransformAdjuster(const TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster>& InAdjuster) { /** Empty default implementation. */ }

	/** Get the (optional) transform adjuster applied to this Target. */
	virtual TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> GetTransformAdjuster() const { return nullptr; /** No adjuster by default. */ }
};




UINTERFACE(MinimalAPI)
class UGizmoRenderTarget : public UInterface
{
	GENERATED_BODY()
};

/**
 * UGizmoRenderTarget is an interface used to provide rendering of a target
 */
class IGizmoRenderTarget
{
	GENERATED_BODY()
public:
	/**
	 * Renders the target using the current tools context.
	 */
	virtual void Render(IToolsContextRenderAPI* RenderAPI) const = 0;
};




UINTERFACE(MinimalAPI)
class UGizmoRenderMultiTarget : public UInterface
{
	GENERATED_BODY()
};

/**
 * IGizmoRenderMultiTarget is an interface used to provide rendering of a target and the 
 * ability to specify which part of a target should be visible.
 * 
 * For a gizmo with multiple parts, the part identifier establishes a correspondence between a gizmo part 
 * and the elements representing that part within the hit target. The valid part identifiers should 
 * be defined in the gizmo. Identifier 0 is reserved for the default ID which should be assigned to 
 * elements that do not correspond to any gizmo part, such as non-hittable decorative elements.
 */
class IGizmoRenderMultiTarget
{
	GENERATED_BODY()
public:
	/**
	 * Renders the target using the current tools context.
	 */
	virtual void Render(IToolsContextRenderAPI* RenderAPI) const = 0;

	/*
	 * Updates the visibility state of the specified gizmo part.
	 */
	UFUNCTION()
	virtual void UpdateVisibilityState(bool bVisible, uint32 InPartIdentifier) = 0;
};


UINTERFACE(MinimalAPI)
class UGizmoStateTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoStateTarget is an interface that is used to pass notifications about significant gizmo state updates
 */
class IGizmoStateTarget
{
	GENERATED_BODY()
public:
	/**
	 * BeginUpdate is called before a standard Gizmo begins changing a parameter (via a ParameterSource)
	 */
	UFUNCTION()
	virtual void BeginUpdate() = 0;

	/**
	 * EndUpdate is called when a standard Gizmo is finished changing a parameter (via a ParameterSource)
	 */
	UFUNCTION()
	virtual void EndUpdate() = 0;

	/**
	 * CancelUpdate is called when a gizmo cancels and reverts the changes to a parameter (via a ParameterSource)
	 */
	UFUNCTION()
	virtual void CancelUpdate() = 0;
};





UINTERFACE(MinimalAPI)
class UGizmoFloatParameterSource : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoFloatParameterSource provides Get and Set for an arbitrary float-valued parameter.
 */
class IGizmoFloatParameterSource
{
	GENERATED_BODY()
public:

	/** @return value of parameter */
	UFUNCTION()
	virtual float GetParameter() const = 0;

	/** notify ParameterSource that a parameter modification is about to begin */
	UFUNCTION()
	virtual void BeginModify() = 0;

	/** set value of parameter */
	UFUNCTION()
	virtual void SetParameter(float NewValue) = 0;

	/** notify ParameterSource that a parameter modification is complete */
	UFUNCTION()
	virtual void EndModify() = 0;
};




UINTERFACE(MinimalAPI)
class UGizmoVec2ParameterSource : public UInterface
{
	GENERATED_BODY()
};
/**
 * IGizmoVec2ParameterSource provides Get and Set for an arbitrary 2D-vector-valued parameter.
 */
class IGizmoVec2ParameterSource
{
	GENERATED_BODY()
public:

	/** @return value of parameter */
	UFUNCTION()
	virtual FVector2D GetParameter() const = 0;

	/** notify ParameterSource that a parameter modification is about to begin */
	UFUNCTION()
	virtual void BeginModify() = 0;

	/** set value of parameter */
	UFUNCTION()
	virtual void SetParameter(const FVector2D& NewValue) = 0;

	/** notify ParameterSource that a parameter modification is complete */
	UFUNCTION()
	virtual void EndModify() = 0;
};
