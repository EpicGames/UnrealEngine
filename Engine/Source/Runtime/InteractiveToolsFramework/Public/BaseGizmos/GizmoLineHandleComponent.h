// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "GizmoLineHandleComponent.generated.h"


/**
 * Simple Component intended to be used as part of 3D Gizmos.
 * Draws line terminated by a short perpendicular handle line based on parameters.
 */
UCLASS(ClassGroup = Utility, HideCategories = (Physics, Collision, Lighting, Rendering, Mobile), MinimalAPI)
class UGizmoLineHandleComponent : public UGizmoBaseComponent
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	FVector Normal = FVector(0, 0, 1);

	UPROPERTY(EditAnywhere, Category = Options)
	float HandleSize = 10.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	float Thickness = 2.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	FVector Direction = FVector(0, 0, 1.);

	UPROPERTY(EditAnywhere, Category = Options)
	float Length;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bImageScale = true;
private:
	//~ Begin UPrimitiveComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.
};
