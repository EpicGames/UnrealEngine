// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoInterfaces.h"
#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"

/**
 * GizmoMath functions implement various vector math/geometry operations
 * required to implement various standard Gizmos. Although some of these
 * are standard queries, in the context of user interface Gizmos they may
 * have behaviors (like assumptions about valid ranges or error-handling)
 * that would not be suitable for general-purpose math code.
 */
namespace GizmoMath
{
	/**
	 * Project a point onto the line defined by LineOrigin and LineDirection
	 * @return projected point
	 */
	INTERACTIVETOOLSFRAMEWORK_API FVector ProjectPointOntoLine(
		const FVector& Point,
		const FVector& LineOrigin, const FVector& LineDirection);

	/**
	 * Find the nearest point to QueryPoint on the line defined by LineOrigin and LineDirection.
	 * Returns this point and the associated line-equation parameter
	 */
	INTERACTIVETOOLSFRAMEWORK_API void NearestPointOnLine(
		const FVector& LineOrigin, const FVector& LineDirection,
		const FVector& QueryPoint,
		FVector& NearestPointOut, float& LineParameterOut);

	/**
	 * Find the closest pair of points on a 3D line and 3D ray.
	 * Returns the points and line-equation parameters on both the line and ray.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void NearestPointOnLineToRay(
		const FVector& LineOrigin, const FVector& LineDirection,
		const FVector& RayOrigin, const FVector& RayDirection,
		FVector& NearestLinePointOut, float& LineParameterOut,
		FVector& NearestRayPointOut, float& RayParameterOut);

	/**
	 * Find the intersection of the ray defined by RayOrigin and RayDirection
	 * with the plane defined by PlaneOrigin and PlaneNormal.
	 * Returns intersection success/failure in bIntersectsOut and the intersection point in PlaneIntersectionPointOut
	 */
	INTERACTIVETOOLSFRAMEWORK_API void RayPlaneIntersectionPoint(
		const FVector& PlaneOrigin, const FVector& PlaneNormal,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, FVector& PlaneIntersectionPointOut);

	/**
	 * Find the intersection of the ray defined by RayOrigin and RayDirection
	 * with the sphere defined by SphereOrigin and SphereRadius.
	 * Returns intersection success/failure in bIntersectsOut and the intersection point in SphereIntersectionPointOut
	 */
	INTERACTIVETOOLSFRAMEWORK_API void RaySphereIntersection(
		const FVector& SphereOrigin, const float SphereRadius,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, FVector& SphereIntersectionPointOut);

	/**
	 * Find the intersection of the ray defined by RayOrigin and RayDirection
	 * with the cylinder defined by CylinderCenter, CylinderAxis, CylinderRadius, and CylinderHeight.
	 * Returns intersection success/failure in bOutIntersects and the hit depth parameter 
	 * along the ray for the first intersection in OutHitDepth.
	 */
	template<typename RealType>
	void INTERACTIVETOOLSFRAMEWORK_API RayCylinderIntersection(
		const FVector& CylinderCenter, const FVector& CylinderAxis, RealType CylinderRadius, RealType CylinderHeight,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, RealType& OutHitDepth);


	/**
	 * Find the intersection of the ray defined by RayOrigin and RayDirection
	 * with the cone defined by ConeCenter, ConeDirection, ConeCosAngle, and ConeHeight.
	 * Where ConeCosAngle is the cosine of the cone opening angle, defined as the angle
	 * at the apex between the cone's axis and slant edge.
	 * Returns intersection success/failure in bOutIntersects and the hit depth parameter
	 * along the ray for the first intersection in OutHitDepth.
	 */
	template <typename RealType>
	void INTERACTIVETOOLSFRAMEWORK_API RayConeIntersection(
		const FVector& ConeCenter, const FVector& ConeDirection, RealType ConeCosAngle, RealType ConeHeight,
		const FVector& RayOrigin, const FVector& RayDirection,
		bool& bIntersectsOut, RealType& OutHitDepth);

	/**
	 * Find the intersection of two floating point intervals in which
	 * the interval inputs contain pairs of floats sorted in increasing order.
	 * Returns the number of intersections intersection and intersection 
	 * results in increasing order. 
	 */
	template <typename RealType>
	void INTERACTIVETOOLSFRAMEWORK_API IntervalIntervalIntersection(
		const RealType Interval0[2], const RealType Interval1[2],
		int& OutNumIntersections, RealType& OutResult0, RealType& OutResult1);

	/**
	 * Find the closest point to QueryPoint that is on the circle
	 * defined by CircleOrigin, CircleNormal, and CircleRadius.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void ClosetPointOnCircle(
		const FVector& QueryPoint, 
		const FVector& CircleOrigin, const FVector& CircleNormal, float CircleRadius,
		FVector& ClosestPointOut);

	/**
	 * Construct two mutually-perpendicular unit vectors BasisAxis1Out and BasisAxis2Out
	 * which are orthogonal to PlaneNormal
	 */
	INTERACTIVETOOLSFRAMEWORK_API void MakeNormalPlaneBasis(
		const FVector& PlaneNormal,
		FVector& BasisAxis1Out, FVector& BasisAxis2Out);

	/**
	 * Project the input Point into the plane defined by PlaneOrigin and PlaneNormal,
	 * and then measure it's signed angle relative to the in-plane X and Y axes
	 * defined by PlaneAxis1 and PlaneAxis2
	 * @return the signed angle
	 */
	INTERACTIVETOOLSFRAMEWORK_API float ComputeAngleInPlane(
		const FVector& Point,
		const FVector& PlaneOrigin, const FVector& PlaneNormal,
		const FVector& PlaneAxis1, const FVector& PlaneAxis2);


	/**
	 * Project the input Point into the plane defined by PlaneOrigin and PlaneNormal,
	 * and then calculate it's UV coordinates in the space defined by the plane 
	 * axes PlaneAxis1 and PlaneAxis2
	 * @return the UV coordinates
	 */
	INTERACTIVETOOLSFRAMEWORK_API FVector2D ComputeCoordinatesInPlane(
		const FVector& Point,
		const FVector& PlaneOrigin, const FVector& PlaneNormal,
		const FVector& PlaneAxis1, const FVector& PlaneAxis2);

	/**
	 * Project the input Point onto the plane defined by PlaneOrigin and PlaneNormal,
	 * @return Nearest position to Point lying in the plane
	 */
	INTERACTIVETOOLSFRAMEWORK_API FVector ProjectPointOntoPlane(
		const FVector& Point,
		const FVector& PlaneOrigin, const FVector& PlaneNormal);


	/**
	 * Round Value to nearest step of Increment
	 * @return snapped/rounded value
	 */
	template <typename RealType>
	INTERACTIVETOOLSFRAMEWORK_API RealType SnapToIncrement(RealType Value, RealType Increment);

	/**
	 * @todo change FVector to a templated vector type specialized by RealType (double or float)
	 * Compute vector orthogonal to V.
	 * @return Orthogonal vector
	 */
	INTERACTIVETOOLSFRAMEWORK_API FVector GetOrthogonalVector(const FVector& V);

	/**
	 * Helper struct used to pass parameters to the BuildDashedLine method.
	 * Also defines unique behavior for building dashes, which differs between world and screen-space.
	 */
	struct FDashLineSettings
	{
		/** Total length of the line. */
		double TotalLength = 0.0;

		/** Scaled Line length, e.g. matching pixel to world scale */
		double ScaledLength;

		/** Converts the given parameter along the length (0-1) to a world fraction. */
		TFunction<double(const double& InParam)> ParamToWorldFunction;

		/** Returns the radius scale at the given param (0-1) */
		TFunction<double(const double& InParam)> ParamToRadiusScaleFunction;

		/** Dashes length */
		float DashLength = 10.0f;

		/** Length of gaps between dashes */
		float DashGapLength = 5.0f;

		/** Start point of the line */
		FVector ScreenStartPoint;

		/** End point of the line */
		FVector ScreenEndPoint;

		/** Resolution of the cylinder used to render the line - 3 or more vertices required */
		int32 NumSides = 32;

		/** Arbitrary maximum to prevent overflow in the case of very long cylinders */
		int32 MaxNumDashes = 512;

		/** Reflects the fraction of the line (in world space) that's visible - e.g. 0.5 means half of the line is visible */
		double LineFractionInView = 1.0;

		/** World-space fraction offset - non-zero only when the line origin is clamped */
		double WorldStartFraction = 0.0;

		/** Radius of the line */
		float Radius = 1.0f;

		/** Color of the line */
		FColor Color;

		/** The local to world matrix which will be applied to the line geometry vertices */
		FMatrix RenderLocalToWorldMatrix;

		/** The material interface used by the line */
		const UMaterialInterface* MaterialInterface;
	};

	/**
	 * Get the screen-space start and end for the given line (from origin, direction, length). Z is depth from view, XY
	 * are in pixel space. OutClampedWorldFraction is used when one or more of the end points is beyond the view bounds,
	 * and reflects the fraction of the line (in world space) that's visible, ie. if only 50% of the line is within the
	 * viewport, this will return 0.5.
	 * OutClampedWorldStartFraction reflects the world-space offset from the line origin to the visible segment start -
	 * non-zero when the origin end of the line is clipped (e.g. camera-follow mode where the origin leaves the viewport).
	 */
	INTERACTIVETOOLSFRAMEWORK_API void GetScreenSpaceLineStartEnd(
		const FSceneView* InSceneView,
		const FVector& InLineOrigin,
		const FVector& InLineDirection,
		const double& InLineLength,
		FVector& OutScreenStartPoint,
		FVector& OutScreenEndPoint,
		double& OutClampedWorldFraction,
		double& OutClampedWorldStartFraction
	);

	INTERACTIVETOOLSFRAMEWORK_API double GetPixelToWorldScaleAtScreenFraction(
		const double& InStartDepthInView, const double& InEndDepthInView, double InScreenFraction
	);

	INTERACTIVETOOLSFRAMEWORK_API void BuildDashedLine(FPrimitiveDrawInterface* InPDI, const FDashLineSettings& InDashSettings);
}
