// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "GizmoRectangleComponent.generated.h"

/**
 * Simple Component intended to be used as part of 3D Gizmos. 
 * Draws outline of 3D rectangle based on parameters.
 */
UCLASS(ClassGroup = Utility, HideCategories = (Physics, Collision, Lighting, Rendering, Mobile), MinimalAPI)
class UGizmoRectangleComponent : public UGizmoBaseComponent
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	FVector DirectionX = FVector(0, 0, 1);

	UPROPERTY(EditAnywhere, Category = Options)
	FVector DirectionY = FVector(0, 1, 0);

	// When true, instead of using the provided DirectionY, the component will
	// use a direction orthogonal to the camera direction and DirectionX. This
	// keeps the rectangle pinned along DirectionX but spun to be flatter
	// relative the camera.
	UPROPERTY(EditAnywhere, Category = Options)
	bool bOrientYAccordingToCamera = false;

	UPROPERTY(EditAnywhere, Category = Options)
	float OffsetX = 0.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	float OffsetY = 0.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	float LengthX = 20.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	float LengthY = 20.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	float Thickness = 2.0f;

	UPROPERTY(EditAnywhere, Category = Options)
	uint8 SegmentFlags = 0x1 | 0x2 | 0x4 | 0x8;

private:
	//~ Begin UPrimitiveComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.
};
