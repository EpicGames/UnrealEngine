// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "SceneView.h"

class IToolsContextRenderAPI;
class UGizmoViewContext;
class UInteractiveGizmo;
class UInteractiveGizmoManager;
class UInteractiveToolManager;
class UViewAdjustedStaticMeshGizmoComponent;

/**
 * Utility functions for standard GizmoComponent rendering
 */

namespace UE::GizmoRenderingUtil
{
	// Interface meant to wrap either an FSceneView or UGizmoViewContext so that a user can write one
	// function to handle either one (for rendering and for hit testing).
	class ISceneViewInterface
	{
	public:
		virtual const FIntRect& GetUnscaledViewRect() const = 0;
		virtual FVector4 WorldToScreen(const FVector&) const = 0;
		virtual FVector GetViewLocation() const = 0;
		virtual FVector GetViewDirection() const = 0;
		virtual FVector GetViewRight() const = 0;
		virtual FVector GetViewUp() const = 0;
		virtual const FMatrix& GetProjectionMatrix() const = 0;
		virtual const FMatrix& GetViewMatrix() const = 0;
		virtual bool IsPerspectiveProjection() const = 0;
		virtual double GetDPIScale() const = 0;
	};

	// Wrapper around FSceneView to access it through ISceneViewInterface
	class FSceneViewWrapper : public ISceneViewInterface
	{
	public:
		explicit FSceneViewWrapper(const FSceneView& SceneView)
		{
			View = &SceneView;
		}

		virtual const FIntRect& GetUnscaledViewRect() const { return View->UnscaledViewRect; }
		virtual FVector4 WorldToScreen(const FVector& VectorIn) const { return View->WorldToScreen(VectorIn); }
		virtual FVector GetViewLocation() const { return View->ViewLocation; }
		virtual FVector GetViewDirection() const { return View->GetViewDirection(); }
		virtual FVector GetViewRight() const { return View->GetViewRight(); }
		virtual FVector GetViewUp() const { return View->GetViewUp(); }
		virtual const FMatrix& GetProjectionMatrix() const { return View->ViewMatrices.GetViewToClip(); }
		virtual const FMatrix& GetViewMatrix() const { return View->ViewMatrices.GetWorldToView(); }
		virtual bool IsPerspectiveProjection() const { return View->IsPerspectiveProjection(); }
		virtual double GetDPIScale() const { return 1.0; }

	private:
		const FSceneView* View;
	};


	// Can be used as TranslucencySortPriority to make gizmo elements show up above other translucent objects
	const int32 GIZMO_TRANSLUCENCY_SORT_PRIORITY = 5000;

	/**
	 * Gets a custom material suitable to use for gizmo components. The material is drawn on top of opaque geometry with
	 *  dithering for portions behind opaque materials, but uses the custom depth buffer to properly occlude itself.
	 * Components using this material should set bRenderCustomDepth to true so they can occlude other gizmo elements. It
	 *  is also suggested that TranslucencySortPriority be set to something like UE::GizmoRenderingUtil::GIZMO_TRANSLUCENCY_SORT_PRIORITY
	 *  so that the component is drawn on top of other translucent materials.
	 * 
	 * @param Outer Object to set as outer for the material instance. Typically can be left as nullptr to use transient package.
	 */
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetDefaultGizmoComponentMaterial(const FLinearColor& Color, UObject* Outer = nullptr);

	struct FDefaultGizmoMaterialExtraParams 
	{
		bool bDimOccluded = true;
	};
	/** 
	 * Like the other overload, but with additional parameters.
	 */
	UMaterialInterface* GetDefaultGizmoComponentMaterial(const FLinearColor& Color, const FDefaultGizmoMaterialExtraParams& Params, UObject* Outer = nullptr);

	/**
	 * Helper that creates a component with the default gizmo material and sets up the component-side properties that are needed for
	 *  it to properly work (translucency sort priority, etc). This may not be necessary if your component is using some other
	 *  gizmo material that doesn't require component-side flags.
	 */
	INTERACTIVETOOLSFRAMEWORK_API UViewAdjustedStaticMeshGizmoComponent* CreateDefaultMaterialGizmoMeshComponent(
		UStaticMesh* Mesh, UGizmoViewContext* GizmoViewContext, UObject* OwnerComponentOrActor, 
		const FLinearColor& Color, bool bAddHoverMaterial = true);

	/**
	 * Overload that takes a gizmo manager to get the view context from that.
	 */
	INTERACTIVETOOLSFRAMEWORK_API UViewAdjustedStaticMeshGizmoComponent* CreateDefaultMaterialGizmoMeshComponent(
		UStaticMesh* Mesh, UInteractiveGizmoManager* GizmoManager, UObject* OwnerComponentOrActor, 
		const FLinearColor& Color, bool bAddHoverMaterial = true);

	/**
	 * Gets a red/green/blue color based on the axis (X, Y, or Z).
	 */
	INTERACTIVETOOLSFRAMEWORK_API FLinearColor GetDefaultAxisColor(EAxis::Type Axis);

	/**
	 * @return Conversion factor between pixel and world-space coordinates at 3D point Location in View.
	 * @warning This is a local estimate and is increasingly incorrect as the 3D point gets further from Location
	 */
	INTERACTIVETOOLSFRAMEWORK_API float CalculateLocalPixelToWorldScale(
		const FSceneView* View,
		const FVector& Location,
		const double InDPIScale = 1.0);

	/** @note: if bInWithDPIScale is true, we use ISceneViewInterface::DPIScale or the provided InDPIScale when set. */
	template <typename RealType = float>
	INTERACTIVETOOLSFRAMEWORK_API RealType CalculateLocalPixelToWorldScale(
		const UE::GizmoRenderingUtil::ISceneViewInterface* View,
		const FVector& Location,
		const bool bInWithDPIScale = false,
		const TOptional<double>& InDPIScale = TOptional<double>());

	extern template float CalculateLocalPixelToWorldScale<float>(const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, bool, const TOptional<double>&);

	extern template double CalculateLocalPixelToWorldScale<double>(const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, bool, const TOptional<double>&);

	/**
	 * @return Legacy view dependent conversion factor.
	 * @return OutWorldFlattenScale vector to be applied in world space, can be used to flatten excluded 
	 *         dimension in orthographic views as it reverses the scale in that dimension.
	 */
	template <typename RealType = float>
	INTERACTIVETOOLSFRAMEWORK_API RealType CalculateViewDependentScaleAndFlatten(
		const UE::GizmoRenderingUtil::ISceneViewInterface* View,
		const FVector& Location,
		const RealType Scale,
		FVector& OutWorldFlattenScale);

	extern template float CalculateViewDependentScaleAndFlatten<float>(const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, float, FVector&);

	extern template double CalculateViewDependentScaleAndFlatten<double>(const UE::GizmoRenderingUtil::ISceneViewInterface*, const FVector&, double, FVector&);

	INTERACTIVETOOLSFRAMEWORK_API float CalculateViewDependentScaleAndFlatten(
		const FSceneView* View,
		const FVector& Location,
		const float Scale,
		FVector& OutWorldFlattenScale);
}

// This namespace is deprecated- use UE::GizmoRenderingUtil instead.
namespace GizmoRenderingUtil
{
	UE_DEPRECATED(5.5, "This function was moved to the UE::GizmoRenderingUtil namespace.")
	INTERACTIVETOOLSFRAMEWORK_API float CalculateLocalPixelToWorldScale(
		const FSceneView* View,
		const FVector& Location);
	
	UE_DEPRECATED(5.5, "This function was moved to the UE::GizmoRenderingUtil namespace.")
	INTERACTIVETOOLSFRAMEWORK_API float CalculateLocalPixelToWorldScale(
		const UGizmoViewContext* ViewContext,
		const FVector& Location);

	UE_DEPRECATED(5.5, "This function was moved to the UE::GizmoRenderingUtil namespace.")
	INTERACTIVETOOLSFRAMEWORK_API float CalculateViewDependentScaleAndFlatten(
		const FSceneView* View,
		const FVector& Location,
		const float Scale,
		FVector& OutWorldFlattenScale);
}
