// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GizmoElementBase.h"
#include "InteractiveGizmoBuilder.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"

#include "GizmoUtil.generated.h"

class FString;
class UGizmoElementCircleBase;
class UInteractiveGizmo;
class UInteractiveGizmoManager;

/** A utility class to access GizmoElements that aren't otherwise publicly exposed. Use responsibly. */
class FGizmoElementAccessor
{
public:
	INTERACTIVETOOLSFRAMEWORK_API static TConstArrayView<TObjectPtr<UGizmoElementBase>> GetSubElements(const UGizmoElementBase& InGizmoElement);
	INTERACTIVETOOLSFRAMEWORK_API static void GetSubElementsRecursive(const UGizmoElementBase& InGizmoElement, TArray<TObjectPtr<UGizmoElementBase>>& OutElements);
	INTERACTIVETOOLSFRAMEWORK_API static bool GetSubElementsRecursive(const UGizmoElementBase& InGizmoElement, TArray<TObjectPtr<UGizmoElementBase>>& OutElements, const uint32 InPartId);

	/** Updates the given render state from the given GizmoElement. */
	INTERACTIVETOOLSFRAMEWORK_API bool UpdateRenderState(UGizmoElementBase& InGizmoElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState) const;

	/** Updates the given render state from the given GizmoElement. */
	INTERACTIVETOOLSFRAMEWORK_API bool UpdateRenderState(UGizmoElementBase& InGizmoElement, IToolsContextRenderAPI* InRenderAPI, const FVector& InLocalCenter, UGizmoElementBase::FRenderTraversalState& InOutRenderState, bool& bOutHasAlignRot, FQuat& OutAlignRot) const;

	/** Returns whether element is enabled for given interaction state. If the provided interaction state isn't set, it will use the currently applied state to the given element. */
	INTERACTIVETOOLSFRAMEWORK_API static bool IsEnabledForInteractionState(const UGizmoElementBase& InGizmoElement, const TOptional<EGizmoElementInteractionState>& InInteractionState = {});

	INTERACTIVETOOLSFRAMEWORK_API static bool IsPartial(
		UGizmoElementCircleBase& InGizmoElement,
		const FVector& InWorldCenter,
		const FVector& InWorldNormal,
		const FVector& InViewLocation,
		const FVector& InViewDirection,
		const bool bIsPerspectiveProjection);
};

/**
 * Gizmo builder that simply calls a particular lambda when building a gizmo. Makes it easy to
 *  register gizmo build behavior without writing a new class.
 */
UCLASS(MinimalAPI)
class USimpleLambdaInteractiveGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:

	virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override
	{
		if (BuilderFunc)
		{
			return BuilderFunc(SceneState);
		}
		return nullptr;
	}

	TUniqueFunction<UInteractiveGizmo* (const FToolBuilderState& SceneState)> BuilderFunc;
};

namespace UE::GizmoUtil
{
	/**
	 * Uses the gizmo manager to create a gizmo of the given class (assuming that the gizmo type does not need
	 *  any special setup beyond instantiation) without having to register a custom builder for that class ahead of time.
	 * 
	 * This function lets the user bypass the need to define, register, and use a builder class, while still registering
	 *  the gizmo properly with the gizmo manager. Under the hood, it creates and registers a temporary generic builder, 
	 *  uses it to make the gizmo, and then immediately deregisters the builder. 
	 */
	INTERACTIVETOOLSFRAMEWORK_API UInteractiveGizmo* CreateGizmoViaSimpleBuilder(UInteractiveGizmoManager* GizmoManager,
		TSubclassOf<UInteractiveGizmo> GizmoClass, const FString& InstanceIdentifier, void* Owner);

	/**
	 * Template version of CreateGizmoViaSimpleBuilder that does a cast on return.
	 */
	template <typename GizmoClass>
	GizmoClass* CreateGizmoViaSimpleBuilder(UInteractiveGizmoManager* GizmoManager,
		const FString& InstanceIdentifier, void* GizmoOwner)
	{
		return Cast<GizmoClass>(CreateGizmoViaSimpleBuilder(GizmoManager, GizmoClass::StaticClass(), InstanceIdentifier, GizmoOwner));
	}
}