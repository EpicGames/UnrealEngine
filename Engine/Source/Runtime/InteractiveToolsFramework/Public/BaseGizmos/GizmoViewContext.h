// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GizmoRenderingUtil.h"
#include "SceneView.h"

#include "GizmoViewContext.generated.h"

/** 
 * A context object that is meant to hold the scene information for the hovered viewport
 * on a game thread, to be used by a gizmo later for hit testing. The organization mirrors
 * FSceneView so that functions could be written in a templated way to use either FSceneView
 * or UGizmoViewContext, though UGizmoViewContext only keeps the needed data.
 */
UCLASS(MinimalAPI)
class UGizmoViewContext : public UObject, public UE::GizmoRenderingUtil::ISceneViewInterface
{
	GENERATED_BODY()
public:
	
	// Wrapping class for the matrices so that they can be accessed in the same way
	// that they are accessed in FSceneView.
	class FMatrices
	{
	public:
		void ResetFromViewMatrices(const FViewMatrices& InViewMatrices)
		{
			ProjectionMatrix = InViewMatrices.GetViewToClip();
			InvProjectionMatrix = InViewMatrices.GetClipToView();
			ViewMatrix = InViewMatrices.GetWorldToView();
			InvViewMatrix = InViewMatrices.GetViewToWorld();
			ViewProjectionMatrix = InViewMatrices.GetWorldToClip();
			InvViewProjectionMatrix = InViewMatrices.GetClipToWorld();
		}

		void ResetFromSceneView(const FSceneView& SceneView)
		{
			ResetFromViewMatrices(SceneView.ViewMatrices);
		}

		const FMatrix& GetProjectionMatrix() const { return ProjectionMatrix; }
		const FMatrix& GetInvProjectionMatrix() const { return InvProjectionMatrix; }
		const FMatrix& GetViewMatrix() const { return ViewMatrix; }
		const FMatrix& GetInvViewMatrix() const { return InvViewMatrix; }
		const FMatrix& GetViewProjectionMatrix() const { return ViewProjectionMatrix; }
		const FMatrix& GetInvViewProjectionMatrix() const { return InvViewProjectionMatrix; }

	protected:
		FMatrix ProjectionMatrix = FMatrix::Identity;
		FMatrix InvProjectionMatrix = FMatrix::Identity;
		FMatrix ViewMatrix = FMatrix::Identity;
		FMatrix InvViewMatrix = FMatrix::Identity;
		FMatrix ViewProjectionMatrix = FMatrix::Identity;
		FMatrix InvViewProjectionMatrix = FMatrix::Identity;
	};

	void Reset(const FIntRect& InUnscaledViewRect, const FViewMatrices& InViewMatrices, const bool bInIsPerspectiveProjection, const FVector& InViewLocation)
	{
		UnscaledViewRect = InUnscaledViewRect;
		ViewMatrices.ResetFromViewMatrices(InViewMatrices);
		bIsPerspectiveProjection = bInIsPerspectiveProjection;
		ViewLocation = InViewLocation;
	}

	// Use this to reinitialize the object each frame for the hovered viewport.
	void ResetFromSceneView(const FSceneView& SceneView)
	{
		Reset(SceneView.UnscaledViewRect, SceneView.ViewMatrices, SceneView.IsPerspectiveProjection(), SceneView.ViewLocation);
	}

	// ISceneViewInterface
	const FIntRect& GetUnscaledViewRect() const override { return UnscaledViewRect; }
	FVector GetViewLocation() const override { return ViewLocation; }
	FVector GetViewRight() const override { return ViewMatrices.GetViewMatrix().GetColumn(0); }
	FVector GetViewUp() const override { return ViewMatrices.GetViewMatrix().GetColumn(1); }
	FVector GetViewDirection() const override { return ViewMatrices.GetViewMatrix().GetColumn(2); }
	const FMatrix& GetProjectionMatrix() const override { return ViewMatrices.GetProjectionMatrix(); }
	const FMatrix& GetViewMatrix() const override { return ViewMatrices.GetViewMatrix(); }
	bool IsPerspectiveProjection() const override { return bIsPerspectiveProjection; }
	virtual double GetDPIScale() const override { return DPIScale; }
	void SetDPIScale(const double InDPIScale) { DPIScale = InDPIScale; }

	FVector4 WorldToScreen(const FVector& WorldPoint) const override
	{
		return ViewMatrices.GetViewProjectionMatrix().TransformFVector4(FVector4(WorldPoint, 1));
	}

	// WorldToScreen -> ScreenToPixel. A result of (-1, -1) indicates invalid W.
	FVector2D WorldToPixel(const FVector& WorldPoint) const
	{
		FVector2D ScreenPos;
		if (ScreenToPixel(WorldToScreen(WorldPoint), ScreenPos))
		{
			return ScreenPos;
		}

		return -FVector2D::One();
	}

	bool ScreenToPixel(const FVector4& ScreenPoint, FVector2D& OutPixelLocation) const
	{
		// implementation copied from FSceneView::ScreenToPixel() 

		if (ScreenPoint.W != 0.0f)
		{
			//Reverse the W in the case it is negative, this allow to manipulate a manipulator in the same direction when the camera is really close to the manipulator.
			double InvW = (ScreenPoint.W > 0.0 ? 1.0 : -1.0) / ScreenPoint.W;
			double Y = (GProjectionSignY > 0.0) ? ScreenPoint.Y : 1.0 - ScreenPoint.Y;
			OutPixelLocation = FVector2D(
				UnscaledViewRect.Min.X + (0.5 + ScreenPoint.X * 0.5 * InvW) * UnscaledViewRect.Width(),
				UnscaledViewRect.Min.Y + (0.5 - Y * 0.5 * InvW) * UnscaledViewRect.Height()
			);
			return true;
		}
		else
		{
			return false;
		}
	}
	
	FRay ScreenToRay(const FVector4& ScreenPosition) const
	{
		// Derived from FViewportCursorLocation::FViewportCursorLocation()
		
		const FMatrix& InvViewMatrix = ViewMatrices.GetInvViewMatrix();
		const FMatrix& InvProjectionMatrix = ViewMatrices.GetInvProjectionMatrix();
	
		if (IsPerspectiveProjection())
		{
			return FRay(
				GetViewLocation(),
				InvViewMatrix.TransformVector(FVector(InvProjectionMatrix.TransformFVector4(FVector4(
					ScreenPosition.X * GNearClippingPlane,
					ScreenPosition.Y * GNearClippingPlane,
					0.0f,
					GNearClippingPlane
				)))).GetSafeNormal(),
				true
			);
		}
		else
		{
			FVector Direction = InvViewMatrix.TransformVector(FVector4(0, 0, 1)).GetSafeNormal(); 
			return FRay(
				InvViewMatrix.TransformFVector4(InvProjectionMatrix.TransformFVector4(FVector4(
					ScreenPosition.X,
					ScreenPosition.Y,
					0.5,
					1.0
				))) - UE_FLOAT_HUGE_DISTANCE * Direction, // Incorporate adjustment from UEditorInteractiveToolsContext::GetRayFromMousePos()
				Direction,
				true
			);
		} 
	}
	
	FVector4 PixelToScreen(const FVector2D& PixelLocation, bool bClamped=true) const
	{
		double X = PixelLocation.X - UnscaledViewRect.Min.X;
		double Y = PixelLocation.Y - UnscaledViewRect.Min.Y; 

		if (bClamped)
		{
			X = FMath::Clamp(X, 0.0, UnscaledViewRect.Width());
			Y = FMath::Clamp(Y, 0.0, UnscaledViewRect.Height());
		}
	
		return FVector4(
			-1.0 + X / UnscaledViewRect.Width() * 2.0,
			1.0 + Y / UnscaledViewRect.Height() * -2.0,
			0.0,
			1.0
		);
	}
	
	FRay PixelToScreenRay(const FVector2D& PixelLocation, bool bClamped=true) const
	{
		FVector4 ScreenPosition = PixelToScreen(PixelLocation, bClamped);
		return ScreenToRay(ScreenPosition);
	}

	FMatrices ViewMatrices;
	FIntRect UnscaledViewRect;
	FVector	ViewLocation;

protected:
	bool bIsPerspectiveProjection;
	double DPIScale = 1.0; // Usually set from ViewportClient
};
