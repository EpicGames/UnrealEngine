// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "HitTargets.generated.h"

class UPrimitiveComponent;


/**
 * UGizmoLambdaHitTarget is an IGizmoClickTarget implementation that
 * forwards the hit-test function to a TFunction
 */
UCLASS(MinimalAPI)
class UGizmoLambdaHitTarget : public UObject, public IGizmoClickTarget
{
	GENERATED_BODY()
public:
	/** This function is called to determine if target is hit */
	TUniqueFunction<FInputRayHit(const FInputDeviceRay&)> IsHitFunction;

	/** This function is called to update hover state of the target */
	TFunction<void(bool)> UpdateHoverFunction;

	/** This function is called to update interacting state of the target */
	TFunction<void(bool)> UpdateInteractingFunction;
	
	/** This function is called to update selected state of the target */
	TFunction<void(bool)> UpdateSelectedFunction;

	/** This function is called to update subdued state of the target */
	TFunction<void(bool)> UpdateSubdueFunction;

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHoverState(bool bHovering) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateInteractingState(bool bInteracting) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSelectedState(bool bSelected) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSubdueState(bool bSubdued) override;
};



/**
 * UGizmoComponentHitTarget is an IGizmoClickTarget implementation that
 * hit-tests a UPrimitiveComponent
 */
UCLASS(MinimalAPI)
class UGizmoComponentHitTarget : public UObject, public IGizmoClickTarget
{
	GENERATED_BODY()
public:

	/**
	 * Component->LineTraceComponent() is called to determine if the target is hit
	 */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> Component;

	/**
	 * If set, this condition is checked before performing the hit test. This gives a way 
	 * to disable the hit test without hiding the component. This is useful, for instance,
	 * in a repositionable transform gizmo in world-coordinate mode, where the rotation
	 * components need to be hittable for movement, but not for repositioning.
	 */
	TFunction<bool(const FInputDeviceRay&)> Condition = nullptr;

	/** This function is called to update hover state of the target */
	TFunction<void(bool)> UpdateHoverFunction;

	/** This function is called to update interacting state of the target */
	TFunction<void(bool)> UpdateInteractingFunction;

	/** This function is called to update selected state of the target */
	TFunction<void(bool)> UpdateSelectedFunction;

	/** This function is called to update subdued state of the target */
	TFunction<void(bool)> UpdateSubduedFunction;

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit IsHit(const FInputDeviceRay& ClickPos) const;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateHoverState(bool bHovering) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateInteractingState(bool bHovering) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSelectedState(bool bSelected) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSubdueState(bool bSubdued) override;

public:
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoComponentHitTarget* Construct(
		UPrimitiveComponent* Component,
		UObject* Outer = (UObject*)GetTransientPackage());
};
