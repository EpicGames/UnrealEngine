// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "InteractiveToolObjects.h"
#include "InteractiveToolChange.h"
#include "BaseGizmos/GizmoActor.h"
#include "BaseGizmos/TransformProxy.h"
#include "ParameterSourcesFloat.h"
#include "BoxTypes.h"
#include "IntervalGizmo.generated.h"

class UInteractiveGizmoManager;
class IGizmoAxisSource;
class IGizmoTransformSource;
class IGizmoStateTarget;
class UGizmoComponentAxisSource;
class IGizmoFloatParameterSource;
class UGizmoLocalFloatParameterSource;
class UGizmoViewContext;


/**
 * AIntervalGizmoActor is an Actor type intended to be used with UIntervalGizmo,
 * as the in-scene visual representation of the Gizmo.
 *
 * FIntervalGizmoActorFactory returns an instance of this Actor type (or a subclass).
 *
 * If a particular sub-Gizmo is not required, simply set that UProperty to null.
 */
UCLASS(Transient, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType, MinimalAPI)
class AIntervalGizmoActor : public AGizmoActor
{
	GENERATED_BODY()
public:

	INTERACTIVETOOLSFRAMEWORK_API AIntervalGizmoActor();
	
public:

	UPROPERTY()
	TObjectPtr<UGizmoLineHandleComponent> UpIntervalComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UGizmoLineHandleComponent> DownIntervalComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UGizmoLineHandleComponent> ForwardIntervalComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UGizmoLineHandleComponent> BackwardIntervalComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UGizmoLineHandleComponent> RightIntervalComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UGizmoLineHandleComponent> LeftIntervalComponent = nullptr;

public:
	/**
	 * Create a new instance of AIntervalGizmoActor and populate the various
	 * sub-components with standard GizmoXComponent instances
	 */
	static INTERACTIVETOOLSFRAMEWORK_API AIntervalGizmoActor* ConstructDefaultIntervalGizmo(UWorld* World, 
		UGizmoViewContext* GizmoViewContext);

};


/**
 * FIntervalGizmoActorFactory creates new instances of AIntervalGizmoActor which
 * are used by UIntervalGizmo to implement 3D interval Gizmos.
 * An instance of FIntervalGizmoActorFactory is passed to UIntervalGizmo
 * (by way of UIntervalGizmoBuilder), which then calls CreateNewGizmoActor()
 * to spawn new Gizmo Actors.
 *
 * By default CreateNewGizmoActor() returns a default Gizmo Actor suitable for
 * a three-axis Interval Gizmo, override this function to customize
 * the Actor sub-elements.
 */
class FIntervalGizmoActorFactory
{
public:
	FIntervalGizmoActorFactory(UGizmoViewContext* GizmoViewContextIn)
		: GizmoViewContext(GizmoViewContextIn)
	{
	}

	/**
	 * @param World the UWorld to create the new Actor in
	 * @return new AIntervalGizmoActor instance with members initialized with Components suitable for a transformation Gizmo
	 */
	virtual AIntervalGizmoActor* CreateNewGizmoActor(UWorld* World) const
	{
		return AIntervalGizmoActor::ConstructDefaultIntervalGizmo(World, GizmoViewContext);
	}

protected:
	/**
	 * Needs to be set (and kept alive elsewhere) so that created handle gizmos can
	 * adjust their length according to view (when not using world-scaling).
	 */
	UGizmoViewContext* GizmoViewContext = nullptr;
};



UCLASS(MinimalAPI)
class UIntervalGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:
	/**
	 * If set, this Actor Builder will be passed to UIntervalGizmo instances.
	 * Otherwise new instances of the base FIntervalGizmoActorFactory are created internally.
	 */
	TSharedPtr<FIntervalGizmoActorFactory> GizmoActorBuilder;

	/**
	 * If set, this hover function will be passed to UIntervalGizmo instances to use instead of the default.
	 * Hover is complicated for UIntervalGizmo because all it knows about the different gizmo scene elements
	 * is that they are UPrimitiveComponent (coming from the AIntervalGizmoActor). The default hover
	 * function implementation is to try casting to UGizmoBaseComponent and calling ::UpdateHoverState().
	 * If you are using different Components that do not subclass UGizmoBaseComponent, and you want hover to
	 * work, you will need to provide a different hover update function.
	 */
	TFunction<void(UPrimitiveComponent*, bool)> UpdateHoverFunction;

	/**
	 * If set, this coord-system function will be passed to UIntervalGizmo instances to use instead
	 * of the default UpdateCoordSystemFunction. By default theUIntervalGizmo will query the external Context
	 * to ask whether it should be using world or local coordinate system. Then the default UpdateCoordSystemFunction
	 * will try casting to UGizmoBaseCmponent and passing that info on via UpdateWorldLocalState();
	 * If you are using different Components that do not subclass UGizmoBaseComponent, and you want the coord system
	 * to be configurable, you will need to provide a different update function.
	 */
	TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> UpdateCoordSystemFunction;


	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


UCLASS(MinimalAPI)
class UIntervalGizmo : public UInteractiveGizmo
{
	GENERATED_BODY()

public:

	static INTERACTIVETOOLSFRAMEWORK_API FString GizmoName;

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetWorld(UWorld* WorldIn);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetGizmoActorBuilder(TSharedPtr<FIntervalGizmoActorFactory> Builder);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpdateHoverFunction(TFunction<void(UPrimitiveComponent*, bool)> HoverFunction);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetUpdateCoordSystemFunction(TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> CoordSysFunction);

	// UInteractiveGizmo overrides
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime) override;

	enum class EDefaultParameterRanges
	{
		// Init all interval ranges to [0, +/- FLT_MAX] in the axis direction
		HalfRange,
		// Init all interval ranges to [-FLT_MAX, FLT_MAX]
		FullRange,
		// Init interval ranges to 'half range' if 'matched' by the opposite interval on the same axis, or 'full range' otherwise
		HalfIfMatched
	};

	// Input parameter sources and related configuration
	struct FParameterSources
	{
		// +/- X axis parameter sources
		UGizmoLocalFloatParameterSource* RightInterval = nullptr;
		UGizmoLocalFloatParameterSource* LeftInterval = nullptr;
		// +/- Y axis parameter sources
		UGizmoLocalFloatParameterSource* ForwardInterval = nullptr;
		UGizmoLocalFloatParameterSource* BackwardInterval = nullptr;
		// +/- Z axis parameter sources
		UGizmoLocalFloatParameterSource* UpInterval = nullptr;
		UGizmoLocalFloatParameterSource* DownInterval = nullptr;

		EDefaultParameterRanges InitParameterRanges = EDefaultParameterRanges::HalfIfMatched;
		
	};

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetActiveTarget(UTransformProxy* TransformTargetIn, UGizmoLocalFloatParameterSource* UpInterval, UGizmoLocalFloatParameterSource* DownInterval, UGizmoLocalFloatParameterSource* ForwardInterval,
		                         IToolContextTransactionProvider* TransactionProvider = nullptr);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetActiveTarget(UTransformProxy* TransformTargetIn, const FParameterSources& ParameterSources,
		IToolContextTransactionProvider* TransactionProvider = nullptr);

	/** Sets functions that allow the endpoints of the intervals to be snapped to world geometry when ShouldAlignDestination is true */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetWorldAlignmentFunctions(
		TUniqueFunction<bool()>&& ShouldAlignDestination,
		TUniqueFunction<bool(const FRay&, FVector&)>&& DestinationAlignmentRayCaster
	);

	/** Notifies listeners that a sequence of edits to the gizmo is beginning/ending (at the start/end of a drag). */
	virtual void BeginEditSequence() { OnBeginIntervalGizmoEdit.Broadcast(this); }
	virtual void EndEditSequence() { OnEndIntervalGizmoEdit.Broadcast(this); }

	/**
	* Clear the parameter sources for this gizmo
	*/
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearSources();

	/**
	 * Clear the active target object for the Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ClearActiveTarget();

	/**
	 * Gets the location and orientation of the interval gizmo.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetGizmoTransform() const;

	/** Set visibility of the Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetVisibility(bool bVisible);

	/** State target is shared across gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoTransformChangeStateTarget> StateTarget = nullptr;

	/** 
	 * Called when an interval is changed. The delegate gives a pointer to the gizmo, the direction of the interval in
	 * gizmo space, and the new parameter value.
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnIntervalChanged, UIntervalGizmo*, const FVector&, float);
	FOnIntervalChanged OnIntervalChanged;

	/** Called when the gizmo is notified about the start of a sequence of interval changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBeginIntervalEdit, UIntervalGizmo*);
	FOnBeginIntervalEdit OnBeginIntervalGizmoEdit;

	/** Called when the gizmo is notified about the end of a sequence of interval changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndIntervalEdit, UIntervalGizmo*);
	FOnEndIntervalEdit OnEndIntervalGizmoEdit;
protected:

	/** GizmoActors will be spawned in this World */
	UPROPERTY()
	TObjectPtr<UWorld> World;

	UPROPERTY()
	TObjectPtr<AIntervalGizmoActor> GizmoActor = nullptr;

	/** The gizmo tracks the location and orientation of the transform in this TransformProxy. */
	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	/** list of current-active child components */
	UPROPERTY()
	TArray<TObjectPtr<UPrimitiveComponent>> ActiveComponents;

	/** list of currently-active child gizmos */
	UPROPERTY()
	TArray<TObjectPtr<UInteractiveGizmo>> ActiveGizmos;

	// +/- Z axis parameter sources
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> UpIntervalSource = nullptr;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> DownIntervalSource = nullptr;
	// +/- Y axis parameter sources
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> ForwardIntervalSource = nullptr;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> BackwardIntervalSource = nullptr;
	// +/- X axis parameter sources
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> RightIntervalSource = nullptr;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> LeftIntervalSource = nullptr;
	
	/** X-axis source is shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> AxisXSource = nullptr;
	
	/** Y-axis source is shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> AxisYSource = nullptr;

	/** Z-axis source is shared across Gizmos, and created internally during SetActiveTarget() */
	UPROPERTY()
	TObjectPtr<UGizmoComponentAxisSource> AxisZSource = nullptr;

protected:
	TSharedPtr<FIntervalGizmoActorFactory> GizmoActorBuilder;

	// This function is called on each active GizmoActor Component to update it's hover state.
	// If the Component is not a UGizmoBaseCmponent, the client needs to provide a different implementation
	// of this function via the ToolBuilder
	TFunction<void(UPrimitiveComponent*, bool)> UpdateHoverFunction;

	// This function is called on each active GizmoActor Component to update it's coordinate system (eg world/local).
	// If the Component is not a UGizmoBaseCmponent, the client needs to provide a different implementation
	// of this function via the ToolBuilder
	TFunction<void(UPrimitiveComponent*, EToolContextCoordinateSystem)> UpdateCoordSystemFunction;

	TUniqueFunction<bool()> ShouldAlignDestination = []() { return false; };
	TUniqueFunction<bool(const FRay&, FVector&)> DestinationAlignmentRayCaster = [](const FRay&, FVector&) {return false; };
protected:

	/** @return a new instance of the standard axis-handle Gizmo */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* AddIntervalHandleGizmo(
		USceneComponent* RootComponent,
		UPrimitiveComponent* HandleComponent, 
		IGizmoAxisSource* AxisSource,
		IGizmoFloatParameterSource* FloatParameterSource,
		float MinParameter,
		float MaxParameter,
		IGizmoStateTarget* StateTargetIn);

private:

	// Valid ranges for intervals
	UE::Geometry::FInterval1f UpIntervalRange, DownIntervalRange, ForwardIntervalRange, BackwardIntervalRange, RightIntervalRange, LeftIntervalRange;

	// Call IterFn on each Source/Component combination where the Source is not null. Note IterFn will still be called if the component and/or axis are null.
	void EnumerateValidIntervals(TFunctionRef<void(UGizmoLocalFloatParameterSource* Source, UGizmoLineHandleComponent* Component, UGizmoComponentAxisSource* Axis, UE::Geometry::FInterval1f& IntervalRange, FVector3d Direction, float DirectionAxisSign)> IterFn);

	// Call IterFn on each interval Component (including those without a Source)
	void EnumerateAllIntervals(TFunctionRef<void(UGizmoLocalFloatParameterSource* Source, UGizmoLocalFloatParameterSource* OppositeSource, UGizmoLineHandleComponent* Component, UE::Geometry::FInterval1f& IntervalRange, float DirectionAxisSign)> IterFn);
};

/**
 * UGizmoAxisIntervalParameterSource is an IGizmoFloatParameterSource implementation that
 * interprets the float value as the parameter of a line equation, and maps this parameter to a 3D translation
 * along a line with origin/direction given by an IGizmoAxisSource. This translation is applied to an IGizmoTransformSource.
 *
 * This ParameterSource is intended to be used to create 3D Axis Interval Gizmos.
 */
UCLASS(MinimalAPI)
class UGizmoAxisIntervalParameterSource : public UGizmoBaseFloatParameterSource
{
	GENERATED_BODY()
public:
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetParameter() const override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(float NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify();
	
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify();


public:
	
	UPROPERTY()
	TScriptInterface<IGizmoFloatParameterSource> FloatParameterSource;

public:

	UPROPERTY()
	float MinParameter;

	UPROPERTY()
	float MaxParameter;

public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static INTERACTIVETOOLSFRAMEWORK_API UGizmoAxisIntervalParameterSource* Construct(
		IGizmoFloatParameterSource* FloatSourceIn,
		float ParameterMin = -FLT_MAX,
		float ParameterMax = FLT_MAX,
		UObject* Outer = (UObject*)GetTransientPackage());
};
