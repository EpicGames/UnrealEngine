// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoInterfaces.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ParameterSourcesFloat.generated.h"

/**
 * FGizmoVec2ParameterChange represents a change in the value of an IGizmoFloatParameterSource.
 * IGizmoFloatParameterSource implementations use this to track changes and emit delta information.
 */
USTRUCT()
struct FGizmoFloatParameterChange
{
	GENERATED_BODY()

	UPROPERTY()
	float InitialValue = 0;

	UPROPERTY()
	float CurrentValue = 0;


	float GetChangeDelta() const
	{
		return CurrentValue - InitialValue;
	}

	explicit FGizmoFloatParameterChange(float StartValue = 0)
	{
		InitialValue = CurrentValue = StartValue;
	}
};


/**
 * UGizmoBaseFloatParameterSource is a base implementation of IGizmoFloatParameterSource,
 * which is not functional but adds an OnParameterChanged delegate for further subclasses.
 */
UCLASS(MinimalAPI)
class UGizmoBaseFloatParameterSource : public UObject, public IGizmoFloatParameterSource
{
	GENERATED_BODY()
public:
	virtual float GetParameter() const
	{
		check(false);    // this is an abstract base class
		return 0.0f;
	}

	virtual void SetParameter(float NewValue)
	{
		check(false);    // this is an abstract base class
	}


	virtual void BeginModify()
	{
	}

	virtual void EndModify()
	{
	}


public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGizmoFloatParameterSourceChanged, IGizmoFloatParameterSource*, FGizmoFloatParameterChange);
	FOnGizmoFloatParameterSourceChanged OnParameterChanged;
};



/**
 * UGizmoLocalFloatParameterSource is an implementation of IGizmoFloatParameterSource
 * (by way of UGizmoBaseFloatParameterSource) which locally stores the relevant Parameter
 * and emits update events via the OnParameterChanged delegate.
 */
UCLASS(MinimalAPI)
class UGizmoLocalFloatParameterSource : public UGizmoBaseFloatParameterSource
{
	GENERATED_BODY()
public:
	virtual float GetParameter() const override
	{
		return Value;
	}

	virtual void SetParameter(float NewValue) override
	{
		Value = NewValue;
		LastChange.CurrentValue = NewValue;
		OnParameterChanged.Broadcast(this, LastChange);
	}

	virtual void BeginModify()
	{
		LastChange = FGizmoFloatParameterChange(Value);
	}

	virtual void EndModify()
	{
	}


public:

	UPROPERTY()
	float Value = 0.0f;

	UPROPERTY()
	FGizmoFloatParameterChange LastChange;
};



