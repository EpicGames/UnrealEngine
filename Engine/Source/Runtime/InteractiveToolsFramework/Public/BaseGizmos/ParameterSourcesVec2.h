// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "ParameterSourcesVec2.generated.h"

//
// various implementations of IGizmoVec2ParameterSource
// 


/**
 * FGizmoVec2ParameterChange represents a change in the value of an IGizmoVec2ParameterSource.
 * IGizmoVec2ParameterSource implementations use this to track changes and emit delta information.
 */
USTRUCT()
struct FGizmoVec2ParameterChange
{
	GENERATED_BODY()

	UPROPERTY()
	FVector2D InitialValue = FVector2D::ZeroVector;

	UPROPERTY()
	FVector2D CurrentValue = FVector2D::ZeroVector;


	FVector2D GetChangeDelta() const
	{
		return CurrentValue - InitialValue;
	}

	explicit FGizmoVec2ParameterChange(const FVector2D& StartValue = FVector2D::ZeroVector)
	{
		InitialValue = CurrentValue = StartValue;
	}
};



/**
 * UGizmoBaseVec2ParameterSource is a base implementation of IGizmoVec2ParameterSource,
 * which is not functional but adds an OnParameterChanged delegate for further subclasses.
 */
UCLASS(MinimalAPI)
class UGizmoBaseVec2ParameterSource : public UObject, public IGizmoVec2ParameterSource
{
	GENERATED_BODY()
public:
	virtual FVector2D GetParameter() const
	{
		check(false);    // this is an abstract base class
		return FVector2D::ZeroVector;
	}

	virtual void SetParameter(const FVector2D& NewValue)
	{
		check(false);    // this is an abstract base class
	}


	virtual void BeginModify()
	{
	}

	virtual void EndModify()
	{
	}


public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGizmoVec2ParameterSourceChanged, IGizmoVec2ParameterSource*, FGizmoVec2ParameterChange);
	FOnGizmoVec2ParameterSourceChanged OnParameterChanged;
};



/**
 * UGizmoBaseVec2ParameterSource is an implementation of IGizmoVec2ParameterSource 
 * (by way of UGizmoBaseVec2ParameterSource) which locally stores the relevant Parameter
 * and emits update events via the OnParameterChanged delegate.
 */
UCLASS(MinimalAPI)
class UGizmoLocalVec2ParameterSource : public UGizmoBaseVec2ParameterSource
{
	GENERATED_BODY()
public:
	virtual FVector2D GetParameter() const override
	{
		return Value;
	}

	virtual void SetParameter(const FVector2D& NewValue) override
	{
		Value = NewValue;
		LastChange.CurrentValue = NewValue;
		OnParameterChanged.Broadcast(this, LastChange);
	}

	virtual void BeginModify()
	{
		LastChange = FGizmoVec2ParameterChange(Value);
	}

	virtual void EndModify()
	{
	}


public:

	UPROPERTY()
	FVector2D Value = FVector2D::ZeroVector;

	UPROPERTY()
	FGizmoVec2ParameterChange LastChange;
};



