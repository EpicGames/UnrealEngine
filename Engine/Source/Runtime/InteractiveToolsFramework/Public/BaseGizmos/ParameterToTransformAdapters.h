// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseGizmos/ParameterSourcesFloat.h"
#include "BaseGizmos/ParameterSourcesVec2.h"
#include "ParameterToTransformAdapters.generated.h"


//
// Various 1D and 2D ParameterSource converters intended to be used to create 3D transformation gizmos.
// Based on base classes in ParameterSourcesFloat.h and ParameterSourcesVec2.h
// 



/**
 * UGizmoAxisTranslationParameterSource is an IGizmoFloatParameterSource implementation that
 * interprets the float value as the parameter of a line equation, and maps this parameter to a 3D translation 
 * along a line with origin/direction given by an IGizmoAxisSource. This translation is applied to an IGizmoTransformSource.
 *
 * This ParameterSource is intended to be used to create 3D Axis Translation Gizmos.
 */
UCLASS(MinimalAPI)
class UGizmoAxisTranslationParameterSource : public UGizmoBaseFloatParameterSource
{
	GENERATED_BODY()
public:


	/**
	 * Optional axis delta constraint function. This can be used to snap/constrain the "parameter delta" value,
	 * for example to snap it to fixed increments along the axis. 
	 * @return true if constrained delta was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(double AxisDelta, double& SnappedDelta)> AxisDeltaConstraintFunction = [](double, double&) { return false; };

	/**
	 * Optional position constraint function. Called during interaction with the new transform origin.
	 * To snap the transform to a new position, return as second value, and return true from your lambda.
	 * Note that returned snap point will be projected onto the current translation origin/axis.
	 * @return true if constraint point was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(const FVector& Input, FVector& PreProjectedOutput)> PositionConstraintFunction = [](const FVector&, FVector&) { return false; };

	virtual float GetParameter() const override
	{
		return Parameter;
	}

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(float NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify() override;


public:
	/** The Parameter line-equation value is converted to a 3D Translation along this Axis */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** This TransformSource is updated by applying the constructed 3D translation */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;


public:
	/** Parameter is the line-equation parameter that this FloatParameterSource provides */
	UPROPERTY()
	float Parameter = 0.0f;

	/** Active parameter change (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FGizmoFloatParameterChange LastChange;

	/** tranlsation axis for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurTranslationAxis;

	/** tranlsation origin for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurTranslationOrigin;

	/** Saved copy of Initial Transform for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FTransform InitialTransform;

public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static UGizmoAxisTranslationParameterSource* Construct(
		IGizmoAxisSource* AxisSourceIn,
		IGizmoTransformSource* TransformSourceIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoAxisTranslationParameterSource* NewSource = NewObject<UGizmoAxisTranslationParameterSource>(Outer);
		NewSource->AxisSource = Cast<UObject>(AxisSourceIn);
		NewSource->TransformSource = Cast<UObject>(TransformSourceIn);
		return NewSource;
	}
};





/**
 * UGizmoAxisRotationParameterSource is an IGizmoVec2ParameterSource implementation that
 * interprets the FVector2D parameter as a position in a 2D plane, and maps this position to a 3D translation in
 * a plane with origin/normal given by an IGizmoAxisSource. This translation is applied to an IGizmoTransformSource.
 * 
 * This ParameterSource is intended to be used to create 3D Plane Translation Gizmos.
 */
UCLASS(MinimalAPI)
class UGizmoPlaneTranslationParameterSource : public UGizmoBaseVec2ParameterSource
{
	GENERATED_BODY()
public:

	/**
	 * Optional axis delta constraint function. This can be used to snap/constrain the "parameter delta" value,
	 * for example to snap it to fixed increments along the axis. 
	 * The AxisDeltaConstraintFunction is used separately for the X and Y axes of the plane translation
	 * @return true if constrained delta was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(double AxisDelta, double& SnappedDelta)> AxisXDeltaConstraintFunction = [](double, double&) { return false; };
	TUniqueFunction<bool(double AxisDelta, double& SnappedDelta)> AxisYDeltaConstraintFunction = [](double, double&) { return false; };

	/**
	 * Optional position constraint function. Called during interaction with the new transform origin.
	 * To snap the transform to a new position, return as second value, and return true from your lambda.
	 * @return true if constraint point was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(const FVector& Input, FVector& PreProjectedOutput)> PositionConstraintFunction = [](const FVector&, FVector&) { return false; };
	/*
	 * When true, the output of PositionConstraintFunction will be projected onto the current translation origin/axis.
	 */
	bool bProjectConstrainedPosition = true;

	virtual FVector2D GetParameter() const override
	{
		return Parameter;
	}

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(const FVector2D& NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify() override;


public:
	/** AxisSource provides the 3D plane (origin/normal/u/v) that is used to interpret the 2D parameters */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** This TransformSource is updated by applying the constructed 3D translation */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;


public:
	/** Parameter is the two line-equation parameters that this Vec2ParameterSource provides */
	UPROPERTY()
	FVector2D Parameter = FVector2D::ZeroVector;

	/** Active parameter change (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FGizmoVec2ParameterChange LastChange;

	/** plane origin for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurTranslationOrigin;

	/** plane normal for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurTranslationNormal;

	/** in-plane axis X for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurTranslationAxisX;

	/** in-plane axis Y for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurTranslationAxisY;

	/** Saved copy of Initial Transform for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FTransform InitialTransform;

public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static UGizmoPlaneTranslationParameterSource* Construct(
		IGizmoAxisSource* AxisSourceIn,
		IGizmoTransformSource* TransformSourceIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoPlaneTranslationParameterSource* NewSource = NewObject<UGizmoPlaneTranslationParameterSource>(Outer);
		NewSource->AxisSource = Cast<UObject>(AxisSourceIn);
		NewSource->TransformSource = Cast<UObject>(TransformSourceIn);
		return NewSource;
	}
};






/**
 * UGizmoAxisRotationParameterSource is an IGizmoFloatParameterSource implementation that
 * interprets the float parameter as an angle, and maps this angle to a 3D rotation
 * around an IGizmoAxisSource (ie 3D axis). This rotation is applied to an IGizmoTransformSource.
 * This ParameterSource is intended to be used to create 3D Rotation Gizmos.
 */
UCLASS(MinimalAPI)
class UGizmoAxisRotationParameterSource : public UGizmoBaseFloatParameterSource
{
	GENERATED_BODY()
public:

	/**
	 * Optional axis-delta-angle constraint function. This can be used to snap/constrain the "angle delta" value,
	 * for example to snap it to fixed angle steps around the axis. 
	 * @return true if constrained angle was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(double AngleDelta, double& SnappedAngleDelta)> AngleDeltaConstraintFunction = [](double, double&) { return false; };

	/**
	 * Optional rotation constraint function. Called during interaction with the rotation delta
	 * To snap the rotation delta, return the snapped quat
	 * @return The snapped value of the rotation delta
	 */
	TUniqueFunction<FQuat(const FQuat&)> RotationConstraintFunction = [](const FQuat& DeltaRotation) { return DeltaRotation; };

	virtual float GetParameter() const override
	{
		return Angle;
	}

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(float NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify() override;


public:
	/** float-parameter Angle is mapped to a 3D Rotation around this Axis */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** This TransformSource is updated by applying the constructed 3D rotation */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;


public:
	/** Angle is the parameter that this FloatParameterSource provides */
	UPROPERTY()
	float Angle = 0.0f;

	/** Active parameter change (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FGizmoFloatParameterChange LastChange;

	/** Rotation axis for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurRotationAxis;

	/** Rotation origin for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurRotationOrigin;

	/** Saved copy of Initial Transform for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FTransform InitialTransform;


public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static UGizmoAxisRotationParameterSource* Construct(
		IGizmoAxisSource* AxisSourceIn,
		IGizmoTransformSource* TransformSourceIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoAxisRotationParameterSource* NewSource = NewObject<UGizmoAxisRotationParameterSource>(Outer);
		NewSource->AxisSource = Cast<UObject>(AxisSourceIn);
		NewSource->TransformSource = Cast<UObject>(TransformSourceIn);
		return NewSource;
	}
};









 /**
  * UGizmoUniformScaleParameterSource is an IGizmoVec2ParameterSource implementation that
  * interprets the Vec2 parameter as the position in a 2D plane, and maps this parameter to a
  * change in a uniform scale value. This scale is applied to an IGizmoTransformSource.
  * 
  * The X/Y plane parameter deltas are converted to a uniform scale delta simply by adding them.
  * This assumpes that the 3D plane is oriented such that "positive" along the X and Y 
  * tangent axes corresponds to something semantically meaningful. It's up to the client to do this.
  *
  * This ParameterSource is intended to be used to create 3D Uniform Scale Gizmos.
  */
UCLASS(MinimalAPI)
class UGizmoUniformScaleParameterSource : public UGizmoBaseVec2ParameterSource
{
	GENERATED_BODY()
public:

	/**
	 * Optional scale axis delta constraint function. This can be used to snap/constrain the "scale delta" value,
	 * for example to snap it to fixed increments along the axis. 
	 * @return true if constrained delta was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(double ScaleAxisDelta, double& SnappedScaleAxisDelta)> ScaleAxisDeltaConstraintFunction = [](double, double&) { return false; };
	/**
	 * Optional scale constraint function. Called during interaction with the scale delta
	 * To snap the scale delta, return the snapped value
	 * @return The snapped value of the scale delta
	 */
	TUniqueFunction<double(const double&)> ScaleConstraintFunction = [](const double& DeltaScale) { return DeltaScale; };


	virtual FVector2D GetParameter() const override
	{
		return Parameter;
	}

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(const FVector2D& NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify() override;


public:
	/** AxisSource provides the 3D plane (origin/normal/u/v) that is used to interpret the 2D parameters */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** This TransformSource is updated by applying the constructed 3D translation */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;

	/** Coordinate delta is multiplied by this amount */
	UPROPERTY()
	float ScaleMultiplier = 0.02f;

public:
	/** Parameter is the two line-equation parameters that this Vec2ParameterSource provides */
	UPROPERTY()
	FVector2D Parameter = FVector2D::ZeroVector;

	/** Active parameter change (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FGizmoVec2ParameterChange LastChange;

	/** plane origin for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleOrigin;

	/** plane normal for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleNormal;

	/** in-plane axis X for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleAxisX;

	/** in-plane axis Y for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleAxisY;

	/** Saved copy of Initial Transform for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FTransform InitialTransform;

public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static UGizmoUniformScaleParameterSource* Construct(
		IGizmoAxisSource* AxisSourceIn,
		IGizmoTransformSource* TransformSourceIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoUniformScaleParameterSource* NewSource = NewObject<UGizmoUniformScaleParameterSource>(Outer);
		NewSource->AxisSource = Cast<UObject>(AxisSourceIn);
		NewSource->TransformSource = Cast<UObject>(TransformSourceIn);
		return NewSource;
	}
};





/**
 * UGizmoAxisScaleParameterSource is an IGizmoFloatParameterSource implementation that
 * interprets the float value as the parameter of a line equation, and maps this parameter to a scale factor
 * along a line with origin/direction given by an IGizmoAxisSource. This scale is applied to an IGizmoTransformSource.
 *
 * This ParameterSource is intended to be used to create 3D Axis Scale Gizmos.
 */
UCLASS(MinimalAPI)
class UGizmoAxisScaleParameterSource : public UGizmoBaseFloatParameterSource
{
	GENERATED_BODY()
public:

	/**
	 * Optional scale axis delta constraint function. This can be used to snap/constrain the "scale delta" value,
	 * for example to snap it to fixed increments along the axis. 
	 * @return true if constrained delta was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(const double& ScaleAxisDelta, double& SnappedScaleAxisDelta)> ScaleAxisDeltaConstraintFunction = [](const double&, double&) { return false; };
	/**
	 * Optional scale constraint function. Called during interaction with the scale delta
	 * To snap the scale delta, return the snapped value
	 * @return The snapped value of the scale delta
	 */
	TUniqueFunction<double(const double&)> ScaleConstraintFunction = [](const double& DeltaScale) { return DeltaScale; };

	virtual float GetParameter() const override
	{
		return Parameter;
	}

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(float NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify() override;

public:
	/** The Parameter line-equation value is converted to a 3D Translation along this Axis */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** This TransformSource is updated by applying the constructed 3D rotation */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;

	/** Coordinate delta is multiplied by this amount */
	UPROPERTY()
	float ScaleMultiplier = 0.02f;

	/** If true, the minimal output scale will be zero. */
	UPROPERTY()
	bool bClampToZero = false;

public:
	/** Parameter is the line-equation parameter that this FloatParameterSource provides */
	UPROPERTY()
	float Parameter = 0.0f;

	/** Active parameter change (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FGizmoFloatParameterChange LastChange;

	/** scale axis for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleAxis;

	/** scale origin for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleOrigin;

	/** Saved copy of Initial transform for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FTransform InitialTransform;

public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static UGizmoAxisScaleParameterSource* Construct(
		IGizmoAxisSource* AxisSourceIn,
		IGizmoTransformSource* TransformSourceIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoAxisScaleParameterSource* NewSource = NewObject<UGizmoAxisScaleParameterSource>(Outer);
		NewSource->AxisSource = Cast<UObject>(AxisSourceIn);
		NewSource->TransformSource = Cast<UObject>(TransformSourceIn);
		return NewSource;
	}
};








/**
 * UGizmoPlaneScaleParameterSource  is an UGizmoBaseVec2ParameterSource implementation that
 * maps a 2D parameter delta to a change in a 3D scaling vector, based on the tangent axes of a 3D plane
 * This scale is applied to an IGizmoTransformSource.
 *
 * This ParameterSource is intended to be used to create 3D Plane Scale Gizmos.
 */
UCLASS(MinimalAPI)
class UGizmoPlaneScaleParameterSource : public UGizmoBaseVec2ParameterSource
{
	GENERATED_BODY()
public:

	/**
	 * Optional position constraint function. Called during interaction with the new transform origin.
	 * To snap the transform to a new position, return as second value, and return true from your lambda.
	 * Note that returned snap point will be projected onto the current translation origin/axis.
	 * @return true if constraint point was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(const FVector&, FVector&)> PositionConstraintFunction = [](const FVector&, FVector&) { return false; };

	/**
	 * Optional scale axis delta constraint function. This can be used to snap/constrain the "scale delta" value,
	 * for example to snap it to fixed increments along the axis. 
	 * The Axis[X/Y]DeltaConstraintFunctions are used separately for the X and Y axes of the scale translation
	 * @return true if constrained delta was found and should be used, false to ignore
	 */
	TUniqueFunction<bool(double ScaleAxisDelta, double& SnappedScaleAxisDelta)> ScaleAxisXDeltaConstraintFunction = [](double, double&) { return false; };
	TUniqueFunction<bool(double ScaleAxisDelta, double& SnappedScaleAxisDelta)> ScaleAxisYDeltaConstraintFunction = [](double, double&) { return false; };
	/**
	 * Optional scale constraint function. Called during interaction with the scale delta
	 * To snap the scale delta, return the snapped value
	 * @return The snapped value of the scale delta as a FVector2D containing the snapped values for the X and Y axes of the scale translation
	 */
	TUniqueFunction<FVector2D(const FVector2D&)> ScaleConstraintFunction = [](const FVector2D& DeltaScale) { return DeltaScale; };


	virtual FVector2D GetParameter() const override
	{
		return Parameter;
	}

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetParameter(const FVector2D& NewValue) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginModify() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndModify() override;


public:
	/** AxisSource provides the 3D plane (origin/normal/u/v) that is used to interpret the 2D parameters */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** This TransformSource is updated by applying the constructed 3D translation */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> TransformSource;

	/** Coordinate delta is multiplied by this amount */
	UPROPERTY()
	float ScaleMultiplier = 0.02f;

	/** If true, the scaling will be done an equal amount in each axis, using the minimal value */
	UPROPERTY()
	bool bUseEqualScaling = false;

	/** If true, the minimal output scale will be zero. */
	UPROPERTY()
	bool bClampToZero = false;
public:
	/** Parameter is the two line-equation parameters that this Vec2ParameterSource provides */
	UPROPERTY()
	FVector2D Parameter = FVector2D::ZeroVector;

	/** Active parameter change (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FGizmoVec2ParameterChange LastChange;

	/** plane origin for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleOrigin;

	/** plane normal for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleNormal;

	/** in-plane axis X for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleAxisX;

	/** in-plane axis Y for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FVector CurScaleAxisY;

	/** Saved copy of Initial Transform for current parameter edit (only valid between BeginModify/EndModify) */
	UPROPERTY()
	FTransform InitialTransform;

public:

	/**
	 * Create a standard instance of this ParameterSource, with the given AxisSource and TransformSource
	 */
	static UGizmoPlaneScaleParameterSource* Construct(
		IGizmoAxisSource* AxisSourceIn,
		IGizmoTransformSource* TransformSourceIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoPlaneScaleParameterSource* NewSource = NewObject<UGizmoPlaneScaleParameterSource>(Outer);
		NewSource->AxisSource = Cast<UObject>(AxisSourceIn);
		NewSource->TransformSource = Cast<UObject>(TransformSourceIn);
		return NewSource;
	}
};

