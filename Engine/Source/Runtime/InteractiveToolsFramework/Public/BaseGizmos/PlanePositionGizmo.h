// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseGizmos/GizmoComponents.h"
#include "CoreMinimal.h"
#include "InputState.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "Math/MathFwd.h"
#include "Math/Ray.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Templates/Function.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "PlanePositionGizmo.generated.h"

class IGizmoAxisSource;
class IGizmoClickTarget;
class IGizmoStateTarget;
class IGizmoVec2ParameterSource;
class UClickDragInputBehavior;
class UObject;
struct FToolBuilderState;
namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}

UCLASS(MinimalAPI)
class UPlanePositionGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


/**
 * UPlanePositionGizmo implements a gizmo interaction where 2D parameter value is manipulated
 * by dragging a point on a 3D plane in space. The 3D position is converted to 2D coordinates
 * based on the tangent axes of the plane.
 * 
 * As with other base gizmos, this class only implements the interaction. The visual aspect of the
 * gizmo, the plane, and the parameter storage are all provided externally.
 *
 * The plane is provided by an IGizmoAxisSource. The origin and normal define the plane and then
 * the tangent axes of the source define the coordinate space. 
 * 
 * The interaction target (ie the thing you have to click on to start the dragging interaction) is provided by an IGizmoClickTarget. 
 *
 * The new 2D parameter value is sent to an IGizmoVec2ParameterSource
 *
 * Internally a UClickDragInputBehavior is used to handle mouse input, configured in ::Setup()
 */
UCLASS(MinimalAPI)
class UPlanePositionGizmo : public UInteractiveGizmo, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * Helper that initializes AxisSource, ParameterSource, HitTarget, and StateTarget for the common case
	 *  of being used to control the translation of a transform. Safe to use for reinitialization.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool InitializeAsTranslateGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& InitializationParams,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);
	
	/**
	 * Helper that initializes AxisSource, ParameterSource, HitTarget, and StateTarget for the common case
	 *  of being used to control the scale of a transform. Safe to use for reinitialization.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool InitializeAsScaleGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& InitializationParams,
		bool bDisallowNegativeScaling,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);

	/**
	 * Helper that initializes AxisSource, ParameterSource, HitTarget, and StateTarget for the common case
	 *  of being used to control the uniform scale of a transform, where the interaction plane remains aligned
	 *  to the camera. Safe to use for reinitialization.
	 * Note: if SharedState contains a CameraAxisSource, this subgizmo will use it and will assume that something
	 *  else is responsible for updating it. Otherwise this subgizmo will set the CameraAxisSource and update it
	 *  in its own Tick function.
	 */
	bool InitializeAsUniformScaleGizmo(
		const UE::GizmoUtil::FTransformSubGizmoCommonParams& Params,
		bool bDisallowNegativeScaling,
		UE::GizmoUtil::FTransformSubGizmoSharedState* SharedState);
	
	// UInteractiveGizmo overrides

	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime) override;

	// IClickDragBehaviorTarget implementation

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnTerminateDragSequence() override;

	// IHoverBehaviorTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndHover() override;

public:
	// The below properties can be manipulated for more fine-grained control, but typically it is sufficient
	// to use one of the initialization methods above.

	/** AxisSource provides the 3D plane on which the interaction happens */
	UPROPERTY()
	TScriptInterface<IGizmoAxisSource> AxisSource;

	/** The 3D plane coordinates are converted to 2D coordinates in the plane tangent space, and the change in value is sent to this ParameterSource */
	UPROPERTY()
	TScriptInterface<IGizmoVec2ParameterSource> ParameterSource;
	
	/** The HitTarget provides a hit-test against some 3D element (presumably a visual widget) that controls when interaction can start */
	UPROPERTY()
	TScriptInterface<IGizmoClickTarget> HitTarget;

	/** StateTarget is notified when interaction starts and ends, so that things like undo/redo can be handled externally */
	UPROPERTY()
	TScriptInterface<IGizmoStateTarget> StateTarget;

	/** The mouse click behavior of the gizmo is accessible so that it can be modified to use different mouse keys. */
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> MouseBehavior;

public:
	/** If enabled, then the sign on the parameter delta is always "increasing" when moving away from the origin point, rather than just being a projection onto the axis */
	UPROPERTY()
	bool bEnableSignedAxis = false;

	/** If enabled, flip sign of parameter delta on X axis */
	UPROPERTY()
	bool bFlipX = false;

	/** If enabled, flip sign of parameter delta on Y axis */
	UPROPERTY()
	bool bFlipY = false;

	/** 
	 * This gets checked to see if we should use the custom ray caster to get a destination point for the gizmo, rather
	 * than grabbing the intersection with the gizmo plane.
	 */
	TUniqueFunction<bool()> ShouldUseCustomDestinationFunc = []() {return false; };

	struct FCustomDestinationParams
	{
		// Right now we use the custom destination function for aligning to items in the scene, which
		// we just need the world ray for. If we want to use functions that use other inputs as the
		// basis for the destination, we would add those parameters here and would make sure that the 
		// gizmo passes them in.
		const FRay* WorldRay = nullptr;
	};

	/**
	 * If ShouldUseCustomDestinationFunc() returns true, this function is used to get a destination point, and
	 * the output parameters are picked in such a way that the axis origin moves to the closest point in the plane 
	 * to the destination point.
	 * Used, for instance, for aligning to items in the scene.
	 */
	TUniqueFunction<bool(const FCustomDestinationParams& WorldRay, FVector& OutputPoint)> CustomDestinationFunc =
		[](const FCustomDestinationParams& Params, FVector& OutputPoint) { return false; };

public:
	/** If true, we are in an active click+drag interaction, otherwise we are not */
	UPROPERTY()
	bool bInInteraction = false;


	//
	// The values below are used in the context of a single click-drag interaction, ie if bInInteraction = true
	// They otherwise should be considered uninitialized
	//

	UPROPERTY()
	FVector InteractionOrigin;

	UPROPERTY()
	FVector InteractionNormal;

	UPROPERTY()
	FVector InteractionAxisX;

	UPROPERTY()
	FVector InteractionAxisY;


	UPROPERTY()
	FVector InteractionStartPoint;

	UPROPERTY()
	FVector InteractionCurPoint;

	UPROPERTY()
	FVector2D InteractionStartParameter;

	UPROPERTY()
	FVector2D InteractionCurParameter;

	UPROPERTY()
	FVector2D ParameterSigns = FVector2D(1, 1);

protected:
	FVector LastHitPosition;
	FVector2D InitialTargetParameter;

	FVector2D InteractionStartOriginParameterOffset;
	FInputDeviceRay LastInputRay = FInputDeviceRay(FRay());

	// Helper that can hold some extra upkeep to do during Tick (used to
	//  update a camera axis source if needed).
	TUniqueFunction<void(float DeltaTime)> CustomTickFunction;
};

