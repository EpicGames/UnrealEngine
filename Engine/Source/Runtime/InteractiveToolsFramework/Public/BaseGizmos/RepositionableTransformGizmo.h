// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "ToolContextInterfaces.h"

#include "RepositionableTransformGizmo.generated.h"

class IToolContextTransactionProvider;
class UTransformProxy;

UCLASS(MinimalAPI)
class URepositionableTransformGizmoBuilder : public UCombinedTransformGizmoBuilder
{
	GENERATED_BODY()

public:

	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


/**
 * A transform gizmo that also allows the user to reposition it by middle-clicking rotation/translation components.
 */
UCLASS(MinimalAPI)
class URepositionableTransformGizmo : public UCombinedTransformGizmo
{
	GENERATED_BODY()

public:

	/**
	 * Set the active target object for the Gizmo
	 * @param Target active target
	 * @param TransactionProvider optional IToolContextTransactionProvider implementation to use - by default uses GizmoManager
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetActiveTarget(UTransformProxy* Target, IToolContextTransactionProvider* TransactionProvider = nullptr) override;

	/**
	 * Allows the user to provide functions to use for aligning the gizmo destination to items in the scene. Note that this affects
	 * both the movement and repositioning.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetWorldAlignmentFunctions(
		TUniqueFunction<bool()>&& ShouldAlignTranslationIn,
		TUniqueFunction<bool(const FRay&, FVector&)>&& TranslationAlignmentRayCasterIn) override;

	/**
	 * Allows the user to provide functions to use in aligning the gizmo during repositioning only.
	 * This is because the repositining is likely to want different raycasts that don't ignore the 
	 * moved components themselves, so that the gizmo can be aligned to a target component.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetPivotAlignmentFunctions(
		TUniqueFunction<bool()>&& ShouldAlignPivotIn,
		TUniqueFunction<bool(const FRay&, FVector&)>&& PivotAlignmentRayCasterIn);
protected:

	TUniqueFunction<bool()> ShouldAlignPivot = []() { return false; };
	TUniqueFunction<bool(const FRay&, FVector&)> PivotAlignmentRayCaster = [](const FRay&, FVector&) {return false; };

	/** Subset of ActiveGizmos, for use by SetPivotAlignmentFunctions */
	TArray<UInteractiveGizmo*> PivotAlignmentGizmos;

	UPROPERTY()
	TObjectPtr<UGizmoTransformChangeStateTarget> RepositionStateTarget;

	// Helper functions
	INTERACTIVETOOLSFRAMEWORK_API void ModifyPivotAxisGizmo(UInteractiveGizmo* SubGizmoIn);
	INTERACTIVETOOLSFRAMEWORK_API void ModifyPivotPlaneGizmo(UInteractiveGizmo* SubGizmoIn);
	INTERACTIVETOOLSFRAMEWORK_API void ModifyPivotRotateGizmo(UInteractiveGizmo* SubGizmoIn);
private:
	void ModifyPivotFreeTranslationGizmo(UInteractiveGizmo* SubGizmoIn);
	void ModifyPivotFreeRotationGizmo(UInteractiveGizmo* SubGizmoIn);
};
