// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "TransformProxy.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/AnyButtonInputBehavior.h"

#include "ScalableSphereGizmo.generated.h"

UCLASS(MinimalAPI)
class UScalableSphereGizmoBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};

/**
 * UScalableSphereGizmo provides a sphere that can be scaled in all directions by dragging
 * anywhere on the three axial circles that represent it
 */
UCLASS(MinimalAPI)
class UScalableSphereGizmo : public UInteractiveGizmo, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	// UInteractiveGizmo interface
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// IHoverBehaviorTarget interface
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override { }
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos);
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndHover();
	
	/**
	 * Set the Target to which the gizmo will be attached
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTarget(UTransformProxy* InTarget);

	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginDrag(const FInputDeviceRay& Ray);
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnUpdateDrag(const FInputDeviceRay& Ray);
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndDrag(const FInputDeviceRay& Ray);

	/** Check if the input Ray hit any of the components of the internal actor */
	INTERACTIVETOOLSFRAMEWORK_API bool HitTest(const FRay& Ray, FHitResult& OutHit, FVector& OutAxis, FTransform& OutTransform);

	/* Set the Radius of the Sphere*/
	INTERACTIVETOOLSFRAMEWORK_API void SetRadius(float InRadius);

	/** Called when the radius is chaged (by dragging or setting). Sends new radius as parameter. */
	TFunction<void(const float)> UpdateRadiusFunc = nullptr;

	// The error threshold for hit detection with the sphere
	UPROPERTY()
	float HitErrorThreshold{ 12.f };

	// The text that will be used as the transaction description for undo/redo
	UPROPERTY()
	FText TransactionDescription;

private:

	// Check if the Circle represented by CircleNormal centered at the gizmos location has been hit by the Ray
	INTERACTIVETOOLSFRAMEWORK_API bool CheckCircleIntersection(const FRay& Ray, FVector CircleNormal, FVector& OutHitLocation, FVector& OutHitAxis);
	
	// The radius of the sphere
	UPROPERTY()
	float Radius;

	// Whether the sphere is currently being hovered over
	UPROPERTY()
	bool bIsHovering{ false };

	// Whether the sphere is currently being dragged
	UPROPERTY()
	bool bIsDragging{ false };

	UPROPERTY()
	TObjectPtr<UTransformProxy> ActiveTarget;

	// The current axis that is being dragged along
	UPROPERTY()
	FVector ActiveAxis;

	// The position the drag was started on
	UPROPERTY()
	FVector DragStartWorldPosition;

	// The position the drag is on currently (projected onto the line it is being dragged along)
	UPROPERTY()
	FVector DragCurrentPositionProjected;

	// The initial parameter along the drag axist
	UPROPERTY()
	float InteractionStartParameter;
};


/**
 * A behavior that forwards clicking and dragging to the gizmo.
 */
UCLASS(MinimalAPI)
class UScalableSphereGizmoInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	virtual FInputCapturePriority GetPriority() override { return FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY); }

	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(UScalableSphereGizmo* Gizmo);

	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& input) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& input, EInputCaptureSide eSide) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& input, const FInputCaptureData& data) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& data) override;

protected:
	UScalableSphereGizmo* Gizmo;
	FRay LastWorldRay;
	FVector2D LastScreenPosition;
	bool bInputDragCaptured;
};
