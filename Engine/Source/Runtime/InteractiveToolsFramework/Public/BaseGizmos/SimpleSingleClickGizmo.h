// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h" // IClickBehaviorTarget, IHoverBehaviorTarget
#include "InteractiveGizmo.h"

#include "SimpleSingleClickGizmo.generated.h"

class IGizmoClickTarget;
class UPrimitiveComponent;
class USingleClickInputBehavior;

/**
 * Simple gizmo that triggers an OnClicked callback when it is clicked.
 */
UCLASS(MinimalAPI)
class USimpleSingleClickGizmo : public UInteractiveGizmo, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	
	INTERACTIVETOOLSFRAMEWORK_API bool InitializeWithComponent(UPrimitiveComponent* ComponentIn);
	
	// UInteractiveGizmo
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;

	// IClickBehaviorTarget
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndHover() override;

public:

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnClick, USimpleSingleClickGizmo&, const FInputDeviceRay&);
	FOnClick OnClick;

	/** The HitTarget provides a hit-test against some 3D element (presumably a visual widget) that controls when interaction can start */
	UPROPERTY()
	TScriptInterface<IGizmoClickTarget> HitTarget;

	/** The mouse click behavior of the gizmo is accessible so that it can be modified to use different mouse keys. */
	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickBehavior;

};