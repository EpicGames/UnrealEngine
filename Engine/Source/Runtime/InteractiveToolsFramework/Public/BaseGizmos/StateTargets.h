// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoInterfaces.h"
#include "Changes/TransformChange.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveToolChange.h"
#include "InteractiveToolManager.h"
#include "Internationalization/Text.h"
#include "Math/Transform.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "StateTargets.generated.h"


/**
 * UGizmoNilStateTarget is an implementation of IGizmoStateTarget that does nothing
 */
UCLASS(MinimalAPI)
class UGizmoNilStateTarget : public UObject, public IGizmoStateTarget
{
	GENERATED_BODY()
public:
	virtual void BeginUpdate() final
	{
	}

	virtual void EndUpdate() final
	{
	}

	virtual void CancelUpdate() final
	{
	}
};



/**
 * UGizmoLambdaStateTarget is an implementation of IGizmoStateTarget that forwards
 * calls to its interface functions to external TFunctions
 */
UCLASS(MinimalAPI)
class UGizmoLambdaStateTarget : public UObject, public IGizmoStateTarget
{
	GENERATED_BODY()
public:
	virtual void BeginUpdate()
	{
		if (BeginUpdateFunction)
		{
			BeginUpdateFunction();
		}
	}

	virtual void EndUpdate()
	{
		if (EndUpdateFunction)
		{
			EndUpdateFunction();
		}
	}
	
	virtual void CancelUpdate()
	{
		if (CancelUpdateFunction)
		{
			CancelUpdateFunction();
		}
	}

	TUniqueFunction<void(void)> BeginUpdateFunction = TUniqueFunction<void(void)>();
	TUniqueFunction<void(void)> EndUpdateFunction = TUniqueFunction<void(void)>();
	TUniqueFunction<void(void)> CancelUpdateFunction = TUniqueFunction<void(void)>();
};



/**
 * UGizmoObjectModifyStateTarget is an implementation of IGizmoStateTarget that 
 * opens and closes change transactions on a target UObject via a GizmoManager.
 */
UCLASS(MinimalAPI)
class UGizmoObjectModifyStateTarget : public UObject, public IGizmoStateTarget
{
	GENERATED_BODY()
public:
	virtual void BeginUpdate()
	{
		if (TransactionManager)
		{
			TransactionManager->BeginUndoTransaction(TransactionDescription);
		}
		if (ModifyObject.IsValid())
		{
			ModifyObject->Modify();
		}
	}

	virtual void EndUpdate()
	{
		if (TransactionManager)
		{
			TransactionManager->EndUndoTransaction();
		}
	}
	
	virtual void CancelUpdate() override
	{
		if (TransactionManager)
		{
			TransactionManager->CancelUndoTransaction();
		}
	}


	/**
	 * The object that will be changed, ie have Modify() called on it on BeginUpdate()
	 */
	TWeakObjectPtr<UObject> ModifyObject;

	/**
	 * Localized text description of the transaction (will be visible in Editor on undo/redo)
	 */
	FText TransactionDescription;

	/**
	 * Pointer to the GizmoManager or ToolManager that is used to open/close the transaction
	 */
	UPROPERTY()
	TScriptInterface<IToolContextTransactionProvider> TransactionManager;

public:
	/**
	 * Create and initialize an standard instance of UGizmoObjectModifyStateTarget
	 * @param ModifyObjectIn the object this StateTarget will call Modify() on
	 * @param DescriptionIn Localized text description of the transaction
	 * @param GizmoManagerIn pointer to the GizmoManager used to manage transactions
	 */
	static UGizmoObjectModifyStateTarget* Construct(
		UObject* ModifyObjectIn,
		const FText& DescriptionIn,
		IToolContextTransactionProvider* TransactionManagerIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoObjectModifyStateTarget* NewTarget = NewObject<UGizmoObjectModifyStateTarget>(Outer);
		NewTarget->ModifyObject = ModifyObjectIn;
		NewTarget->TransactionDescription = DescriptionIn;

		// have to explicitly configure this because we only have IToolContextTransactionProvider pointer
		NewTarget->TransactionManager.SetInterface(TransactionManagerIn);
		NewTarget->TransactionManager.SetObject(CastChecked<UObject>(TransactionManagerIn));

		return NewTarget;
	}
};







/**
 * UGizmoTransformChangeStateTarget is an implementation of IGizmoStateTarget that
 * emits an FComponentWorldTransformChange on a target USceneComponent. This StateTarget
 * also opens/closes an undo transaction via GizmoManager.
 *
 * The DependentChangeSources and ExternalDependentChangeSources lists allow additional
 * FChange objects to be inserted into the transaction, provided by IToolCommandChangeSource implementations.
 */
UCLASS(MinimalAPI)
class UGizmoTransformChangeStateTarget : public UObject, public IGizmoStateTarget
{
	GENERATED_BODY()
public:
	virtual void BeginUpdate()
	{
		if (TargetComponent.IsValid())
		{
			if (TransactionManager)
			{
				TransactionManager->BeginUndoTransaction(ChangeDescription);
			}

			InitialTransform = TargetComponent->GetComponentTransform();

			for (TUniquePtr<IToolCommandChangeSource>& Source : DependentChangeSources)
			{
				Source->BeginChange();
			}

			for (IToolCommandChangeSource* Source : ExternalDependentChangeSources)
			{
				Source->BeginChange();
			}
		}
	}

	virtual void EndUpdate()
	{
		if (TargetComponent.IsValid())
		{
			FinalTransform = TargetComponent->GetComponentTransform();

			if (TransactionManager)
			{
				TUniquePtr<FComponentWorldTransformChange> TransformChange 
					= MakeUnique<FComponentWorldTransformChange>(InitialTransform, FinalTransform);
				TransactionManager->EmitObjectChange(TargetComponent.Get(), MoveTemp(TransformChange), ChangeDescription);

				for (TUniquePtr<IToolCommandChangeSource>& Source : DependentChangeSources)
				{
					TUniquePtr<FToolCommandChange> Change = Source->EndChange();
					if (Change)
					{
						UObject* Target = Source->GetChangeTarget();
						TransactionManager->EmitObjectChange(Target, MoveTemp(Change), Source->GetChangeDescription());
					}
				}

				for (IToolCommandChangeSource* Source : ExternalDependentChangeSources)
				{
					TUniquePtr<FToolCommandChange> Change = Source->EndChange();
					if (Change)
					{
						UObject* Target = Source->GetChangeTarget();
						TransactionManager->EmitObjectChange(Target, MoveTemp(Change), Source->GetChangeDescription());
					}
				}

				TransactionManager->EndUndoTransaction();
			}
		}
	}
	
	virtual void CancelUpdate() override
	{
		if (TargetComponent.IsValid() && TransactionManager)
		{
			FinalTransform = InitialTransform;

			for (TUniquePtr<IToolCommandChangeSource>& Source : DependentChangeSources)
			{
				Source->CancelChange();
			}

			for (IToolCommandChangeSource* Source : ExternalDependentChangeSources)
			{
				Source->CancelChange();
			}

			TransactionManager->CancelUndoTransaction();
		}
	}


	/**
	 * The object that will be changed, ie have Modify() called on it on BeginUpdate()
	 */
	TWeakObjectPtr<USceneComponent> TargetComponent;

	/**
	 * Localized text description of the transaction (will be visible in Editor on undo/redo)
	 */
	FText ChangeDescription;

	/**
	 * Pointer to the GizmoManager or ToolManager that is used to open/close the transaction
	 */
	UPROPERTY()
	TScriptInterface<IToolContextTransactionProvider> TransactionManager;

	/** Start Transform, saved on BeginUpdate() */
	FTransform InitialTransform;
	/** End Transform, saved on EndUpdate() */
	FTransform FinalTransform;


	/** 
	 * Dependent-change generators. This will be told about update start/end, and any generated changes will also be emitted. 
	 * This allows (eg) TransformProxy change events to be collected at the same time as changes to a gizmo target component.
	 */
	TArray<TUniquePtr<IToolCommandChangeSource>> DependentChangeSources;

	/**
	 * Dependent-change generators that are not owned by this class, otherwise handled identically to DependentChangeSources
	 */
	TArray<IToolCommandChangeSource*> ExternalDependentChangeSources;

public:

	/**
	 * Create and initialize an standard instance of UGizmoTransformChangeStateTarget
	 * @param TargetComponentIn the USceneComponent this StateTarget will track transform changes on
	 * @param DescriptionIn Localized text description of the transaction
	 * @param TransactionManagerIn pointer to the GizmoManager or ToolManager used to manage transactions
	 */
	static UGizmoTransformChangeStateTarget* Construct(
		USceneComponent* TargetComponentIn,
		const FText& DescriptionIn,
		IToolContextTransactionProvider* TransactionManagerIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoTransformChangeStateTarget* NewTarget = NewObject<UGizmoTransformChangeStateTarget>(Outer);
		NewTarget->TargetComponent = TargetComponentIn;
		NewTarget->ChangeDescription = DescriptionIn;

		// have to explicitly configure this because we only have IToolContextTransactionProvider pointer
		NewTarget->TransactionManager.SetInterface(TransactionManagerIn);
		NewTarget->TransactionManager.SetObject(CastChecked<UObject>(TransactionManagerIn));
		
		return NewTarget;
	}


};

