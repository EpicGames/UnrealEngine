// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveGizmo.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "TransformGizmoUtil.generated.h"

class UInteractiveToolsContext;
class UInteractiveToolManager;
class UInteractiveGizmoManager;
class FCombinedTransformGizmoActorFactory;


namespace UE
{
namespace TransformGizmoUtil
{
	//
	// The functions below are helper functions that simplify usage of a UCombinedTransformGizmoContextObject 
	// that is registered as a ContextStoreObject in an InteractiveToolsContext
	//

	/**
	 * If one does not already exist, create a new instance of UCombinedTransformGizmoContextObject and add it to the
	 * ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore now has a UCombinedTransformGizmoContextObject (whether it already existed, or was created)
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool RegisterTransformGizmoContextObject(UInteractiveToolsContext* ToolsContext);

	/**
	 * Remove any existing UCombinedTransformGizmoContextObject from the ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore no longer has a UCombinedTransformGizmoContextObject (whether it was removed, or did not exist)
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool DeregisterTransformGizmoContextObject(UInteractiveToolsContext* ToolsContext);


	/**
	 * Spawn a new standard 3-axis Transform gizmo (see UCombinedTransformGizmoContextObject::Create3AxisTransformGizmo for details)
	 * GizmoManager's ToolsContext must have a UCombinedTransformGizmoContextObject registered (see UCombinedTransformGizmoContextObject for details)
	 */
	INTERACTIVETOOLSFRAMEWORK_API UCombinedTransformGizmo* Create3AxisTransformGizmo(UInteractiveGizmoManager* GizmoManager, void* Owner = nullptr, const FString& InstanceIdentifier = FString());
	/**
	 * Spawn a new standard 3-axis Transform gizmo (see UCombinedTransformGizmoContextObject::Create3AxisTransformGizmo for details)
	 * ToolManager's ToolsContext must have a UCombinedTransformGizmoContextObject registered (see UCombinedTransformGizmoContextObject for details)
	 */
	INTERACTIVETOOLSFRAMEWORK_API UCombinedTransformGizmo* Create3AxisTransformGizmo(UInteractiveToolManager* ToolManager, void* Owner = nullptr, const FString& InstanceIdentifier = FString());


	/**
	 * Spawn a new custom Transform gizmo (see UCombinedTransformGizmoContextObject::CreateCustomTransformGizmo for details)
	 * GizmoManager's ToolsContext must have a UCombinedTransformGizmoContextObject registered (see UCombinedTransformGizmoContextObject for details)
	 */
	INTERACTIVETOOLSFRAMEWORK_API UCombinedTransformGizmo* CreateCustomTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());
	/**
	 * Spawn a new custom Transform gizmo (see UCombinedTransformGizmoContextObject::CreateCustomTransformGizmo for details)
	 * ToolManager's ToolsContext must have a UCombinedTransformGizmoContextObject registered (see UCombinedTransformGizmoContextObject for details)
	 */
	INTERACTIVETOOLSFRAMEWORK_API UCombinedTransformGizmo* CreateCustomTransformGizmo(UInteractiveToolManager* ToolManager, ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());


	/**
	 * Spawn a new custom Transform gizmo (see UCombinedTransformGizmoContextObject::CreateCustomTransformGizmo for details)
	 * GizmoManager's ToolsContext must have a UCombinedTransformGizmoContextObject registered (see UCombinedTransformGizmoContextObject for details)
	 */
	INTERACTIVETOOLSFRAMEWORK_API UCombinedTransformGizmo* CreateCustomRepositionableTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());
	/**
	 * Spawn a new custom Transform gizmo (see UCombinedTransformGizmoContextObject::CreateCustomRepositionableTransformGizmo for details)
	 * ToolManager's ToolsContext must have a UCombinedTransformGizmoContextObject registered (see UCombinedTransformGizmoContextObject for details)
	 */
	INTERACTIVETOOLSFRAMEWORK_API UCombinedTransformGizmo* CreateCustomRepositionableTransformGizmo(UInteractiveToolManager* ToolManager, ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());
}
}


/**
 * UCombinedTransformGizmoContextObject is a utility object that registers a set of Gizmo Builders
 * for UCombinedTransformGizmo and variants. The intended usage is to call RegisterGizmosWithManager(),
 * and then the UCombinedTransformGizmoContextObject will register itself as a ContextObject in the
 * InteractiveToolsContext's ContextObjectStore. Then the Create3AxisTransformGizmo()/etc functions
 * will spawn different variants of UCombinedTransformGizmo. The above UE::TransformGizmoUtil:: functions
 * will look up the UCombinedTransformGizmoContextObject instance in the ContextObjectStore and then
 * call the associated function below.
 */
UCLASS(Transient, MinimalAPI)
class UCombinedTransformGizmoContextObject : public UObject
{
	GENERATED_BODY()
public:

public:
	// builder identifiers for default gizmo types. Perhaps should have an API for this...
	static INTERACTIVETOOLSFRAMEWORK_API const FString DefaultAxisPositionBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString DefaultPlanePositionBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString DefaultAxisAngleBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString DefaultThreeAxisTransformBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString CustomThreeAxisTransformBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString CustomRepositionableThreeAxisTransformBuilderIdentifier;

	INTERACTIVETOOLSFRAMEWORK_API void RegisterGizmosWithManager(UInteractiveToolManager* ToolManager);
	INTERACTIVETOOLSFRAMEWORK_API void DeregisterGizmosWithManager(UInteractiveToolManager* ToolManager);

	/**
	 * Activate a new instance of the default 3-axis transformation Gizmo. RegisterDefaultGizmos() must have been called first.
	 * @param Owner optional void pointer to whatever "owns" this Gizmo. Allows Gizmo to later be deleted using DestroyAllGizmosByOwner()
	 * @param InstanceIdentifier optional client-defined *unique* string that can be used to locate this instance
	 * @return new Gizmo instance that has been created and initialized
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UCombinedTransformGizmo* Create3AxisTransformGizmo(UInteractiveGizmoManager* GizmoManager, void* Owner = nullptr, const FString& InstanceIdentifier = FString());

	/**
	 * Activate a new customized instance of the default 3-axis transformation Gizmo, with only certain elements included. RegisterDefaultGizmos() must have been called first.
	 * @param Elements flags that indicate which standard gizmo sub-elements should be included
	 * @param Owner optional void pointer to whatever "owns" this Gizmo. Allows Gizmo to later be deleted using DestroyAllGizmosByOwner()
	 * @param InstanceIdentifier optional client-defined *unique* string that can be used to locate this instance
	 * @return new Gizmo instance that has been created and initialized
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UCombinedTransformGizmo* CreateCustomTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());

	/**
	 * Variant of CreateCustomTransformGizmo that creates a URepositionableTransformGizmo, which is an extension to UCombinedTransformGizmo that 
	 * supports various snapping interactions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UCombinedTransformGizmo* CreateCustomRepositionableTransformGizmo(UInteractiveGizmoManager* GizmoManager, ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmoCreated, UCombinedTransformGizmo*);
	FOnGizmoCreated OnGizmoCreated;

protected:
	TSharedPtr<FCombinedTransformGizmoActorFactory> GizmoActorBuilder;
	bool bDefaultGizmosRegistered = false;
};
