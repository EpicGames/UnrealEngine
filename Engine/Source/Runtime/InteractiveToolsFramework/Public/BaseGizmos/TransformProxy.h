// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/StateTargets.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "InteractiveToolChange.h"
#include "Internationalization/Text.h"
#include "Math/Transform.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "TransformProxy.generated.h"

class USceneComponent;

/**
 * UTransformProxy is used to transform a set of sub-objects. An internal
 * FTransform is generated based on the sub-object set, and the relative
 * FTransform of each sub-object is stored. Then as this main transform
 * is updated, the sub-objects are also updated.
 * 
 * Currently only USceneComponent sub-objects are supported.
 * 
 * If only one sub-object is set, the main transform is the sub-object transform.
 * Otherwise the main transform is centered at the average origin and
 * has no rotation.
 */
UCLASS(Transient, MinimalAPI)
class UTransformProxy : public UObject
{
	GENERATED_BODY()
public:

	/**
	 * Add a component sub-object to the proxy set. 
	 * @param bModifyComponentOnTransform if true, Component->Modify() is called before the Component transform is updated
	 * @warning The internal shared transform is regenerated each time a component is added
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddComponent(USceneComponent* Component, bool bModifyComponentOnTransform = true);

	/**
	 * Add a component sub-object to the proxy set with custom transform access functions. This can be used
	 * to do things like add an ISM Instance to the TransformProxy.
	 * @param GetTransformFunc return current transform
	 * @param SetTransformFunc set current transform
	 * @param UserDefinedIndex an arbitrary integer that can be provided, not currently used but may be useful in subclasses
	 * @param bModifyComponentOnTransform if true, Component->Modify() is called before the Component transform is updated
	 * @warning The internal shared transform is regenerated each time a component is added
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddComponentCustom(
		USceneComponent* Component, 
		TUniqueFunction<FTransform(void)> GetTransformFunc,
		TUniqueFunction<void(const FTransform&)> SetTransformFunc,
		int64 UserDefinedIndex = 0,
		bool bModifyComponentOnTransform = true);


	/**
	 * @return the shared transform for all the sub-objects
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetTransform() const;

	/**
	 * Update the main transform and then update the sub-objects based on their relative transformations
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransform(const FTransform& Transform);


	/**
	 * In some use cases SetTransform() will be called repeatedly (eg during an interactive gizmo edit). External
	 * code may know and/or need to know when such a sequence starts/ends. The OnBeginTransformEdit/OnEndTransformEdit delegates
	 * can provide these notifications, however client code must call BeginTransformEditSequence()/EndTransformEditSequence() to fire those delegates 
	 * as the TransformProxy doesn't know about this external state. 
	 * @note FTransformProxyChangeSource will call these functions on Begin/End
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginTransformEditSequence();

	/**
	 * External clients should call this when done a sequence of SetTransform calls (see BeginTransformEditSequence)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndTransformEditSequence();

	/**
	 * Clients should call this before a sequence of SetTransform calls that have bSetPivotMode as true
	 * (see comment in BeginTransformEditSequence).
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginPivotEditSequence();

	/**
	 * Clients should call this when done with a sequence of SetTransform that have bSetPivotMode as true 
	 * (see comment in BeginTransformEditSequence).
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndPivotEditSequence();

public:
	/**
	 * This delegate is fired whenever the transform changes in a way that updates the contained components. I.e.,
	 * SetTransform is called with bSetPivotMode being false.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTransformChanged, UTransformProxy*, FTransform);
	FOnTransformChanged OnTransformChanged;

	/**
	 * This delegate is fired whenever the transform is changed by a FTransformProxyChange, ie on undo/redo.
	 * OnTransformChanged is also fired in those cases, this extra event can be used when undo/redo state changes need to be handled specifically.
	 */
	FOnTransformChanged OnTransformChangedUndoRedo;

	/** This delegate is fired when BeginTransformEditSequence() is called to indicate that a sequence of transform updates has started */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBeginTransformEdit, UTransformProxy*);
	FOnBeginTransformEdit OnBeginTransformEdit;

	/** This delegate is fired when EndTransformEditSequence() is called to indicate that a sequence of transform updates has ended */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndTransformEdit, UTransformProxy*);
	FOnEndTransformEdit OnEndTransformEdit;

	/**
	 * This delegate is fired whenever the internal transform changes due to a pivot reposition, ie on AddComponent and
	 * when SetTransform is called with bSetPivotMode being true.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPivotChanged, UTransformProxy*, FTransform);
	FOnPivotChanged OnPivotChanged;

	/** This delegate is fired when BeginTransformEditSequence() is called to indicate that a sequence of pivot updates has started */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBeginPivotEdit, UTransformProxy*);
	FOnBeginPivotEdit OnBeginPivotEdit;

	/** This delegate is fired when EndTransformEditSequence() is called to indicate that a sequence of pivot updates has ended */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndPivotEdit, UTransformProxy*);
	FOnEndPivotEdit OnEndPivotEdit;

	/**
	 * If true, relative rotation of shared transform is applied to objects before relative translation (ie they rotate in place)
	 */
	UPROPERTY()
	bool bRotatePerObject = false;

	/**
	 * If true, then on SetTransform() the components are not moved, and their local transforms are recalculated
	 */
	UPROPERTY()
	bool bSetPivotMode = false;

	
protected:

	struct FRelativeObject
	{
		TWeakObjectPtr<USceneComponent> Component;
		bool bModifyComponentOnTransform;

		int64 UserDefinedIndex;		// no current purpose, reserved for subclass/future use

		TUniqueFunction<FTransform(void)> GetTransformFunc;
		TUniqueFunction<void(const FTransform&)> SetTransformFunc;

		/** The initial transform of the object, set during UpdateSharedTransform() */
		FTransform StartTransform;
		/** The transform of the object relative to */
		FTransform RelativeTransform;
	};

	/** List of sub-objects */
	TArray<FRelativeObject> Objects;

	/** The main transform */
	UPROPERTY()
	FTransform SharedTransform;

	/** The main transform */
	UPROPERTY()
	FTransform InitialSharedTransform;

	/** Recalculate main SharedTransform when object set changes*/
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateSharedTransform();

	/** Recalculate per-object relative transforms */
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateObjectTransforms();

	/** Propagate a transform update to the sub-objects */
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateObjects();
};



/**
 * FTransformProxyChange tracks a change to the base transform for a TransformProxy
 */
class FTransformProxyChange : public FToolCommandChange
{
public:
	FTransform From;
	FTransform To;
	bool bSetPivotMode = false;

	INTERACTIVETOOLSFRAMEWORK_API virtual void Apply(UObject* Object) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Revert(UObject* Object) override;

	virtual FString ToString() const override { return TEXT("FTransformProxyChange"); }
};

/**
 * FTransformProxyChangeSource generates FTransformProxyChange instances on Begin/End.
 * Instances of this class can (for example) be attached to a UGizmoTransformChangeStateTarget for use TransformGizmo change tracking.
 */
class FTransformProxyChangeSource : public IToolCommandChangeSource
{
public:
	FTransformProxyChangeSource(UTransformProxy* ProxyIn)
	{
		Proxy = ProxyIn;
	}

	virtual ~FTransformProxyChangeSource() = default;

	TWeakObjectPtr<UTransformProxy> Proxy;
	TUniquePtr<FTransformProxyChange> ActiveChange;

	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginChange() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual TUniquePtr<FToolCommandChange> EndChange() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual UObject* GetChangeTarget() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FText GetChangeDescription() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void CancelChange() override;

	/**
	 * If true, the emitted changes will always have bSetPivotMode set to true, regardless of
	 * the current proxy settings. This is meant to accompany a UGizmoTransformProxyTransformSource 
	 * that has bOverrideSetPivotMode set to true, used for gizmos that reposition a proxy that
	 * otherwise behaves normally.
	 */
	bool bOverrideSetPivotMode = false;
};
