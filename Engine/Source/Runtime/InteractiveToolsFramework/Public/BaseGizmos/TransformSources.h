// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseGizmos/TransformProxy.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "TransformSources.generated.h"

class USceneComponent;
class UTransformProxy;



/**
 * UGizmoBaseTransformSource is a base implementation of IGizmoTransformSource that 
 * adds an OnTransformChanged delegate. This class cannot be used directly and must be subclassed.
 */
UCLASS(MinimalAPI)
class UGizmoBaseTransformSource : public UObject, public IGizmoTransformSource
{
	GENERATED_BODY()
public:
	virtual FTransform GetTransform() const
	{
		return FTransform::Identity;
	}

	virtual void SetTransform(const FTransform& NewTransform)
	{
		check(false);   // not implemented
	}

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmoTransformSourceChanged, IGizmoTransformSource*);
	FOnGizmoTransformSourceChanged OnTransformChanged;
};




/**
 * UGizmoComponentWorldTransformSource implements IGizmoTransformSource (via UGizmoBaseTransformSource)
 * based on the internal transform of a USceneComponent.
 */
UCLASS(MinimalAPI)
class UGizmoComponentWorldTransformSource : public UGizmoBaseTransformSource
{
	GENERATED_BODY()
public:

	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetTransform() const override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransform(const FTransform& NewTransform) override;

	UPROPERTY()
	TObjectPtr<USceneComponent> Component;

	/** If true, Component->Modify() is called on SetTransform */
	UPROPERTY()
	bool bModifyComponentOnTransform = true;


public:
	/**
	 * Construct a default instance of UGizmoComponentWorldTransformSource with the given Component
	 */
	static UGizmoComponentWorldTransformSource* Construct(
		USceneComponent* Component,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoComponentWorldTransformSource* NewSource = NewObject<UGizmoComponentWorldTransformSource>(Outer);
		NewSource->Component = Component;
		return NewSource;
	}
};


/**
 * @deprecated This class was used with UGizmoScaledTransformSource, but UGizmoScaledAndUnscaledTransformSources
 *  will likely handle any similar use cases in a cleaner way.
 *
 * FSeparateScaleProvider provides TFunction-implementable API that sets/gets a
 * Scaling Vector from an external source. 
 */
struct FSeparateScaleProvider
{
	TFunction<FVector(void)> GetScale = []() { return FVector::OneVector; };
	TFunction<void(FVector)> SetScale = [](FVector) {};
};


/**
 * @deprecated Use UGizmoScaledAndUnscaledTransformSources instead.
 *
 * Old description:
 * UGizmoScaledTransformSource wraps another IGizmoTransformSource implementation and adds a
 * separate scaling vector to the Transform. The main use of this class is to support scaling
 * in a 3D gizmo without actually scaling the Gizmo itself. Generally our pattern is to apply
 * the gizmo's position/rotation transform to the target object via a TransformProxy, but
 * that does not work with Scaling. So this class stores the scaling vector separately, provided by
 * an external source via FSeparateScaleProvider, and in GetTransform/SetTransform rewrites the
 * Transform from the child IGizmoTransformSource with the new scale.
 */
UCLASS(MinimalAPI)
class UGizmoScaledTransformSource : public UGizmoBaseTransformSource
{
	GENERATED_BODY()
public:

	/**
	 * IGizmoTransformSource implementation, returns child transform with local sclae
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetTransform() const override;

	/**
	 * IGizmoTransformSource implementation, removes scale and sends to ScaleProvider, then forwards remaining rotate+translate transform to child
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransform(const FTransform& NewTransform) override;

	/**
	 * Child transform source
	 */
	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> ChildTransformSource;

	/**
	 * Provider for external scale value/storage
	 */
	FSeparateScaleProvider ScaleProvider;

	/**
	 * Return the child transform with combined scale
	 */
	INTERACTIVETOOLSFRAMEWORK_API FTransform GetScaledTransform() const;

public:
	/**
	 * Construct a default instance of UGizmoComponentWorldTransformSource with the given Component
	 */
	static UGizmoScaledTransformSource* Construct(
		IGizmoTransformSource* ChildSource,
		FSeparateScaleProvider ScaleProviderIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoScaledTransformSource* NewSource = NewObject<UGizmoScaledTransformSource>(Outer);
		NewSource->ChildTransformSource = Cast<UObject>(ChildSource);
		NewSource->ScaleProvider = ScaleProviderIn;
		return NewSource;
	}
};






/**
 * UGizmoTransformProxyTransformSource implements IGizmoTransformSource (via UGizmoBaseTransformSource)
 * based on the internal transform of a UTransformProxy.
 */
UCLASS(MinimalAPI)
class UGizmoTransformProxyTransformSource : public UGizmoBaseTransformSource
{
	GENERATED_BODY()
public:

	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetTransform() const override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransform(const FTransform& NewTransform) override;

	UPROPERTY()
	TObjectPtr<UTransformProxy> Proxy;

	/**
	 * If true, the underlying proxy is modified with its SetPivotMode flag temporarily set to true.
	 * This allows the transform source to be used for proxy repositioning on a proxy that otherwise
	 * operates normally.
	 */
	bool bOverrideSetPivotMode = false;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGizmoTransformSourcePivotChanged, IGizmoTransformSource*);
	FOnGizmoTransformSourcePivotChanged OnPivotChanged;

public:
	/**
	 * Construct a default instance of UGizmoComponentWorldTransformSource with the given Proxy
	 */
	static UGizmoTransformProxyTransformSource* Construct(
		UTransformProxy* Proxy,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoTransformProxyTransformSource* NewSource = NewObject<UGizmoTransformProxyTransformSource>(Outer);
		NewSource->Proxy = Proxy;
		return NewSource;
	}
};


/**
 * A wrapper around two IGizmoTransformSource's that generally forwards transforms to/from its ScaledTransformSource,
 * but also forwards an unscaled version of the transform to UnscaledTransformSource on SetTransform calls.
 * This handles the common case of wanting to apply the entire transform to one IGizmoTransformSource, but only
 * the unscaled transform to a gizmo component (since we don't want to scale the gizmo component but do want to 
 * rotate/translate it).
 */
UCLASS(MinimalAPI)
class UGizmoScaledAndUnscaledTransformSources : public UGizmoBaseTransformSource
{
	GENERATED_BODY()
public:

	/** Gets the transform from ScaledTransformSource. */
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetTransform() const override;

	/** Calls SetTransform on ScaledTransformSource and passes the unscaled version to UnscaledTransformSource. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetTransform(const FTransform& NewTransform) override;

	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> ScaledTransformSource;

	UPROPERTY()
	TScriptInterface<IGizmoTransformSource> UnscaledTransformSource;

public:
	/**
	 * Constructs a UGizmoScaledAndUnscaledTransformSources by wrapping the provided IGizmoTransformSource as
	 * its scaled transform source, and the given gizmo in a UGizmoComponentWorldTransformSource as its
	 * unscaled transform source.
	 */
	static UGizmoScaledAndUnscaledTransformSources* Construct(
		IGizmoTransformSource* ScaledSource,
		USceneComponent* GizmoComponentIn,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoScaledAndUnscaledTransformSources* NewSource = NewObject<UGizmoScaledAndUnscaledTransformSources>(Outer);
		NewSource->ScaledTransformSource = Cast<UObject>(ScaledSource);

		UGizmoComponentWorldTransformSource* WrappedGizmo = UGizmoComponentWorldTransformSource::Construct(GizmoComponentIn, Outer);
		WrappedGizmo->bModifyComponentOnTransform = false;

		NewSource->UnscaledTransformSource = WrappedGizmo;
		return NewSource;
	}

	/**
	 * Constructs a UGizmoScaledAndUnscaledTransformSources around the given two IGizmoTransformSource's.
	 */
	static UGizmoScaledAndUnscaledTransformSources* Construct(
		IGizmoTransformSource* ScaledSource,
		IGizmoTransformSource* UnscaledSource,
		UObject* Outer = (UObject*)GetTransientPackage())
	{
		UGizmoScaledAndUnscaledTransformSources* NewSource = NewObject<UGizmoScaledAndUnscaledTransformSources>(Outer);
		NewSource->ScaledTransformSource = Cast<UObject>(ScaledSource);
		NewSource->UnscaledTransformSource = Cast<UObject>(UnscaledSource);
		return NewSource;
	}
};
