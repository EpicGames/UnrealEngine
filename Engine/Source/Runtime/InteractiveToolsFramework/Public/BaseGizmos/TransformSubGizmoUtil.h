// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "InteractiveGizmo.h"
#include "Math/Axis.h"
#include "Math/MathFwd.h" // FTransform
#include "Misc/Optional.h"

struct FLinearColor;
class FText;
class IToolContextTransactionProvider;
class UCombinedTransformGizmo;
class UGizmoScaledAndUnscaledTransformSources;
class UGizmoTransformChangeStateTarget;
class UGizmoConstantFrameAxisSource;
class UGizmoComponentAxisSource;
class UObject;
class UPrimitiveComponent;
class UStaticMesh;
class UTransformProxy;
class UViewAdjustedStaticMeshGizmoComponent;

/**
 * This file is meant for utilities that are specific to allowing gizmos to act as sub gizmos inside a
 *  UCombinedTransformGizmo.
 */

namespace UE::GizmoUtil
{
	/**
	 * Common parameters needed for initializing a sub gizmo. Used in several sub gizmo reinitialization
	 *  functions (such as UAxisPositionGizmo::InitializeAsTranslateGizmo).
	 */
	struct FTransformSubGizmoCommonParams
	{
		UPrimitiveComponent* Component = nullptr;
		UTransformProxy* TransformProxy = nullptr;
		EAxis::Type Axis = EAxis::None;

		// If true, we use the axis of the root gizmo component. If false, we use an axis of the
		//  component itself, which is determined by its transform. Does not attempt to use shared
		//  state for axis if false.
		bool bAxisIsBasedOnRootComponent = true;

		// Optional parameters:

		// Default transaction name and provider are used if not provided. Not used if one is gotten from
		//  shared state.
		TOptional<FText> TransactionName;
		IToolContextTransactionProvider* TransactionProvider = nullptr;

		// Transient package is used if custom outer is not provided
		UObject* OuterForSubobjects = nullptr;
		
		// If true, the gizmo moves the parent root component rather than just its own component, like
		//  the standard TRS sub gizmos do. If false, the gizmo moves just its component, leaving the
		//  parent in place. Does not attempt to use shared state for transform source or state target
		//  if false.
		bool bManipulatesRootComponent = true;
		
		//~ The default hover behavior is to forward the hover information to the component via the
		//~  IGizmoBaseComponentInterface. We could give the user a way to place a custom hover function
		//~  like we have in deprecated initialization methods, but that is unclear to be worth it, as 
		//~  it is likely to be cleaner to implement the interface on any custom components.
		//~TFunction<void(bool)> CustomUpdateHoverFunction;

		// Returns the Axis parameter as an index in the range [0,2] where 0 
		// corresponds to X (or None), 1 corresponds to Y, and 2 corresponds to Z.
		int GetClampedAxisIndex() const
		{
			return (Axis == EAxis::Y) ? 1
				: (Axis == EAxis::Z) ? 2
				: 0;
		}
	};

	/**
	 * A struct to hold some objects that can be reused across multiple sub gizmos that operate on the same
	 *  compound gizmo, to avoid creating redundant objects. For instance, the transform source can be the
	 *  same for the sub gizmos that manipulate the same overall TRS gizmo, so the first sub gizmo would
	 *  create that transform source, and the subsequent ones will reuse it.
	 * 
	 * Note: While the initialization functions will try to do reasonable things with this shared state
	 *  (for instance, not attempt to use shared state for the axis source if the axis source is marked
	 *  as being based off of this component, rather than the root), it is still up to the user to decide
	 *  whether shared state should be an option. For instance, if your sub gizmo manipulates the root as
	 *  normal but for some reason targets a different TransformProxy than other subgizmos, then it
	 *  shouldn't share a TransformSource or StateTarget with the other gizmos, so either the shared state
	 *  struct should not be used (preferable), or those properties in the struct should be nulled out
	 *  before/after the call.
	 * In general, if your sub gizmo is part of the overall TRS gizmo itself (i.e. represents one of its elements),
	 *  then using the shared state struct will make sense, and will save you a few redundant object creations.
	 *  If your sub gizmo is just attached to the TRS gizmo but moves independently, then it may not be safe to
	 *  use the shared state struct, and may not be worth the potential trouble even if you could save an object.
	 */
	struct FTransformSubGizmoSharedState
	{
		UGizmoScaledAndUnscaledTransformSources* TransformSource = nullptr;
		UGizmoTransformChangeStateTarget* StateTarget = nullptr;
		UGizmoConstantFrameAxisSource* CameraAxisSource = nullptr;
		UGizmoComponentAxisSource* CardinalAxisSources[3] = { 0 };
		UGizmoComponentAxisSource* UnitCardinalAxisSources[3] = { 0 };
	};

	/**
	 * Rotates a transform such that its basis still lies along the cardinal axes but rotated in such a 
	 *  way that the given axis is in the X direction. For example, if you have a transform that makes a
	 *  component work well as an X axis in a TRS gizmo, using this function with AxisToBeX set to Y will
	 *  rotate the transform to work well as the Y axis, because the basis will have been rotated to be
	 *  YZX.
	 */
	INTERACTIVETOOLSFRAMEWORK_API FTransform GetRotatedBasisTransform(const FTransform& TransformIn, EAxis::Type AxisToBeX);

	// Parameter struct for InitializeSubGizmoElementsWithMesh
	struct FInitMeshSubGizmoParams
	{
		// Required parameters:

		UCombinedTransformGizmo* ParentGizmo = nullptr;
		// If there are multiple elements, the same mesh will be set for all of them. This makes
		//  it easy to, for example, set all of the arrow components at once.
		ETransformGizmoSubElements Elements = ETransformGizmoSubElements::None;
		UStaticMesh* Mesh = nullptr;
		
		// Optional parameters:

		FTransform ComponentToGizmo = FTransform::Identity;

		// When true, ComponentToGizmo is adjusted such that the gizmo basis points along
		//  the relevant element axis. This allows ComponentToGizmo to be specified once
		//  for the x axis case, and be reused for the y/z axes with the proper rotation.
		bool bRotateTransformBasisBasedOnElement = true;

		// When true, the sub gizmo is mirrored across the gizmo origin depending on where
		//  in relation to the gizmo the camera is positioned.
		bool bMirrorBasedOnOctant = false;

		// Leaving this unset causes the color to be determined by axis.
		TOptional<FLinearColor> Color;

		// Mesh to swap in when the user is dragging the gizmo.
		UStaticMesh* SubstituteInteractionMesh = nullptr;
		// Only used when SubstituteInteractionMesh is being set
		FTransform SubstituteMeshToComponent = FTransform::Identity;
	};

	/**
	 * Given a UCombinedTransformGizmo, swaps selected elements with custom meshes.
	 * 
	 * @param ComponentsOut Optional output array of newly created gizmo components, as pairs with the
	 *  single element that they were created for.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void InitializeSubGizmoElementsWithMesh(const FInitMeshSubGizmoParams& Params, 
		TArray<TPair<ETransformGizmoSubElements, UViewAdjustedStaticMeshGizmoComponent*>>* ElementComponentsOut = nullptr);

}
