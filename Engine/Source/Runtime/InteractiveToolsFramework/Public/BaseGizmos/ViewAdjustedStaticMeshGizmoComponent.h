// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StaticMeshComponent.h"
#include "BaseGizmos/GizmoBaseComponent.h"

#include "ViewAdjustedStaticMeshGizmoComponent.generated.h"

#define UE_API INTERACTIVETOOLSFRAMEWORK_API

class UGizmoViewContext;
namespace UE::GizmoRenderingUtil
{
	class IViewBasedTransformAdjuster;
	class ISceneViewInterface;
}

/**
 * Version of a static mesh component that only takes the dynamic draw path and has the ability to
 * adjust the transform based on view information.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, ClassGroup = Gizmos, meta = (BlueprintSpawnableComponent))
class UViewAdjustedStaticMeshGizmoComponent : public UStaticMeshComponent
	, public IGizmoBaseComponentInterface
{
	GENERATED_BODY()
public:
	/**
	 * The gizmo view context is needed to be able to line trace the component, since its collision data
	 *  needs updating based on view.
	 */
	void SetGizmoViewContext(UGizmoViewContext* GizmoViewContextIn) { GizmoViewContext = GizmoViewContextIn; }
	
	UE_API void SetTransformAdjuster(TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> Adjuster);
	TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> GetTransformAdjuster() const { return TransformAdjuster; }

	/**
	 * The render visibility function is an optional function that can hide the component based on view
	 *  information (for instance to hide an arrow gizmo when looking directly down the arrow). It must 
	 *  be callable from both the game and the render threads (for line traces and rendering).
	 */
	UE_API void SetRenderVisibilityFunction(TFunction<bool(const UE::GizmoRenderingUtil::ISceneViewInterface& View, 
		const FTransform& ComponentToWorld)> RenderVisibilityFunc);
	TFunction<bool(const UE::GizmoRenderingUtil::ISceneViewInterface& View, 
		const FTransform& ComponentToWorld)> GetRenderVisibilityFunction() const { return RenderVisibilityFunc; }
	
	/** Helper method that just sets the same material in all slots. Does not include hover override material. */
	UE_API void SetAllMaterials(UMaterialInterface* Material);

	/**
	 * Sets a material that will override all material slots whenever the component is told that is being
	 *  hovered (via UpdateHoverState).
	 */
	UE_API void SetHoverOverrideMaterial(UMaterialInterface* Material);

	UMaterialInterface* GetHoverOverrideMaterial() { return HoverOverrideMaterial; }
	bool IsBeingHovered() { return bHovered; }

	/** 
	 * Sets a mesh that is swapped in while the component is being interacted with. This is done by not rendering this 
	 *  component and making the substitute component visible.
	 */
	UE_API void SetSubstituteInteractionComponent(UPrimitiveComponent* Component, const FTransform& RelativeTransform = FTransform::Identity);
	
	bool IsHiddenByInteraction() { return bInteracted && SubstituteInteractionComponent; }

	// IGizmoBaseComponentInterface
	UE_API virtual void UpdateHoverState(bool bHoveringIn) override;
	UE_API virtual void UpdateWorldLocalState(bool bWorldIn) override;
	UE_API virtual void UpdateInteractingState(bool bInteracting) override;

	// UMeshComponent
	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	UE_API virtual FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const override;
	UE_API virtual FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const override;

	// UPrimitiveComponent
	UE_API virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;

	// USceneComponent
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	// UActorComponent
	virtual bool IsHLODRelevant() const override { return false; }

	// UObject
	virtual bool NeedsLoadForServer() const override { return false; }

protected:
	// UStaticMeshComponent
	UE_API virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;

private:
		
	// Needed for proper line traces
	UPROPERTY()
	TObjectPtr<UGizmoViewContext> GizmoViewContext = nullptr;

	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> SubstituteInteractionComponent = nullptr;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> HoverOverrideMaterial = nullptr;

	TSharedPtr<UE::GizmoRenderingUtil::IViewBasedTransformAdjuster> TransformAdjuster;

	bool bHovered = false;
	bool bInteracted = false;

	TFunction<bool(const UE::GizmoRenderingUtil::ISceneViewInterface& View, 
		const FTransform& ComponentToWorld)> RenderVisibilityFunc = nullptr;
};

#undef UE_API
