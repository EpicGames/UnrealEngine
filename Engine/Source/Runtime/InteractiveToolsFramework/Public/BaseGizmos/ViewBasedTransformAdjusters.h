// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/BoxSphereBounds.h"
#include "Math/Transform.h"
#include "Templates/SharedPointer.h"

class USceneComponent;
class UViewAdjustedStaticMeshGizmoComponent;

namespace UE::GizmoRenderingUtil
{
	class ISceneViewInterface;

	/**
	 * Interface for a helper that can adjust a component's transform based on view information,
	 *  used by UViewAdjustedStaticMeshGizmoComponent. Adjusters are typically expected to be safely 
	 *  shareable across game and render threads, so if they are not constant, they are expected to
	 *  handle updates safely in their implementation.
	 */
	class IViewBasedTransformAdjuster
	{
	public:
		virtual ~IViewBasedTransformAdjuster() = default;

		/**
		 * Given the component location and the view information, gives the desired transform of the component.
		 */
		virtual FTransform GetAdjustedComponentToWorld(
			const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld) = 0;

		/**
		 * If an adjuster is used by a render proxy, it will use this endpoint for getting the transform.
		 *  This allows an adjuster to be potentially shared between game and render threads. Of course,
		 *  a component might instead choose to create a new adjuster whenever the render proxy is recreated,
		 *  and recreate the proxy whenever any relevant parameters change.
		 */
		virtual FTransform GetAdjustedComponentToWorld_RenderThread(
			const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld)
		{
			// By default, just route to the other implementation. This is safe to do for any adjusters
			//  that remain constant for the life of the render proxy.
			return GetAdjustedComponentToWorld(View, CurrentComponentToWorld);
		}

		/**
		 * Not every adjuster will care about the world/local setting, but it is useful for the
		 *  a component that uses it to be able to blindly pass this information down without
		 *  caring about the type of adjuster.
		 */
		virtual void UpdateWorldLocalState(bool bWorldIn) {}

		/**
		 * Returns the transform that should be returned by a USceneComponent CalcBounds method, which
		 *  can't know about the view. Although the default is to return the original bounds, any adjuster
		 *  that scales the mesh arbitrarily large will probably need to return infinite bounds, whereas
		 *  an adjuster that only changes orienation must account for any possible orientation.
		 * Note that returning infinite bounds causes the relevant components to no longer be frustum-culled,
		 *  but this is acceptible for gizmos, which are typically few and on-screen.
		 */
		virtual FBoxSphereBounds GetViewIndependentBounds(const FTransform& LocalToWorld,
			const FBoxSphereBounds& OriginalBounds)
		{
			return OriginalBounds;
		}
	};

	/**
	 * Adjuster that maintains same view size but otherwise keeps the component transform. Note
	 *  that this will look wrong if the component is not at the gizmo origin, because the distance
	 *  relative to gizmo origin won't be scaled. Use FGizmoSubComponentTransformAdjuster if that is needed.
	 */
	class FSimpleConstantViewScaleAdjuster : public IViewBasedTransformAdjuster
	{
	public:
		// IGizmoViewTransformAdjuster
		INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetAdjustedComponentToWorld(
			const ISceneViewInterface& View,
			const FTransform& CurrentComponentToWorld) override;

		INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds GetViewIndependentBounds(const FTransform& LocalToWorld,
			const FBoxSphereBounds& OriginalBounds) override;
	private:
	};

	/**
	 * An adjuster that can do various transformations common for sub gizmos, which are based off
	 *  of the parent gizmo transform (in addition to the actual component transform).
	 */
	class FSubGizmoTransformAdjuster : public IViewBasedTransformAdjuster,
		// Used for getting a weak pointer to itself when enqueuing state change on the render thread (in SetGizmoOriginTransform)
		public TSharedFromThis<FSubGizmoTransformAdjuster, ESPMode::ThreadSafe>
	{
	public:
		struct FSettings
		{
			// Some compilers (clang, at least) have issues with using a nested class with default initializers
			// and a default constructor as a default argument in a function, hence this constructor.
			FSettings() {};

			// Keeps view size and offset relative to parent gizmo constant
			bool bKeepConstantViewSize = true;
			// Mirrors the component depending on which octant of the parent gizmo the view is located
			bool bMirrorBasedOnOctant = true;
			// Considers the parent gizmo transform to be unrotated, and applies the relative component transform on top of that.
			//  This is frequently not necessary to set because code upstream will typically update it every tick anyway
			//  via UViewAdjustedStaticMeshGizmoComponent's UpdateWorldLocalState implementation.
			bool bUseWorldAxesForGizmo = false;
			// If positive, keeps the component a given distance in front of the ortho camera, to avoid
			//  clipping it. Applied before bKeepConstantViewSize.
			double DistanceInFrontOfCameraInOrtho = 0;
		};

		FSubGizmoTransformAdjuster(
			const FTransform& GizmoOriginToComponent = FTransform::Identity, const FSettings& SettingsIn = FSettings())
			: GizmoOriginToComponent_GameThread(GizmoOriginToComponent)
			, GizmoOriginToComponent_RenderThread(GizmoOriginToComponent)
			, Settings(SettingsIn)
		{}

		/**
		 * Update the transform of the gizmo origin relative to this component. This is safe to do even if the
		 *  adjuster is used by the render thread (the render-side update gets queued properly).
		 */
		INTERACTIVETOOLSFRAMEWORK_API void SetGizmoOriginTransform(const FTransform& GizmoOriginToComponent);

		void SetMirrorBasedOnOctant(bool bOn) 
		{
			Settings.bMirrorBasedOnOctant = bOn;
		}
		void SetSettings(const FSettings& SettingsIn)
		{
			Settings = SettingsIn;
		}

		/**
		 * Static helper method to create and add this adjuster to a gizmo component for the common case of keeping a 
		 *  constant size gizmo.
		 * 
		 * @param GizmoRootComponent The distance relative to this component is kept constant, and this is the component whose rotation
		 *  is considered to be 0 when using global mode.
		 * @param bMirrorBasedOnOctant If true, mirrors the component around the ComponentToKeepDistanceConstantTo
		 *  depending on which octant of that component the view is located in.
		 */
		INTERACTIVETOOLSFRAMEWORK_API static TSharedPtr<FSubGizmoTransformAdjuster> AddTransformAdjuster(
			UViewAdjustedStaticMeshGizmoComponent* ComponentIn, USceneComponent* GizmoRootComponent, 
			bool bMirrorBasedOnOctant);

		INTERACTIVETOOLSFRAMEWORK_API static TSharedPtr<FSubGizmoTransformAdjuster> AddTransformAdjuster(
			UViewAdjustedStaticMeshGizmoComponent* ComponentIn, USceneComponent* GizmoRootComponent,
			TFunction<bool()> ShouldMirrorBasedOnOctant);


		// IGizmoViewTransformAdjuster
		INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetAdjustedComponentToWorld(
			const UE::GizmoRenderingUtil::ISceneViewInterface& View,
			const FTransform& CurrentComponentToWorld) override;

		INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetAdjustedComponentToWorld_RenderThread(
			const ISceneViewInterface& View, const FTransform& CurrentComponentToWorld) override;
		
		virtual void UpdateWorldLocalState(bool bWorldIn) override
		{
			Settings.bUseWorldAxesForGizmo = bWorldIn;
		}

		INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds GetViewIndependentBounds(const FTransform& LocalToWorld,
			const FBoxSphereBounds& OriginalBounds) override;
	private:
		FTransform GizmoOriginToComponent_GameThread = FTransform::Identity;
		FTransform GizmoOriginToComponent_RenderThread = FTransform::Identity;
		// We might decide to have a render thread version of the settings as well, but it's unclear whether it's
		//  worth it, since it seems like a brief inconsistency during the update is unlikely to cause much of a
		//  problem: a user is unlikely to be changing multiple bools at once, and reading an outdated bool seems
		//  fine.
		FSettings Settings;
	};

	/**
	 * Adjuster that takes the the view frame at the component world location, and applies a constant
	 *  relative transform to that. This can be used to create billboard-like gizmo components that keep
	 *  some orientation to the camera.
	 * This doesn't scale the offset relative to gizmo center, but the simplest option is typically to
	 *  keep the component at gizmo origin and adjust the view relative transform appropriately.
	 */
	//~ TODO: May someday want a non-const version of this where the relative transform is editable, for sliders.
	class FConstantViewRelativeTransformAdjuster : public IViewBasedTransformAdjuster
	{
	public:
		FConstantViewRelativeTransformAdjuster(const FTransform& ViewRelativeTransformIn, bool bConstantSize = true)
			: ViewRelativeTransform(ViewRelativeTransformIn)
			, bKeepConstantViewSize(bConstantSize)
		{
		}

		// IGizmoViewTransformAdjuster
		INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetAdjustedComponentToWorld(
			const UE::GizmoRenderingUtil::ISceneViewInterface& View,
			const FTransform& CurrentComponentToWorld) override;

		INTERACTIVETOOLSFRAMEWORK_API virtual FBoxSphereBounds GetViewIndependentBounds(const FTransform& LocalToWorld,
			const FBoxSphereBounds& OriginalBounds) override;
	private:
		FTransform ViewRelativeTransform = FTransform::Identity;
		bool bKeepConstantViewSize = true;
	};
}
