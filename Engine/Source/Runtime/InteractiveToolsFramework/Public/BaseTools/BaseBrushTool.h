// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MeshSurfacePointTool.h"
#include "InteractiveTool.h"
#include "Engine/HitResult.h"
#include "BaseBrushTool.generated.h"

class UBrushStampIndicator;
class ULocalTwoAxisPropertyEditInputBehavior;

/**
 * Standard properties for a Brush-type Tool
 */
UCLASS(MinimalAPI)
class UBrushBaseProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UBrushBaseProperties();

	/** Relative size of brush */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Size", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0", DisplayPriority = 1, HideEditConditionToggle, EditCondition = "bSpecifyRadius == false", ModelingQuickSettings = 100))
	float BrushSize;

	/** If true, ignore relative Brush Size and use explicit world Radius */
	UPROPERTY(EditAnywhere, Category = Brush, AdvancedDisplay)
	bool bSpecifyRadius;

	/** Radius of brush */
	UPROPERTY(EditAnywhere, Category = Brush, AdvancedDisplay, meta = (EditCondition = "bSpecifyRadius",
		DisplayName = "Radius", UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.1", ClampMax = "50000.0", ModelingQuickSettings = 100))
	float BrushRadius;

	//~Set by tool and determines whether the bEnablePressureSensitivity flag below is shown and used.
	//~ All that a subclass of UBaseBrushTool needs to do to support pressure sensitivity is to call
	//~ the super OnTick, use the provided radius getters where relevant, and set this to true in setup. 
	//~ Its StylusAPI also should be set via its builder.
	UPROPERTY()
	bool bToolSupportsPressureSensitivity = false;

	/** When set, stylus pressure is used to adjust radius */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (EditCondition = "bToolSupportsPressureSensitivity", EditConditionHides))
	bool bEnablePressureSensitivity = false;

	/** Strength of the brush (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Strength",
		UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", 
		HideEditConditionToggle, EditConditionHides, EditCondition = "bShowStrength", DisplayPriority = 2,
		ModelingQuickEdit, ModelingQuickSettings = 200))
	float BrushStrength;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Falloff", 
		UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", 
		HideEditConditionToggle, EditConditionHides, EditCondition="bShowFalloff", DisplayPriority = 3,
		ModelingQuickEdit, ModelingQuickSettings = 300))
	float BrushFalloffAmount;


	/** Allow the Brush to hit the back-side of the mesh */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayPriority = 6, HideEditConditionToggle, EditConditionHides, EditCondition = "bShowHitBackFaces"))
	bool bHitBackFaces = false;

	/** If false, then BrushStrength will not be shown in DetailsView panels (otherwise no effect) */
	UPROPERTY( meta = (TransientToolProperty))
	bool bShowStrength = true;

	/** If false, then BrushFalloffAmount will not be shown in DetailsView panels (otherwise no effect) */
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowFalloff = true;

	/** If false, then bHitBackFaces will not be shown in DetailsView panels (otherwise no effect) */
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowHitBackFaces = false;
};


/**
 * Generic Brush Stamp data
 */
USTRUCT()
struct FBrushStampData
{
	GENERATED_BODY();
	/** Radius of brush stamp */
	float Radius;
	/** World Position of brush stamp */
	FVector WorldPosition;
	/** World Normal of brush stamp */
	FVector WorldNormal;

	/** Hit Result provided by implementations - may not be fully populated */
	FHitResult HitResult;

	/** Falloff of brush stamp */
	float Falloff;
};

/**
 * A behavior that captures a keyboard hotkey to enter a sub-mode "bAdjustingBrush" while the key is pressed.
 * The target tool must call OnDragStart() and OnDragUpdate() to feed the screen coordinates of the mouse for the duration
 * of the behavior. And use "GetIsBrushBeingAdjusted" to pause/disable the brush motion while it is being adjusted.
 *
 * OnDragStart() defines the starting location of an adjustment
 * OnDragUpdate() adjusts the brush strength and radius based on the magnitude of the screen coordinate delta in the
 * vertical and horizontal directions respectively.
 */
UCLASS()
class UE_DEPRECATED(5.5, "UBrushAdjusterInputBehavior is deprecated, use UTwoAxisPropertyEditInputBehavior instead.") UBrushAdjusterInputBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	
	void Initialize(UBaseBrushTool* InBrushTool);

	void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	
	void SetStrengthAdjustSpeed(float InStrengthSpeed) { StrengthAdjustSpeed = InStrengthSpeed; };
	void SetSizeAdjustSpeed(float InSizeSpeed) { SizeAdjustSpeed = InSizeSpeed; };
	
	void OnDragStart(FVector2D InScreenPosition);
	void OnDragUpdate(FVector2D InScreenPosition);
	
	bool IsBrushBeingAdjusted() const { return bAdjustingBrush; };

	// UAnyButtonInputBehavior
	virtual EInputDevices GetSupportedDevices() override;
	virtual bool IsPressed(const FInputDeviceState& Input) override;
	virtual bool IsReleased(const FInputDeviceState& Input) override;
	virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& Input) override;
	virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& Input, EInputCaptureSide eSide) override;
	virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& Input, const FInputCaptureData& data) override;
	virtual void ForceEndCapture(const FInputCaptureData& data) override;
	// UAnyButtonInputBehavior end

private:

	void ResetAdjustmentOrigin(FVector2D InScreenPosition, bool bHorizontalAdjust);

	// true for the duration of the behavior (while hotkey is down)
	bool bAdjustingBrush = false;
	// screen coordinate when hotkey pressed (or when re-centered if changing direction)
	FVector2D BrushOrigin;
	// screen coordinate when hotkey pressed (or when re-centered if changing direction)
	FVector2D AdjustmentOrigin;
	// the radius of the brush when we started a drag
	float StartBrushRadius;
	// the strength of the brush when we started a drag
	float StartBrushStrength;
	// supports separate horizontal / vertical adjustments (switching dynamically based on magnitude)
	bool bAdjustingHorizontally = true;

	// the speed (centimeters per unit of screen coordinate) to adjust brush size when dragging
	float SizeAdjustSpeed = 0.002f; // sensible default
	// the speed (in strength per unit of screen coordinate) to adjust brush strength when dragging
	float StrengthAdjustSpeed = 0.005f; // sensible default

	// the target brush tool to adjust
	UBaseBrushTool* BrushTool;
};

/**
 * UBaseBrushTool implements standard brush-style functionality for an InteractiveTool.
 * This includes:
 *   1) brush radius property set with dimension-relative brush sizing and default brush radius hotkeys
 *   2) brush indicator visualization
 *   3) tracking of last brush stamp location via .LastBrushStamp FProperty
 *   4) status of brush stroke via .bInBrushStroke FProperty
 *   5) "B" hotkey to adjust brush radius / strength by click-dragging in the viewport
 */
UCLASS(MinimalAPI)
class UBaseBrushTool : public UMeshSurfacePointTool
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UBaseBrushTool();

	INTERACTIVETOOLSFRAMEWORK_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	// UInteractiveTool
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnTick(float DeltaTime) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD( FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI ) override;
	
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	
	// IClickDragBehaviorTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	
	// UMeshSurfacePointTool implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginDrag(const FRay& Ray) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnUpdateDrag(const FRay& Ray) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndDrag(const FRay& Ray) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnCancelDrag() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

public:

	/** Properties that control brush size/etc*/
	UPROPERTY()
	TObjectPtr<UBrushBaseProperties> BrushProperties;

	/** Set to true by Tool if user is currently in an active brush stroke*/
	UE_DEPRECATED(5.5, "The bInBrushStroke property is deprecated. This will now always be true in OnBeginDrag/OnUpdateDrag/OnEndDrag.")
	UPROPERTY()
	bool bInBrushStroke = false;

	/** Uniform scale factor that scales from world space (where brush usually exists) to local space */
	UPROPERTY()
	float WorldToLocalScale = 1.0f;

	/** Position of brush at last update (both during stroke and during Hover) */
	UPROPERTY()
	FBrushStampData LastBrushStamp;

public:

	INTERACTIVETOOLSFRAMEWORK_API virtual void IncreaseBrushSizeAction();
	INTERACTIVETOOLSFRAMEWORK_API virtual void DecreaseBrushSizeAction();
	INTERACTIVETOOLSFRAMEWORK_API virtual void IncreaseBrushStrengthAction();
	INTERACTIVETOOLSFRAMEWORK_API virtual void DecreaseBrushStrengthAction();
	INTERACTIVETOOLSFRAMEWORK_API virtual void IncreaseBrushFalloffAction();
	INTERACTIVETOOLSFRAMEWORK_API virtual void DecreaseBrushFalloffAction();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "The IsInBrushStroke function is deprecated. This will now always be true in OnBeginDrag/OnUpdateDrag/OnEndDrag.")
	virtual bool IsInBrushStroke() const { return bInBrushStroke; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	INTERACTIVETOOLSFRAMEWORK_API virtual double GetCurrentBrushRadius() const;
	virtual double GetCurrentBrushRadiusLocal() const { return GetCurrentBrushRadius() * WorldToLocalScale; }

	INTERACTIVETOOLSFRAMEWORK_API void SetBrushEnabled(bool bIsEnabled);

	// return false to disable the hotkey for adjusting the brush radius and strength in the viewport
	virtual bool SupportsBrushAdjustmentInput() { return true; }; 

protected:

	/**
	 * Subclasses should implement this to give an estimate of target dimension for brush size scaling
	 */
	virtual double EstimateMaximumTargetDimension() { return 100.0; }

	virtual double GetActivePressure() const { return ActivePressure; }
protected:
	TInterval<float> BrushRelativeSizeRange;
	double CurrentBrushRadius;
	INTERACTIVETOOLSFRAMEWORK_API void RecalculateBrushRadius();

	UPROPERTY()
	TSoftClassPtr<UBrushBaseProperties> PropertyClass;

	//
	// Brush Indicator support
	//
protected:

	UPROPERTY()
	TObjectPtr<UBrushStampIndicator> BrushStampIndicator;

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetupBrushStampIndicator();
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateBrushStampIndicator();
	INTERACTIVETOOLSFRAMEWORK_API virtual void ShutdownBrushStampIndicator();

	// adjusts size and strength properties while holding hotkey during click/drag
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "BrushAdjusterBehavior is deprecated, use BrushEditBehavior instead.")
	TWeakObjectPtr<UBrushAdjusterInputBehavior> BrushAdjusterBehavior;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	TWeakObjectPtr<ULocalTwoAxisPropertyEditInputBehavior> BrushEditBehavior;

private:
	double ActivePressure = 1.0;
	bool bEnabled = true;
};
