// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "CoreMinimal.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClickDragTool.generated.h"

class UObject;
struct FToolBuilderState;



/**
 * Builder for UClickDragTool
 */
UCLASS(MinimalAPI)
class UClickDragToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * UClickDragTool is a base tool that basically just implements IClickDragBehaviorTarget,
 * and on setup registers a UClickDragInputBehavior. You can subclass this Tool to
 * implement basic click-drag type Tools. If you want to do more advanced things, 
 * like handle modifier buttons/keys, you will need to implement IClickDragBehaviorTarget yourself
 */
UCLASS(MinimalAPI)
class UClickDragTool : public UInteractiveTool, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:

	/**
	 * Register default primary-button-click InputBehaviors
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;


	//
	// IClickBehaviorTarget implementation
	//

	/**
	 * Test if target can begin click-drag interaction at this point
	 * @param ClickPos device position/ray at click point
	 * @return true if target wants to begin sequence
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;

	/**
	 * Notify Target that click press ocurred
	 * @param ClickPos device position/ray at click point
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;

	/**
	 * Notify Target that input position has changed
	 * @param ClickPos device position/ray at click point
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;

	/**
	 * Notify Target that click release occurred
	 * @param ClickPos device position/ray at click point
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;

	/**
	 * Notify Target that click-drag sequence has been explicitly terminated (eg by escape key)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnTerminateDragSequence() override;

};

