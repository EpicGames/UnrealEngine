// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "InputBehaviorSet.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "MeshSurfacePointTool.generated.h"



class UMeshSurfacePointTool;

// This is a temporary interface to provide stylus pressure, currently necessary due to limitations
// in the stylus plugin architecture. Should be removed once InputState/InputBehavior can support stylus events
class IToolStylusStateProviderAPI
{
public:
	virtual float GetCurrentPressure() const = 0;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UMeshSurfacePointToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	IToolStylusStateProviderAPI* StylusAPI = nullptr;

	/** @return true if a single mesh source can be found in the active selection */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance initialized with selected mesh source */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance. Override this in subclasses to build a different Tool class type */
	INTERACTIVETOOLSFRAMEWORK_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const;

	/** Called by BuildTool to configure the Tool with the input MeshSource based on the SceneState */
	INTERACTIVETOOLSFRAMEWORK_API virtual void InitializeNewTool(UMeshSurfacePointTool* Tool, const FToolBuilderState& SceneState) const;

protected:
	INTERACTIVETOOLSFRAMEWORK_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



/**
 * UMeshSurfacePointTool is a base Tool implementation that can be used to implement various
 * "point on surface" interactions. The tool acts on an input IMeshDescriptionSource object,
 * which the standard Builder can extract from the current selection (eg Editor selection).
 * 
 * Subclasses override the OnBeginDrag/OnUpdateDrag/OnEndDrag and OnUpdateHover functions
 * to implement custom behavior.
 */
UCLASS(MinimalAPI)
class UMeshSurfacePointTool : public USingleSelectionTool, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	// UInteractiveTool API Implementation

	/** Register InputBehaviors, etc */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;

	/** Set current stlyus API source */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetStylusAPI(IToolStylusStateProviderAPI* StylusAPI);

	// UMeshSurfacePointTool API

	/**
	 * @return true if the target MeshSource is hit by the Ray
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit);

	/**
	 * This function is called when the user begins a click-drag-release interaction
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnBeginDrag(const FRay& Ray);

	/**
	 * This function is called each frame that the user is in a click-drag-release interaction
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnUpdateDrag(const FRay& Ray);

	/**
	 * This function is called when the user releases the button driving a click-drag-release interaction
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnEndDrag(const FRay& Ray);

	/**
	 * This function is called when the user's drag is cancelled, for example due to the whole tool being shut down.
	 */
	virtual void OnCancelDrag() {}
	

	// IClickDragBehaviorTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnTerminateDragSequence() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;


	// IHoverBehaviorTarget implementation
	
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override { return true; }
	virtual void OnEndHover() override {}


	// IInteractiveToolCameraFocusAPI implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) override;


	/** Called by registered InputBehaviors to set the state of the "shift" button (or device equivalent) */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShiftToggle(bool bShiftDown);

	/** @return current state of the shift toggle */
	virtual bool GetShiftToggle() const { return bShiftToggle; }

	/** Called by registered InputBehaviors to set the state of the "ctrl" button (or device equivalent) */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCtrlToggle(bool bCtrlDown);

	/** @return current state of the ctrl toggle */
	virtual bool GetCtrlToggle() const { return bCtrlToggle; }

	/** @return current input device pressure in range 0-1 */
	INTERACTIVETOOLSFRAMEWORK_API virtual float GetCurrentDevicePressure() const;

	INTERACTIVETOOLSFRAMEWORK_API void SetWorld(UWorld* World);
	INTERACTIVETOOLSFRAMEWORK_API UWorld* GetTargetWorld();

protected:
	/** Current state of the shift modifier toggle */
	bool bShiftToggle = false;

	/** Current state of the ctrl modifier toggle */
	bool bCtrlToggle = false;

	FRay LastWorldRay;

	IToolStylusStateProviderAPI* StylusAPI = nullptr;

	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;

	static constexpr int ShiftModifier = 1;
	static constexpr int CtrlModifier = 2;
};

