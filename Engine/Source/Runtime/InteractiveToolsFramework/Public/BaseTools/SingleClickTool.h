// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "CoreMinimal.h"
#include "InputState.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SingleClickTool.generated.h"

class UObject;
struct FToolBuilderState;



/**
 * Builder for USingleClickTool
 */
UCLASS(MinimalAPI)
class USingleClickToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * USingleClickTool is perhaps the simplest possible interactive tool. It simply
 * reacts to default primary button clicks for the active device (eg left-mouse clicks).
 * 
 * The function ::IsHitByClick() determines what is clickable by this Tool. The default is
 * to return true, which means the click will activate anywhere (the Tool itself has no
 * notion of Actors, Components, etc). You can override this function to, for example,
 * filter out clicks that don't hit a target object, etc.
 * 
 * The function ::OnClicked() implements the action that will occur when a click happens.
 * You must override this to implement any kind of useful behavior.
 */
UCLASS(MinimalAPI)
class USingleClickTool : public UInteractiveTool, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:

	/**
	 * Register default primary-button-click InputBehaviors
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup() override;

	/**
	 * Test if the Target is hit at this 2D position / 3D ray
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos);

	/**
	 * Click the Target at this 2D position / 3D ray. Default behavior is to print debug string.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void OnClicked(const FInputDeviceRay& ClickPos);

};

