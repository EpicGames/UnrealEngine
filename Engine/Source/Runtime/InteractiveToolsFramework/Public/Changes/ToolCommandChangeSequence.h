// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolChange.h"


// This class may be unnecessary, as multiple FChange can be combined in a single transaction.
// Revisit usage in MeshSelectionTool.

/**
 * FCommandChangeSequence contains a list of FCommandChanges and associated target UObjects.
 * The sequence of changes is applied atomically.
 * @warning if the target weak UObject pointers become invalid, those changes are skipped. Possibly leaves things in an undefined state...
 */
class FToolCommandChangeSequence : public FToolCommandChange
{
protected:
	struct FChangeElem
	{
		TWeakObjectPtr<UObject> TargetObject;
		TUniquePtr<FToolCommandChange> Change;
	};

	TArray<TSharedPtr<FChangeElem>> Sequence;

public:
	FToolCommandChangeSequence()
	{
	}

	/** Add a change to the sequence */
	void AppendChange(UObject* Target, TUniquePtr<FToolCommandChange> Change)
	{
		TSharedPtr<FChangeElem> Elem = MakeShared<FChangeElem>();
		Elem->TargetObject = Target;
		Elem->Change = MoveTemp(Change);
		Sequence.Add(Elem);
	}

	/** Apply sequence of changes in-order */
	virtual void Apply(UObject* Object) override
	{
		for (int k = 0; k < Sequence.Num(); ++k)
		{
			TSharedPtr<FChangeElem> Elem = Sequence[k];
			check(Elem->TargetObject.IsValid());
			if (Elem->TargetObject.IsValid())
			{
				Elem->Change->Apply(Elem->TargetObject.Get());
			}
		}
	}

	/** Reverts sequence of changes in reverse-order */
	virtual void Revert(UObject* Object) override
	{
		for (int k = Sequence.Num() - 1; k >= 0; --k)
		{
			TSharedPtr<FChangeElem> Elem = Sequence[k];
			check(Elem->TargetObject.IsValid());
			if (Elem->TargetObject.IsValid())
			{
				Elem->Change->Revert(Elem->TargetObject.Get());
			}
		}
	}

	/** @return string describing this change sequenece */
	virtual FString ToString() const override
	{
		FString Result = "FCommandChangeSequence: ";
		for (int k = 0; k < Sequence.Num(); ++k)
		{
			Result = Result + Sequence[k]->Change->ToString() + TEXT(" ");
		}
		return Result;
	}
};

