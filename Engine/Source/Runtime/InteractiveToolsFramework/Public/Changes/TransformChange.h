// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "InteractiveToolChange.h"
#include "Math/Transform.h"
#include "Templates/Function.h"

class UObject;


/**
 * FComponentWorldTransformChange represents an undoable change to the world transform of a USceneComponent.
 */
class FComponentWorldTransformChange : public FToolCommandChange
{
public:
	INTERACTIVETOOLSFRAMEWORK_API FComponentWorldTransformChange();
	INTERACTIVETOOLSFRAMEWORK_API FComponentWorldTransformChange(const FTransform& From, const FTransform& To);

	FTransform FromWorldTransform;
	FTransform ToWorldTransform;

	/** This function is called on Apply and Revert (last argument is true on Apply)*/
	TFunction<void(FComponentWorldTransformChange*, UObject*, bool)> OnChangeAppliedFunc;

	/** Makes the change to the object */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Revert(UObject* Object) override;

	/** Describes this change (for debugging) */
	INTERACTIVETOOLSFRAMEWORK_API virtual FString ToString() const override;
};



