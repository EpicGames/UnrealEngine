// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/UnrealMath.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"

// predeclarations
class AActor;
class UActorComponent;
class UPrimitiveComponent;
struct FMeshDescription;
class UMaterialInterface;
struct FHitResult;
struct FComponentMaterialSet;

/**
 * @deprecated Use tool targets instead, such as UPrimitiveComponentToolTarget
 *
 * Wrapper around a UObject Component that can provide a MeshDescription, and
 * (optionally) bake a modified MeshDescription back to this Component.
 * An example of a Source might be a StaticMeshComponent. How a modified
 * MeshDescription is committed back is context-dependent (in Editor vs PIE vs Runtime, etc).
 *
 * (Conceivably this doesn't have to be backed by a Component, but most usage will assume there is an Actor)
 */
class FPrimitiveComponentTarget
{
public:
	virtual ~FPrimitiveComponentTarget()= default;

	/** Constructor UPrimitivecomponent*
	 *  @param Component the UPrimitiveComponent* to target
	 */
	FPrimitiveComponentTarget( UPrimitiveComponent* Component ): Component( Component ){}

	/** @return true if component target is still valid. May become invalid for various reasons (eg Component was deleted out from under us) */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsValid() const;

	/** @return the Actor that owns this Component */
	INTERACTIVETOOLSFRAMEWORK_API AActor* GetOwnerActor() const;

	/** @return the Component this is a Source for */
	INTERACTIVETOOLSFRAMEWORK_API UPrimitiveComponent* GetOwnerComponent() const;

	/** @return number of material indices in use by this Component */
	INTERACTIVETOOLSFRAMEWORK_API int32 GetNumMaterials() const;

	/**
	 * Get pointer to a Material provided by this Source
	 * @param MaterialIndex index of the material
	 * @return MaterialInterface pointer, or null if MaterialIndex is invalid
	 */
	INTERACTIVETOOLSFRAMEWORK_API UMaterialInterface* GetMaterial(int32 MaterialIndex) const;

	/**
	 * Get material set provided by this source
	 * @param MaterialSetOut returned material set
	 * @param bAssetMaterials if an underlying asset exists, return the Asset-level material assignment instead of the component materials
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bAssetMaterials = false) const;

	/**
	 * @return the transform on this component
	 * @todo Do we need to return a list of transforms here?
	 */
	INTERACTIVETOOLSFRAMEWORK_API FTransform GetWorldTransform() const;

	/**
	 * Compute ray intersection with the MeshDescription this Source is providing
	 * @param WorldRay ray in world space
	 * @param OutHit hit test data
	 * @return true if ray intersected Component
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool HitTest(const FRay& WorldRay, FHitResult& OutHit) const;

	/**
	 * Set the visibility of the Component associated with this Source (ie to hide during Tool usage)
	 * @param bVisible desired visibility
	 */
	INTERACTIVETOOLSFRAMEWORK_API void SetOwnerVisibility(bool bVisible) const;


	/**
	 * Checks if the underlying asset that would be edited by CommitMesh() is the same for two ComponentTargets
	 * @param OtherTarget Another component target to compare against
	 * @return true if both component targets are known to share the same source asset
	 */
	virtual bool HasSameSourceData(const FPrimitiveComponentTarget& OtherTarget) const = 0;

	/**
	 * Commit an update to the material set. This may generate a transaction.
	 * @param MaterialSet new list of materials
	 * @param bApplyToAsset if true, materials of Asset are updated (if Asset exists), rather than Component
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset);


	struct FCommitParams
	{
		FMeshDescription* MeshDescription{ nullptr };
	};
	using  FCommitter  = TFunction< void( const FCommitParams& ) >;
	virtual FMeshDescription* GetMesh() = 0;
	virtual void CommitMesh( const FCommitter& ) = 0;

	UPrimitiveComponent* Component{};
};

/** @deprecated Use tool target factories instead. */
class FComponentTargetFactory
{
public:
	virtual ~FComponentTargetFactory()= default;
	virtual bool CanBuild( UActorComponent* Candidate ) = 0;
	virtual TUniquePtr<FPrimitiveComponentTarget> Build( UPrimitiveComponent* PrimitiveComponent ) = 0;
};

/**
 * @deprecated Add tool target factories to the tool target manager instead.
 *
 * Add a factory method to make ComponentTarget from UPrimitiveComponent*
 * @param Factory The ComponentTargetFactory
 * @return integer indentifier that identifies this Factory
 */
INTERACTIVETOOLSFRAMEWORK_API int32 AddComponentTargetFactory( TUniquePtr<FComponentTargetFactory> Factory );


/**
 * @return ComponentTargetFactory associated with the given Key, or nullptr if not found
 */
INTERACTIVETOOLSFRAMEWORK_API FComponentTargetFactory* FindComponentTargetFactoryByKey(int32 Key);


/**
 * @return true if ComponentTargetFactory associated with the given Key was found and removed
 */
INTERACTIVETOOLSFRAMEWORK_API bool RemoveComponentTargetFactoryByKey(int32 Key);


/**
 * remove all registered ComponentTargetFactory objects
 */
INTERACTIVETOOLSFRAMEWORK_API void RemoveAllComponentTargetFactoryies();


/**
 * @deprecated Ask your tool target manager to build a target for you instead.
 *
 * Create a TargetComponent for the given Component
 * @param Component A UObject that we would like to use as tool target. This must presently descend from
 * UPrimitiveComponent
 * @return An FComponentTarget instance. Must not return null, though the MeshSource and MeshSink in it's MeshBridge may
 * be
 */
INTERACTIVETOOLSFRAMEWORK_API TUniquePtr<FPrimitiveComponentTarget> MakeComponentTarget(UPrimitiveComponent* Component);


/**
 * @deprecated Ask your tool target manager instead.
 *
 * Determine whether a TargetComponent can be created for the given Component
 * @param Component A UObject that we would like to use as tool target.
 * @return bool signifying whether or not a ComponentTarget can be built
 */
INTERACTIVETOOLSFRAMEWORK_API bool CanMakeComponentTarget(UActorComponent* Component);
