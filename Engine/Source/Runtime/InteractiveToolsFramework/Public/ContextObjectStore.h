// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "ContextObjectStore.generated.h"

class UClass;

/**
 * A context object store allows tools to get access to arbitrary objects which expose data or APIs to enable additional functionality.
  * Some example use cases of context objects: 
  *   - A tool builder may disallow a particular tool if a needed API object is not present in the context store.
  *   - A tool may allow extra actions if it has access to a particular API object in the context store.
  *   - A tool may choose to initialize itself differently based on the presence of a selection-holding data object in the context store.
 */
UCLASS(Transient, MinimalAPI)
class UContextObjectStore : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Finds the first the context object of the given type. Can return a subclass of the given type.
	 * @returns the found context object, casted to the given type, or nullptr if none matches.
	 */
	template <typename TObjectType>
	TObjectType* FindContext() const
	{
		for (UObject* ContextObject : ContextObjects)
		{
			if (TObjectType* CastedObject = Cast<TObjectType>(ContextObject))
			{
				return CastedObject;
			}
		}

		if (UContextObjectStore* ParentStore = Cast<UContextObjectStore>(GetOuter()))
		{
			return ParentStore->FindContext<TObjectType>();
		}

		return nullptr;
	}

	/**
	 * Finds the first context object that derives from the given class.
	 * @returns the found context object, or nullptr if none matches.
	 */
	INTERACTIVETOOLSFRAMEWORK_API UObject* FindContextByClass(UClass* InClass) const;

	/**
	 * Adds a data object to the tool manager's set of shared data objects.
	 * @returns true if the addition is successful.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool AddContextObject(UObject* InContextObject);

	/**
	 * Removes a data object from the tool manager's set of shared data objects.
	 * @returns true if the removal is successful.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool RemoveContextObject(UObject* InContextObject);

	/**
	 * Removes any data objects from the tool manager's set of shared data objects that are of type @param InClass
	 * @returns true if any objects are removed.
	 */
	INTERACTIVETOOLSFRAMEWORK_API bool RemoveContextObjectsOfType(const UClass* InClass);

	template <class TObjectType>
	bool RemoveContextObjectsOfType()
	{
		return RemoveContextObjectsOfType(TObjectType::StaticClass());
	}

	/**
	 * Shuts down the context object store, releasing hold on any stored content objects.
	 */
	INTERACTIVETOOLSFRAMEWORK_API void Shutdown();

protected:
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ContextObjects;
};
