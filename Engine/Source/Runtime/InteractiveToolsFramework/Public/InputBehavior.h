// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputState.h"
#include "Math/NumericLimits.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "InputBehavior.generated.h"

/**
 * Input can be captured separately for Left and Right sides (eg for VR controllers)
 * Currently mouse is Left.
 */
UENUM()
enum class EInputCaptureSide
{
	None = 0,
	Left = 1,
	Right = 2,
	Both = 3,
	Any = 99
};

/**
 * An active capturing behavior may need to keep track of additional data that
 * cannot be stored within the behavior (for example if the same behavior instance
 * is capturing for Left and Right separately). So FInputCaptureUpdate can optionally
 * return this structure, and we will pass it to the next UpdateCapture() call
 */
struct FInputCaptureData
{
	/** Which side do we want to capture on */
	EInputCaptureSide WhichSide;
	/** pointer to data defined by the InputBehavior, which is also responsible for cleaning it up */
	void* CustomData;

	FInputCaptureData()
	{
		WhichSide = EInputCaptureSide::None;
		CustomData = nullptr;
	}
};

/**
 * Used by FInputCaptureRequest to indicate whether the InputBehavior
 * wants to capture or ignore an input event
 */
UENUM()
enum class EInputCaptureRequestType
{
	Begin = 1,
	Ignore = 2
};

// predeclaration
class UInputBehavior;

/**
 * UInputBehavior returns an FInputCaptureRequest from WantsCapture() to indicate
 * whether it wants to capture or ignore an input event
 */
struct FInputCaptureRequest
{
	/** Which input behavior generated this request */
	UInputBehavior* Source;
	/** What type of capture request is this (Begin or Ignore) */
	EInputCaptureRequestType Type;
	/** Which side does request want to capture on */
	EInputCaptureSide Side;

	/** Depth along hit-test ray */
	double HitDepth;

	/** Owner of the requesting behavior. Behavior doesn't know this, so this is initialized to null */
	void* Owner;

	FInputCaptureRequest(
		EInputCaptureRequestType type,
		UInputBehavior* behavior,
		EInputCaptureSide whichSide,
		double hitDepth = TNumericLimits<double>::Max()
	)
	{
		this->Type = type;
		this->Source = behavior;
		this->Side = whichSide;
		this->Owner = nullptr;
		this->HitDepth = hitDepth;
	}

	/** Create a Begin-capture request */
	static FInputCaptureRequest Begin(
		UInputBehavior* behavior, EInputCaptureSide whichSide, double hitDepth = TNumericLimits<double>::Max()
	)
	{
		return FInputCaptureRequest(EInputCaptureRequestType::Begin, behavior, whichSide, hitDepth);
	}

	/** Create an ignore-capture request */
	static FInputCaptureRequest Ignore()
	{
		return FInputCaptureRequest(
			EInputCaptureRequestType::Ignore, nullptr, EInputCaptureSide::Any, TNumericLimits<double>::Max()
		);
	}

	friend bool operator<(const FInputCaptureRequest& l, const FInputCaptureRequest& r);
};

/**
 * FInputCaptureUpdate uses this type to indicate what state the capturing Behavior
 * would like to transition to, based on the input event
 */
UENUM()
enum class EInputCaptureState
{
	Begin = 1, // start capturing (which should always be the case if BeginCapture is called)
	Continue = 2, // Behavior wants to continue capturing
	End = 3, // Behavior wants to end capturing
	Ignore = 4 // Behavior ignored this event
};

/**
 * IInputBehavior returns an FInputCaptureUpdate from BeginCapture() and UpdateCapture(),
 * which indicates to the InputRouter what the Behavior would like to have happen.
 */
struct FInputCaptureUpdate
{
	/** Indicates what capture state the Behavior wants to transition to */
	EInputCaptureState State;
	/** Which Behavior did this update come from */
	UInputBehavior* Source;
	/** custom data for the active capture that should be propagated to next UpdateCapture() call */
	FInputCaptureData Data;

	/**
	 * Create a begin-capturing instance of FInputCaptureUpdate
	 * @param Source UInputBehavior that is returning this update
	 * @param Which Which side we are capturing on
	 * @param CustomData client-provided data that will be passed to UInputBehavior::UpdateCapture() calls. Client owns this memory!
	 */
	static FInputCaptureUpdate Begin(UInputBehavior* SourceBehavior, EInputCaptureSide WhichSide, void* CustomData = nullptr)
	{
		return FInputCaptureUpdate(EInputCaptureState::Begin, SourceBehavior, WhichSide, CustomData);
	}

	/** Create a default continue-capturing instance of FInputCaptureUpdate */
	static FInputCaptureUpdate Continue()
	{
		return FInputCaptureUpdate(EInputCaptureState::Continue, nullptr, EInputCaptureSide::Any);
	}
	/** Create a default end-capturing instance of FInputCaptureUpdate */
	static FInputCaptureUpdate End()
	{
		return FInputCaptureUpdate(EInputCaptureState::End, nullptr, EInputCaptureSide::Any);
	}
	/** Create a default ignore-capturing instance of FInputCaptureUpdate */
	static FInputCaptureUpdate Ignore()
	{
		return FInputCaptureUpdate(EInputCaptureState::Ignore, nullptr, EInputCaptureSide::Any);
	}

	/**
	 * @param StateIn desired capture state
	 * @param Source UInputBehavior that is returning this update
	 * @param Which Which side we are capturing on
	 * @param CustomData client-provided data that will be passed to UInputBehavior::UpdateCapture() calls. Client owns this memory!
	 */
	FInputCaptureUpdate(
		EInputCaptureState StateIn, UInputBehavior* SourceBehaviorIn, EInputCaptureSide WhichSideIn, void* CustomData = nullptr
	)
	{
		State = StateIn;
		Source = SourceBehaviorIn;
		Data.WhichSide = WhichSideIn;
		Data.CustomData = CustomData;
	}
};

/**
 * Each UInputBehavior provides a priority that is used to help resolve situations
 * when multiple Behaviors want to capture based on the same input event
 */
struct FInputCapturePriority
{
	static constexpr int DEFAULT_GIZMO_PRIORITY = 50;
	static constexpr int DEFAULT_TOOL_PRIORITY = 100;

	/**
	 * The binding complexity priority convention is a solution to the problem of ensuring the right behavior gets capture.
	 * Take the case of one behavior looking for LMB and another looking for Alt+LMB.
	 * If they have the same priority, which behavior gets capture is effectively non-deterministic.
	 * Priority would then need to be hand-tuned relative to each other, an inherently brittle task.
	 *
	 * If all behaviors agree that that additional binding complexity increases capture priority by a set amount, all they
	 * must do is correctly publish their priority by this convention, and they will win out over less complex behaviors
	 * and gracefully lose to more complex behaviors.
	 *
	 * An alternative discussed and discarded was to have behaviors assert their negative bindings. In this mode, a behavior
	 * wouldn't just look for LMB, it would look for LMB as well as NOT RMB, MMB, Ctrl, Alt, Shift, etc.
	 * This was rejected. Knowing how much to negatively assert is not well defined. Additionally, users often expect that
	 * pressing (for example) Alt+LMB will trigger an LMB action if nothing is bound to Alt+LMB.
	 * 
	 * The value of BINDING_COMPLEXITY_PRIORITY_STEP can be increased to give more wiggle room for nuanced behavior layering.
	 * As of writing, this number is intended such that `DEFAULT_TOOL_PRIORITY - BINDING_COMPLEXITY_PRIORITY_STEP * 5 > DEFAULT_GIZMO_PRIORITY`.
	 * The actual value should be considered experimental.
	 */
	static constexpr int BINDING_COMPLEXITY_PRIORITY_STEP = 9;

	/** Constant priority value */
	int Priority;

	FInputCapturePriority(int priority = DEFAULT_TOOL_PRIORITY)
	{
		Priority = priority;
	}

	/** @return a priority lower than this priority */
	FInputCapturePriority MakeLower(int DeltaAmount = 1) const
	{
		return FInputCapturePriority(Priority + DeltaAmount);
	}

	/** @return a priority higher than this priority */
	FInputCapturePriority MakeHigher(int DeltaAmount = 1) const
	{
		return FInputCapturePriority(Priority - DeltaAmount);
	}
	
	FInputCapturePriority ApplyBindingComplexity(int InComplexity) const
	{
		return MakeHigher(InComplexity * BINDING_COMPLEXITY_PRIORITY_STEP);
	}

	friend bool operator<(const FInputCapturePriority& l, const FInputCapturePriority& r)
	{
		return l.Priority < r.Priority;
	}
	friend bool operator==(const FInputCapturePriority& l, const FInputCapturePriority& r)
	{
		return l.Priority == r.Priority;
	}
};

/**
 * An InputBehavior implements a state machine for a user interaction.
 * The InputRouter maintains a set of active Behaviors, and when new input
 * events occur, it calls WantsCapture() to check if the Behavior would like to
 * begin capturing the applicable input event stream (eg for a mouse, one or both VR controllers, etc).
 * If the Behavior acquires capture, UpdateCapture() is called until the Behavior
 * indicates that it wants to release the device, or until the InputRouter force-terminates
 * the capture via ForceEndCapture().
 *
 * For example, something like ButtonSetClickBehavior might work as follows:
 *    - in WantsCapture(), if left mouse is pressed and a button is under cursor, return Begin, otherwise Ignore
 *    - in BeginCapture(), save identifier for button that is under cursor
 *    - in UpdateCapture()
 *        - if left mouse is down, return Continue
 *        - if left mouse is released:
 *            - if saved button is still under cursor, call button.Clicked()
 *            - return End
 *
 * Written sufficiently generically, the above Behavior doesn't need to know about buttons,
 * it just needs to know how to hit-test the clickable object(s). Similarly separate
 * Behaviors can be written for mouse, VR, touch, gamepad, etc.
 *
 * Implementing interactions in this way allows the input handling to be separated from functionality.
 */
UCLASS(Transient, MinimalAPI)
class UInputBehavior : public UObject
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UInputBehavior();

	/** The priority is used to resolve situations where multiple behaviors want the same capture */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCapturePriority GetPriority();

	/** Configure the default priority of an instance of this behavior */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetDefaultPriority(const FInputCapturePriority& Priority);

	/** Which device types does this Behavior support */
	INTERACTIVETOOLSFRAMEWORK_API virtual EInputDevices GetSupportedDevices();

	/** Given the input state, does this Behavior want to begin capturing some input devices? */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InputState);

	/** Called after WantsCapture() returns a capture request that was accepted */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginCapture(
		const FInputDeviceState& InputState, EInputCaptureSide eSide
	);

	/**
	 * Called for each new input event during a capture sequence. Return Continue to keep
	 * capturing, or End to finish capturing.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateCapture(
		const FInputDeviceState& InputState, const FInputCaptureData& CaptureData
	);

	/** If this is called, the Behavior has forcibly lost capture (eg due to app losing focus for example) and needs to clean up accordingly */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceEndCapture(const FInputCaptureData& CaptureData);

	//
	// hover support (optional)
	//

	/** return true if this Behavior supports hover (ie passive input events) */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool WantsHoverEvents();

	/** Given the input state, does this Behavior want to begin capturing some input devices for hover */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureRequest WantsHoverCapture(const FInputDeviceState& InputState);

	/** Called after WantsHoverCapture() returns a capture request that was accepted */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate BeginHoverCapture(
		const FInputDeviceState& InputState, EInputCaptureSide eSide
	);

	/** Called on each new hover input event, ie if no other behavior is actively capturing input */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInputCaptureUpdate UpdateHoverCapture(const FInputDeviceState& InputState);

	/** If a different hover capture begins, focus is lost, a tool starts, etc, any active hover visualization needs to terminate */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndHoverCapture();

	/**
	 * When this function returns true, this behavior will always receive ForceEndCapture calls even when it is not actively
	 * capturing. This is useful for behaviors that do not capture, but hold state (ex. Key Modifiers) and want to be notified when to clear that state.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool WantsForceEndCapture();

	/**
	 * When this function returns true, this behavior allows the Input Router to interrupt capturing and pass it on to a different Behavior with a higher priority.
	 */
	UE_EXPERIMENTAL(5.7, "Capture stealing is experimental behavior. Its implementation may change in the future.")
	INTERACTIVETOOLSFRAMEWORK_API virtual bool AllowsCaptureStealing();

	/**
	 * Call to set the default capture stealing attribute.
	 */
	UE_EXPERIMENTAL(5.7, "Capture stealing is experimental behavior. Its implementation may change in the future.")
	INTERACTIVETOOLSFRAMEWORK_API void SetAllowsCaptureStealing(TAttribute<bool> InAllowsCaptureStealing);

protected:
	/** priority returned by GetPriority() */
	FInputCapturePriority DefaultPriority;

	UE_EXPERIMENTAL(5.7, "Capture stealing is experimental behavior. Its implementation may change in the future.")
	/** value returned by AllowsCaptureStealing*/
	TAttribute<bool> CaptureStealing;
};
