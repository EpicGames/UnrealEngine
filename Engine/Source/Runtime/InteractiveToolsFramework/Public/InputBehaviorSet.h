// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputState.h"
#include "Templates/Function.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InputBehaviorSet.generated.h"



/**
 * This is an internal structure used by UInputBehaviorSet. 
 */
USTRUCT()
struct FBehaviorInfo
{
	GENERATED_BODY()
	
	/** Reference to a Behavior */
	UPROPERTY()
	TObjectPtr<UInputBehavior> Behavior = nullptr;

	/** Source object that provided this Behavior */
	void* Source = nullptr;

	/** Group identifier for this Behavior */
	FString Group;

	friend bool operator<(const FBehaviorInfo& l, const FBehaviorInfo& r)
	{
		return l.Behavior->GetPriority() < r.Behavior->GetPriority();
	}
};




/**
 * UInputBehaviorSet manages a set of UInputBehaviors, and provides various functions
 * to query and forward events to the set. Tools and Widgets provide instances of this via
 * IInputBehaviorSource, and UInputRouter collects and manages them (see comments there)
 *
 * Behaviors in the set each have a source pointer and group tag, which allows sets of
 * behaviors to be managed together. For example one UInputBehaviorSet can be merged into 
 * another and removed later.
 */
UCLASS(Transient, MinimalAPI)
class UInputBehaviorSet : public UObject
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UInputBehaviorSet();
	INTERACTIVETOOLSFRAMEWORK_API virtual ~UInputBehaviorSet();

	//
	// Set Management
	//


	/** @return true if there are no Behaviors in set */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsEmpty() const;

	/** 
	 * Add a Behavior to the set
	 * @param Behavior Behavior to add to set
	 * @param Source pointer to owning object, used only to identify Behavior later
	 * @param GroupName string identifier for this Behavior or group of Behaviors
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Add(UInputBehavior* Behavior, void* Source = nullptr, const FString& GroupName = "");

	/**
	 * Merge another BehaviorSet into this Set
	 * @param OtherSet Set of Behaviors to add to this Set
	 * @param Source pointer to owning object, used only to identify Behavior later. If nullptr, source is copied from other Set.
	 * @param NewGroupName string identifier for this Behavior or group of Behaviors. If empty string, group is copied from other Set.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Add(const UInputBehaviorSet* OtherSet, void* NewSource = nullptr, const FString& NewGroupName = "");

	/**
	 * Remove a Behavior from the Set
	 * @param Behavior Behavior to remove
	 * @return true if Behavior was found and removed
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool Remove(UInputBehavior* Behavior);
	/**
	 * Remove a group of Behaviors from the Set
	 * @param GroupName name of group, all Behaviors that were added with this GroupName are removed
	 * @return true if any Behaviors were found and removed
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool RemoveByGroup(const FString& GroupName);
	/**
	 * Remove a group of Behaviors from the Set
	 * @param Source source object pointer, all Behaviors that were added with this Source pointer are removed.
	 * @return true if any Behaviors were found and removed
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool RemoveBySource(void* Source);
	/**
	 * Remove all Behaviors from the set
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RemoveAll();


	//
	// Queries and Event Forwarding
	//


	/**
	 * Call UInputBehavior::WantsCapture() on each valid Behavior and collect up the requests that indicated a Capture was desired.
	 * @param InputState current input device state information
	 * @param ResultOut returned set of non-Ignoring FInputCaptureRequests returned by Behaviors
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void CollectWantsCapture(const FInputDeviceState& InputState, TArray<FInputCaptureRequest>& ResultOut);

	/**
	 * Call UInputBehavior::WantsCapture() on each valid Behavior and collect up the requests that indicated a Capture was desired.
	 * @param InputState current input device state information
	 * @param ResultOut returned set of non-Ignoring FInputCaptureRequests returned by Behaviors
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void CollectWantsHoverCapture(const FInputDeviceState& InputState, TArray<FInputCaptureRequest>& ResultOut);

	/**
	 * Call UInputBehavior::WantsForceEndCapture on each valid Behavior and collect up the requests that indicated that a ForceEndCapture was desired.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void CollectWantsForceEndCapture(TArray<UInputBehavior*>& ResultOut);


protected:

	/** Current set of known Behaviors */
	UPROPERTY()
	TArray<FBehaviorInfo> Behaviors;

	/**
	 * called internally when Behaviors list is updated, to re-sort by Priority, etc (@todo: and emit change event)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BehaviorsModified();


	/** @return true if Behavior supports InputState.InputDevice */
	bool SupportsInputType(UInputBehavior* Behavior, const FInputDeviceState& InputState)
	{
		return (Behavior->GetSupportedDevices() & InputState.InputDevice) != EInputDevices::None;
	}

};





// UInterface for IInputBehavior
UINTERFACE(MinimalAPI)
class UInputBehaviorSource : public UInterface
{
	GENERATED_BODY()
};


/** 
 * UObjects that implement IInputBehaviorSource have an UInputBehaviorSet
 * that they can provide (to UInputRouter, primarily)
 * @todo callback/delegate for when the provided InputBehaviorSet changes
 */
class IInputBehaviorSource
{
	GENERATED_BODY()

public:
	/**
	 * @return The current UInputBehaviorSet for this Source
	 */
	virtual const UInputBehaviorSet* GetInputBehaviors() const = 0;
};


/**
 * An implementation of IInputBehaviorSource that forwards to a user provided-lambda, to allow
 * a tool to supply a behavior source different from the one it is implementing itself. Useful,
 * for instance, when a tool wants to supply different behaviors to separate input routers.
 */
UCLASS(Transient, MinimalAPI)
class ULocalInputBehaviorSource : public UObject, public IInputBehaviorSource
{
	GENERATED_BODY()

public:

	TUniqueFunction<const UInputBehaviorSet* ()> GetInputBehaviorsFunc = []() { return nullptr; };

	// IInputBehaviorSource
	virtual const UInputBehaviorSet* GetInputBehaviors() const override { return GetInputBehaviorsFunc(); };
};
