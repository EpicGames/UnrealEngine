// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InputBehaviorSet.h"
#include "InputState.h"
#include "Misc/Change.h"
#include "ToolContextInterfaces.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InputRouter.generated.h"

class IInputBehaviorSource;
class IToolsContextTransactionsAPI;
class UInputBehaviorSet;
struct FInputDeviceState;


/**
 * UInputRouter mediates between a higher-level input event source (eg like an FEdMode)
 * and a set of InputBehaviors that respond to those events. Sets of InputBehaviors are
 * registered, and then PostInputEvent() is called for each event. 
 *
 * Internally one of the active Behaviors may "capture" the event stream.
 * Separate "Left" and "Right" captures are supported, which means that (eg)
 * an independent capture can be tracked for each VR controller.
 *
 * If the input device supports "hover",  PostHoverInputEvent() will forward
 * hover events to InputBehaviors that also support it.
 *
 */
UCLASS(Transient, MinimalAPI)
class UInputRouter : public UObject
{
	GENERATED_BODY()

protected:
	friend class UInteractiveToolsContext;		// to call Initialize/Shutdown

	INTERACTIVETOOLSFRAMEWORK_API UInputRouter();

	/** Initialize the InputRouter with the necessary Context-level state. UInteractiveToolsContext calls this, you should not. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IToolsContextTransactionsAPI* TransactionsAPI);

	/** Shutdown the InputRouter. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

public:
	/** Add a new behavior Source. Behaviors from this source will be added to the active behavior set. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RegisterSource(IInputBehaviorSource* Source);

	/** Remove Behaviors from this Source from the active set */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DeregisterSource(IInputBehaviorSource* Source);

	/** Insert a new input event which is used to check for new captures, or forwarded to active capture */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool PostInputEvent(const FInputDeviceState& Input);

	/** Returns true if there is an active mouse capture */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasActiveMouseCapture() const;

	// TODO: other capture queries

	/** Insert a new hover input event which is forwarded to all hover-enabled Behaviors */
	INTERACTIVETOOLSFRAMEWORK_API virtual void PostHoverInputEvent(const FInputDeviceState& Input);

	/** If this Behavior is capturing, call ForceEndCapture() to notify that we are taking capture away */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceTerminateSource(IInputBehaviorSource* Source);

	/** Terminate any active captures and end all hovers */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ForceTerminateAll();


public:
	/** If true, then we post an Invalidation (ie redraw) request if any active InputBehavior responds to Hover events (default false) */
	UPROPERTY()
	bool bAutoInvalidateOnHover;

	/** If true, then we post an Invalidation (ie redraw) request on every captured input event (default false) */
	UPROPERTY()
	bool bAutoInvalidateOnCapture;


protected:
	IToolsContextTransactionsAPI* TransactionsAPI;

	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> ActiveInputBehaviors;

	UInputBehavior* ActiveKeyboardCapture = nullptr;
	void* ActiveKeyboardCaptureOwner = nullptr;
	FInputCaptureData ActiveKeyboardCaptureData;

	UInputBehavior* ActiveLeftCapture = nullptr;
	void* ActiveLeftCaptureOwner = nullptr;
	FInputCaptureData ActiveLeftCaptureData;

	UInputBehavior* ActiveRightCapture = nullptr;
	void* ActiveRightCaptureOwner = nullptr;
	FInputCaptureData ActiveRightCaptureData;

	INTERACTIVETOOLSFRAMEWORK_API virtual void PostInputEvent_Keyboard(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API void CheckForKeyboardCaptures(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API void HandleCapturedKeyboardInput(const FInputDeviceState& Input);

	INTERACTIVETOOLSFRAMEWORK_API virtual void PostInputEvent_Mouse(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API void CheckForMouseCaptures(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API void HandleCapturedMouseInput(const FInputDeviceState& Input);

	//
	// Hover support
	//

	UInputBehavior* ActiveLeftHoverCapture = nullptr;
	void* ActiveLeftHoverCaptureOwner = nullptr;

	FInputDeviceState LastMouseInputState;

	INTERACTIVETOOLSFRAMEWORK_API void TerminateHover(EInputCaptureSide Side);
	INTERACTIVETOOLSFRAMEWORK_API bool ProcessMouseHover(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API bool UpdateExistingHoverCaptureIfPresent(const FInputDeviceState& Input);
	INTERACTIVETOOLSFRAMEWORK_API void OnApplicationFocusChanged(bool bInIsFocused);
};
