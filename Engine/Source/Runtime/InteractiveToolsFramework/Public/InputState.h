// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Math/NumericLimits.h"
#include "Math/Ray.h"
#include "Math/UnrealMath.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "InputState.generated.h"

class UObject;


/**
 * Input event data can be applicable to many possible input devices.
 * These flags are used to indicate specific or sets of device types.
 */
UENUM()
enum class EInputDevices
{
	None = 0,
	Keyboard = 1,
	Mouse = 2,
	Gamepad = 4,

	OculusTouch = 8,
	HTCViveWands = 16,
	AnySpatialDevice = OculusTouch | HTCViveWands,

	TabletFingers = 1024
};
ENUM_CLASS_FLAGS(EInputDevices);



/*
 * FInputRayHit is returned by various hit-test interface functions.
 * Generally this is intended to be returned as the result of a hit-test with a FInputDeviceRay 
 */
USTRUCT(BlueprintType)
struct FInputRayHit
{
	GENERATED_BODY()

	/** true if ray hit something, false otherwise */
	UPROPERTY(BlueprintReadWrite, Category = InputRayHit)
	bool bHit;

	/** distance along ray at which intersection occurred */
	UPROPERTY(BlueprintReadWrite, Category = InputRayHit)
	double HitDepth;

	/** Normal at hit point, if available */
	UPROPERTY(BlueprintReadWrite, Category = InputRayHit)
	FVector HitNormal;

	/** True if HitNormal was set */
	UPROPERTY(BlueprintReadWrite, Category = InputRayHit)
	bool bHasHitNormal;

	/** Client-defined integer identifier for hit object/element/target/etc */
	UPROPERTY(BlueprintReadWrite, Category = InputRayHit)
	int32 HitIdentifier;

	/**
	  * Client-defined pointer for hit object/element/target/etc. 
	  * HitOwner and HitObject should be set to the same pointer if the HitOwner derives from UObject.
	  */
	void* HitOwner;

	/**
	  * Client-defined pointer for UObject-derived hit owners.  
	  * HitOwner and HitObject should be set to the same pointer if the HitOwner derives from UObject. 
	  */
	UPROPERTY(BlueprintReadWrite, Category = InputRayHit)
	TWeakObjectPtr<UObject> HitObject;

	/** Set hit object, will also set hit owner to the same value */
	void SetHitObject(UObject* InHitObject)
	{
		HitObject = InHitObject;
		HitOwner = InHitObject;
	}

	FInputRayHit()
	{
		bHit = false;
		HitDepth = (double)TNumericLimits<float>::Max();
		HitNormal = FVector(0, 0, 1);
		bHasHitNormal = false;
		HitIdentifier = 0;
		HitOwner = nullptr;
		HitObject = nullptr;
	}

	explicit FInputRayHit(double HitDepthIn)
	{
		bHit = true;
		HitDepth = HitDepthIn;
		HitNormal = FVector(0, 0, 1);
		bHasHitNormal = false;
		HitIdentifier = 0;
		HitOwner = nullptr;
		HitObject = nullptr;
	}

	explicit FInputRayHit(double HitDepthIn, const FVector& HitNormalIn)
	{
		bHit = true;
		HitDepth = HitDepthIn;
		HitNormal = HitNormalIn;
		bHasHitNormal = true;
		HitIdentifier = 0;
		HitOwner = nullptr;
		HitObject = nullptr;
	}

};





/**
 * Current State of a physical device button (mouse, key, etc) at a point in time.
 * Each "click" of a button should involve at minimum two separate state
 * events, one where bPressed=true and one where bReleased=true.
 * Each of these states should occur only once.
 * In addition there may be additional frames where the button is
 * held down and bDown=true and bPressed=false.
 */
USTRUCT(BlueprintType)
struct FDeviceButtonState
{
	GENERATED_BODY()
	
	/** Button identifier */
	UPROPERTY(transient, BlueprintReadWrite, Category = DeviceButtonState)
	FKey Button;

	/** Was the button pressed down this frame. This should happen once per "click" */
	UPROPERTY(transient, BlueprintReadWrite, Category = DeviceButtonState)
	bool bPressed;

	/** Is the button currently pressed down. This should be true every frame the button is pressed. */
	UPROPERTY(transient, BlueprintReadWrite, Category = DeviceButtonState)
	bool bDown;

	/** Was the button released this frame. This should happen once per "click" */
	UPROPERTY(transient, BlueprintReadWrite, Category = DeviceButtonState)
	bool bReleased;

	/** Was the button double clicked this frame. This should happen only once per "double click" */
	UPROPERTY(Transient, BlueprintReadWrite, Category = DeviceButtonState)
	bool bDoubleClicked;

	FDeviceButtonState()
	{
		Button = FKey();
		bPressed = bDown = bReleased = bDoubleClicked = false;
	}

	FDeviceButtonState(const FKey& ButtonIn)
	{
		Button = ButtonIn;
		bPressed = bDown = bReleased = bDoubleClicked = false;
	}

	/** Update the states of this button */
	void SetStates(bool bPressedIn, bool bDownIn, bool bReleasedIn, bool bDoubleClickedIn = false)
	{
		bPressed = bPressedIn;
		bDown = bDownIn;
		bReleased = bReleasedIn;
		bDoubleClicked = bDoubleClickedIn;
	}
	
	bool IsValid() const
	{
		return Button.IsValid();
	}
};



/**
 * Current state of active keyboard key at a point in time
 * @todo would be useful to track set of active keys
 */
USTRUCT(BlueprintType)
struct FKeyboardInputDeviceState
{
	GENERATED_BODY()
	
	/** state of active key that was modified (ie press or release) */
	UPROPERTY(transient, BlueprintReadWrite, Category = KeyboardInputDeviceState)
	FDeviceButtonState ActiveKey;
};



/**
 * Current State of a physical Mouse device at a point in time.
 */
USTRUCT(BlueprintType)
struct FMouseInputDeviceState
{
	GENERATED_BODY()
	
	/** State of the left mouse button */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FDeviceButtonState Left;
		
	/** State of the middle mouse button */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FDeviceButtonState Middle;
		
	/** State of the right mouse button */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FDeviceButtonState Right;

	/** Change in 'ticks' of the mouse wheel since last state event */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	float WheelDelta;

	/** Current 2D position of the mouse, in application-defined coordinate system */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FVector2D Position2D;

	/** Change in 2D mouse position from last state event */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FVector2D Delta2D;

	/** Ray into current 3D scene at current 2D mouse position */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FRay WorldRay;

	/** The 2D position of the mouse, unconstrained by screen real-estate. Computed by summing unbound deltas. */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FVector2D UnboundedPosition2D;

	/** Change in 2D mouse position, valid also for movements outside the display bounds */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FVector2D UnboundedMouseDelta2D;

	/** Ray into current 3D scene at current unbounded 2D mouse position */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FRay UnboundedWorldRay;

	FMouseInputDeviceState()
	{
		Left = FDeviceButtonState(EKeys::LeftMouseButton);
		Middle = FDeviceButtonState(EKeys::MiddleMouseButton);
		Right = FDeviceButtonState(EKeys::RightMouseButton);
		WheelDelta = false;
		Position2D = FVector2D::ZeroVector;
		Delta2D = FVector2D::ZeroVector;
		WorldRay = FRay();
		UnboundedPosition2D = FVector2D::ZeroVector;
		UnboundedMouseDelta2D = FVector2D::ZeroVector;
		UnboundedWorldRay = FRay();
	}
};



/**
 * Current state of physical input devices at a point in time.
 * Assumption is that the state refers to a single physical input device,
 * ie InputDevice field is a single value of EInputDevices and not a combination.
 */
USTRUCT(BlueprintType, MinimalAPI)
struct FInputDeviceState
{
	GENERATED_BODY()

	UE_EXPERIMENTAL(5.8, "Experimental extension that allows for generic access to state via FKey.")
	DECLARE_DELEGATE_RetVal_OneParam(FDeviceButtonState, FGetDeviceButtonState, const FKey&);

	/** Which InputDevice member is valid in this state */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	EInputDevices InputDevice;

	//
	// keyboard modifiers
	//

	/** Is they keyboard SHIFT modifier key currently pressed down */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	bool bShiftKeyDown;
	
	/** Is they keyboard ALT modifier key currently pressed down */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	bool bAltKeyDown;
	
	/** Is they keyboard CTRL modifier key currently pressed down */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	bool bCtrlKeyDown;
	
	/** Is they keyboard CMD modifier key currently pressed down (only on Apple devices) */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	bool bCmdKeyDown;

	/** Current state of Keyboard device, if InputDevice == EInputDevices::Keyboard */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FKeyboardInputDeviceState Keyboard;

	/** Current state of Mouse device, if InputDevice == EInputDevices::Mouse */
	UPROPERTY(transient, BlueprintReadWrite, Category = MouseInputDeviceState)
	FMouseInputDeviceState Mouse;

	FInputDeviceState() 
	{
		InputDevice = EInputDevices::None;
		bShiftKeyDown = bAltKeyDown = bCtrlKeyDown = bCmdKeyDown = false;
		Keyboard = FKeyboardInputDeviceState();
		Mouse = FMouseInputDeviceState();
	}

	/** Update keyboard modifier key states */
	void SetModifierKeyStates(bool bShiftDown, bool bAltDown, bool bCtrlDown, bool bCmdDown) 
	{
		bShiftKeyDown = bShiftDown;
		bAltKeyDown = bAltDown;
		bCtrlKeyDown = bCtrlDown;
		bCmdKeyDown = bCmdDown;
	}

	/**
	 * @param DeviceType Combination of device-type flags
	 * @return true if this input state is for an input device that matches the query flags 
	 */
	bool IsFromDevice(EInputDevices DeviceType) const
	{
		return ((InputDevice & DeviceType) != EInputDevices::None);
	}
	
	/** @return The state for the current button. */
	UE_EXPERIMENTAL(5.8, "Experimental extension that allows for generic access to state via FKey.")
	INTERACTIVETOOLSFRAMEWORK_API FDeviceButtonState GetButtonState(const FKey& Key) const;

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Experimental extension that allows for generic access to state via FKey.")
	FGetDeviceButtonState OnGetButtonStateFallback;
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	//
	// utility functions to pass as lambdas
	// 

	/** @return true if shift key is down in input state */
	static bool IsShiftKeyDown(const FInputDeviceState& InputState)
	{
		return InputState.bShiftKeyDown;
	}

	/** @return true if ctrl key is down in input state */
	static bool IsCtrlKeyDown(const FInputDeviceState& InputState)
	{
		return InputState.bCtrlKeyDown;
	}

	/** @return true if alt key is down in input state */
	static bool IsAltKeyDown(const FInputDeviceState& InputState)
	{
		return InputState.bAltKeyDown;
	}

	/** @return true if Apple Command key is down in input state */
	static bool IsCmdKeyDown(const FInputDeviceState& InputState)
	{
		return InputState.bCmdKeyDown;
	}
};




/**
 * FInputDeviceRay represents a 3D ray created based on an input device.
 * If the device is a 2D input device like a mouse, then the ray may
 * have an associated 2D screen position.
 */
USTRUCT(BlueprintType)
struct FInputDeviceRay
{
	GENERATED_BODY()

	/** 3D ray in 3D scene, in world coordinates */
	UPROPERTY(transient, BlueprintReadWrite, Category = InputDeviceRay)
	FRay WorldRay;

	/** If true, WorldRay has 2D device position coordinates */
	UPROPERTY(transient, BlueprintReadWrite, Category = InputDeviceRay)
	bool bHas2D = false;

	/** 2D device position coordinates associated with the ray */
	UPROPERTY(transient, BlueprintReadWrite, Category = InputDeviceRay)
	FVector2D ScreenPosition;

	// this is required for a USTRUCT
	FInputDeviceRay()
	{
		WorldRay = FRay();
		bHas2D = false;
		ScreenPosition = FVector2D(0, 0);
	}

	explicit FInputDeviceRay(const FRay& WorldRayIn)
	{
		WorldRay = WorldRayIn;
		bHas2D = false;
		ScreenPosition = FVector2D(0, 0);
	}

	FInputDeviceRay(const FRay& WorldRayIn, const FVector2D& ScreenPositionIn)
	{
		WorldRay = WorldRayIn;
		bHas2D = true;
		ScreenPosition = ScreenPositionIn;
	}

	FInputDeviceRay(const FInputDeviceState& Input)
	{
		if (Input.IsFromDevice(EInputDevices::Mouse))
		{
			WorldRay = Input.Mouse.WorldRay;
			bHas2D = true;
			ScreenPosition = Input.Mouse.Position2D;
		}
		else
		{
			ensure(false);
			WorldRay = FRay(FVector::ZeroVector, FVector(0, 0, 1), true);
			ScreenPosition = FVector2D(0, 0);
			bHas2D = false;
		}
	}
};


