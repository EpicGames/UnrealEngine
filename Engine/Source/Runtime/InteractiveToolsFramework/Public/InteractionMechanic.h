// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehaviorSet.h"
#include "InteractiveTool.h"
#include "ToolContextInterfaces.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "InteractionMechanic.generated.h"

class IToolsContextRenderAPI;
class UInteractiveTool;
class UInteractiveToolPropertySet;

/**
 * A UInteractionMechanic implements a "user interaction". This is generally a subset of an InteractiveTool,
 * for example an interaction to draw a polygon could be used in many tools, but requires handling input events
 * and converting them to click points on a plane, handling various cases like closing a loop, undo/redo of points, etc.
 * Ideally all these aspects should be able to be wrapped up in an UInteractionMechanic that multiple Tools can use.
 * 
 * (This class is still a work in progress)
 */
UCLASS(Transient, MinimalAPI)
class UInteractionMechanic : public UObject
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UInteractionMechanic();

	/**
	 * Called to initialize the InteractionMechanic
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup(UInteractiveTool* ParentTool);

	/**
	 * Called to clean up the InteractionMechanic
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

	/**
	 * Allow the Mechanic to do any custom drawing (ie via PDI/RHI)
	 * @param RenderAPI Abstraction that provides access to Rendering in the current ToolsContext
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI);

	/**
	 * ALlow the Mechanic to Tick
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime);

protected:
	TWeakObjectPtr<UInteractiveTool> ParentTool;

	INTERACTIVETOOLSFRAMEWORK_API UInteractiveTool* GetParentTool() const;

	/**
	 * Add a PropertySet object for this Mechanic to parent Tool
	 * @param PropertySet Property Set object to add
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddToolPropertySource(UInteractiveToolPropertySet* PropertySet);

	/** Enable/Disable a PropertySet object for the parent tool. */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SetToolPropertySourceEnabled(UInteractiveToolPropertySet* PropertySet, bool bEnabled);
};

