// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolContextInterfaces.h"
#include "InteractiveCommand.generated.h"

class IToolsContextTransactionsAPI;

/**
 * UInteractiveCommandArguments are arguments passed to a UInteractiveCommand.
 * Subclasses of UInteractiveCommand will generally be paired with subclasses of UInteractiveCommandArguments.
 * 
 * The base UInteractiveCommandArguments provides support for transactions via 
 * a IToolsContextTransactionsAPI
 */
UCLASS(MinimalAPI)
class UInteractiveCommandArguments : public UObject
{
	GENERATED_BODY()

public:
	virtual void SetTransactionsAPI(IToolsContextTransactionsAPI* TransactionsAPIIn) { TransactionsAPI = TransactionsAPIIn;	}
	virtual bool HasTransactionsAPI() const { return TransactionsAPI != nullptr; }
	virtual IToolsContextTransactionsAPI* GetTransactionsAPI() const { return TransactionsAPI; }

protected:
	IToolsContextTransactionsAPI* TransactionsAPI = nullptr;

};


/**
 * UInteractiveCommandResult subclasses are returned from UInteractiveCommands, to allow
 * commands to return custom information.
 */
UCLASS(MinimalAPI)
class UInteractiveCommandResult : public UObject
{
	GENERATED_BODY()

};


/**
 * A UInteractiveCommand is an atomic action that can be executed via some user interaction.
 * For example clicking a button that deletes an active selection can be considered an Interactive Command.
 * This differs from an Interactive Tool in that there is no ongoing user interaction once the
 * command has been initiated. 
 */
UCLASS(Abstract, MinimalAPI)
class UInteractiveCommand : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * @return a short text string that can be used for the command in (eg) Editor transaction/undo toasts
	 */
	virtual FText GetCommandShortString() const 
	{ 
		return FText(); 
	}

	/**
	 * @return true if it is safe to call ExecuteCommand() with the given Arguments
	 */
	virtual bool CanExecuteCommand(UInteractiveCommandArguments* Arguments) 
	{ 
		return false;
	}

	/**
	 * Execute the command with the given Arguments
	 * @param Result optional command result. Command would have to allocate the UInteractiveCommandResult object to return it, but is not required to. Caller should not assume that a non-null Result is necessarily returned.
	 */
	virtual void ExecuteCommand(UInteractiveCommandArguments* Arguments, UInteractiveCommandResult** Result = nullptr)
	{
	}
};
