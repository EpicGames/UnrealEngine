// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehaviorSet.h"
#include "Misc/EnumClassFlags.h"
#include "ToolContextInterfaces.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InteractiveGizmo.generated.h"

class FCanvas;
class IToolsContextRenderAPI;
class UInputBehavior;
class UInteractiveGizmoManager;

/**
 * UInteractiveGizmo is the base class for all Gizmos in the InteractiveToolsFramework.
 *
 * @todo callback/delegate for if/when .InputBehaviors changes
 * @todo callback/delegate for when Gizmo properties change
 */
UCLASS(Transient, MinimalAPI)
class UInteractiveGizmo : public UObject, public IInputBehaviorSource
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UInteractiveGizmo();

	/**
	 * Called by GizmoManager to initialize the Gizmo *after* GizmoBuilder::BuildGizmo() has been called
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup();

	/**
	 * Called by GizmoManager to shut down the Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

	/**
	 * Allow the Gizmo to do any custom drawing (ie via PDI/RHI)
	 * @param RenderAPI Abstraction that provides access to Rendering in the current ToolsContext
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI);

	/**
	 * Allow the Gizmo to do any custom screen space drawing
	 * @param Canvas the FCanvas to use to do the drawing
	 * @param RenderAPI Abstraction that provides access to Rendering in the current ToolsContext
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD( FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI );

	/**
	 * Allow the Gizmo to do any necessary processing on Tick
	 * @param DeltaTime the time delta since last tick
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime);



	/**
	 * @return GizmoManager that owns this Gizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmoManager* GetGizmoManager() const;



	//
	// Input Behaviors support
	//

	/**
	 * Add an input behavior for this Gizmo
	 * @param Behavior behavior to add
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddInputBehavior(UInputBehavior* Behavior);

	/**
	 * Remove an input behavior from this Gizmo
	 * @param Behavior behavior to add
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RemoveInputBehavior(UInputBehavior* Behavior);

	/**
	 * @return Current input behavior set.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual const UInputBehaviorSet* GetInputBehaviors() const;



protected:

	/** The current set of InputBehaviors provided by this Gizmo */
	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> InputBehaviors;
};



/**
 * ETransformGizmoSubElements identifies the sub-elements of a standard 3-axis transformation Gizmo.
 * Used by GizmoManager to customize UCombinedTransformGizmo instances.
 */
UENUM()
enum class ETransformGizmoSubElements
{
	None = 0,

	TranslateAxisX = 1<<1,
	TranslateAxisY = 1<<2,
	TranslateAxisZ = 1<<3,
	TranslateAllAxes = TranslateAxisX | TranslateAxisY | TranslateAxisZ,

	TranslatePlaneXY = 1<<4,
	TranslatePlaneXZ = 1<<5,
	TranslatePlaneYZ = 1<<6,
	TranslateAllPlanes = TranslatePlaneXY | TranslatePlaneXZ | TranslatePlaneYZ,

	RotateAxisX = 1<<7,
	RotateAxisY = 1<<8,
	RotateAxisZ = 1<<9,
	RotateAllAxes = RotateAxisX | RotateAxisY | RotateAxisZ,

	ScaleAxisX = 1<<10,
	ScaleAxisY = 1<<11,
	ScaleAxisZ = 1<<12,
	ScaleAllAxes = ScaleAxisX | ScaleAxisY | ScaleAxisZ,

	ScalePlaneYZ = 1<<13,
	ScalePlaneXZ = 1<<14,
	ScalePlaneXY = 1<<15,
	ScaleAllPlanes = ScalePlaneXY | ScalePlaneXZ | ScalePlaneYZ,

	ScaleUniform = 1<<16,

	FreeTranslate = 1<<17,
	FreeRotate = 1<<18,

	StandardTranslateRotate = TranslateAllAxes | TranslateAllPlanes | RotateAllAxes 
		| FreeTranslate | FreeRotate,
	TranslateRotateUniformScale = TranslateAllAxes | TranslateAllPlanes | RotateAllAxes 
		| ScaleUniform | FreeTranslate | FreeRotate,
	FullTranslateRotateScale = TranslateAllAxes | TranslateAllPlanes | RotateAllAxes 
		| ScaleAllAxes | ScaleAllPlanes | ScaleUniform | FreeTranslate | FreeRotate
};
ENUM_CLASS_FLAGS(ETransformGizmoSubElements);
