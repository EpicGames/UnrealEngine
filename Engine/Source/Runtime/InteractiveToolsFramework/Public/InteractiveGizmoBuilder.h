// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.generated.h"



/**
 * A UInteractiveGizmoBuilder creates a new instance of an InteractiveGizmo (basically this is a Factory).
 * These are registered with the InteractiveGizmoManager, which calls BuildGizmo().
 * This is an abstract base class, you must subclass it in order to create your particular Gizmo instance
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UInteractiveGizmoBuilder : public UObject
{
	GENERATED_BODY()

public:

	/** 
	 * Create a new instance of this builder's Gizmo
	 * @param SceneState the current scene selection state, etc
	 * @return a new instance of the Gizmo, or nullptr on error/failure
	 */
	virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const
	{
		unimplemented();
		return nullptr;
	}
};
