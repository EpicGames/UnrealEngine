// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "InputRouter.h"
#include "InteractiveToolChange.h"
#include "ToolContextInterfaces.h"
#include "InteractiveGizmoManager.generated.h"

class FCombinedTransformGizmoActorFactory;
class UCombinedTransformGizmo;
class UCombinedTransformGizmoBuilder;
class UContextObjectStore;

USTRUCT()
struct FActiveGizmo
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UInteractiveGizmo> Gizmo = nullptr;
	FString BuilderIdentifier;
	FString InstanceIdentifier;
	void* Owner = nullptr;
};


/**
 * UInteractiveGizmoManager allows users of the Tools framework to create and operate Gizmo instances.
 * For each Gizmo, a (string,GizmoBuilder) pair is registered with the GizmoManager.
 * Gizmos can then be activated via the string identifier.
 * 
 */
UCLASS(Transient, MinimalAPI)
class UInteractiveGizmoManager : public UObject, public IToolContextTransactionProvider
{
	GENERATED_BODY()

protected:
	friend class UInteractiveToolsContext;		// to call Initialize/Shutdown

	INTERACTIVETOOLSFRAMEWORK_API UInteractiveGizmoManager();

	/** Initialize the GizmoManager with the necessary Context-level state. UInteractiveToolsContext calls this, you should not. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IToolsContextQueriesAPI* QueriesAPI, IToolsContextTransactionsAPI* TransactionsAPI, UInputRouter* InputRouter);

	/** Shutdown the GizmoManager. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

public:

	//
	// GizmoBuilder Registration and Gizmo Creation/Shutdown
	//

	/**
	 * Register a new GizmoBuilder
	 * @param BuilderIdentifier string used to identify this Builder
	 * @param Builder new GizmoBuilder instance
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RegisterGizmoType(const FString& BuilderIdentifier, UInteractiveGizmoBuilder* Builder);

	/**
	 * Remove a GizmoBuilder from the set of known GizmoBuilders
	 * @param BuilderIdentifier identification string that was passed to RegisterGizmoType()
	 * @return true if Builder was found and deregistered
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DeregisterGizmoType(const FString& BuilderIdentifier);



	/**
	 * Try to activate a new Gizmo instance
	 * @param BuilderIdentifier string used to identify Builder that should be called
	 * @param InstanceIdentifier optional client-defined string that can be used to locate this instance (must be unique across all Gizmos)
	 * @param Owner void pointer to whatever "owns" this Gizmo. Allows Gizmo to later be deleted using DestroyAllGizmosByOwner()
	 * @return new Gizmo instance that has been created and initialized
	 */	
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* CreateGizmo(const FString& BuilderIdentifier, const FString& InstanceIdentifier = FString(), void* Owner = nullptr);


	/**
	 * Try to activate a new Gizmo instance 
	 * @param BuilderIdentifier string used to identify Builder that should be called
	 * @param InstanceIdentifier optional client-defined string that can be used to locate this instance (must be unique across all Gizmos)
	 * @param Owner void pointer to whatever "owns" this Gizmo. Allows Gizmo to later be deleted using DestroyAllGizmosByOwner()
	 * @return new Gizmo instance that has been created and initialized, and cast to template type
	 */
	template<typename GizmoType>
	GizmoType* CreateGizmo(const FString& BuilderIdentifier, const FString& InstanceIdentifier = FString(), void* Owner = nullptr)
	{
		UInteractiveGizmo* NewGizmo = CreateGizmo(BuilderIdentifier, InstanceIdentifier, Owner);
		return CastChecked<GizmoType>(NewGizmo);
	}


	/**
	 * Shutdown and remove a Gizmo
	 * @param Gizmo the Gizmo to shutdown and remove
	 * @return true if the Gizmo was found and removed
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool DestroyGizmo(UInteractiveGizmo* Gizmo);

	/**
	 * Destroy all Gizmos that were created by the identified GizmoBuilder
	 * @param BuilderIdentifier the Builder string registered with RegisterGizmoType
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DestroyAllGizmosOfType(const FString& BuilderIdentifier);

	/**
	 * Destroy all Gizmos that are owned by the given pointer
	 * @param Owner pointer that was passed to CreateGizmo
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DestroyAllGizmosByOwner(void* Owner);


	/**
	 * Find all the existing Gizmo instances that were created by the identified GizmoBuilder
	 * @param BuilderIdentifier the Builder string registered with RegisterGizmoType
	 * @return list of found Gizmos
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual TArray<UInteractiveGizmo*> FindAllGizmosOfType(const FString& BuilderIdentifier);

	/**
	 * Find the Gizmo that was created with the given instance identifier
	 * @param Identifier the InstanceIdentifier that was passed to CreateGizmo()
	 * @return the found Gizmo, or null
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveGizmo* FindGizmoByInstanceIdentifier(const FString& Identifier) const;




	//
	// Functions that Gizmos can call to interact with Transactions API
	//
	
	/** Post a message via the Transactions API */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DisplayMessage(const FText& Message, EToolMessageLevel Level);

	/** Request an Invalidation via the Transactions API (ie to cause a repaint, etc) */
	INTERACTIVETOOLSFRAMEWORK_API virtual void PostInvalidation();

	/**
	 * Request that the Context open a Transaction, whatever that means to the current Context
	 * @param Description text description of this transaction (this is the string that appears on undo/redo in the UE Editor)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginUndoTransaction(const FText& Description);

	/** Request that the Context close and commit the open Transaction */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndUndoTransaction();
	
	/** Request that the Context cancel and revert the open Transaction */
	INTERACTIVETOOLSFRAMEWORK_API virtual void CancelUndoTransaction() override;

	/**
	 * Forward an FChange object to the Context
	 * @param TargetObject the object that the FChange applies to
	 * @param Change the change object that the Context should insert into the transaction history
	 * @param Description text description of this change (this is the string that appears on undo/redo in the UE Editor)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EmitObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description );


	//
	// State control  (@todo: have the Context call these? not safe for anyone to call)
	//

	/** Tick any active Gizmos. Called by UInteractiveToolsContext */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime);

	/** Render any active Gizmos. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI);

	/** Let active Gizmos do screen space drawing.  Called by UInteractiveToolsContext */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD( FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI );

	//
	// access to APIs, etc
	//

	/** @return current IToolsContextQueriesAPI */
	virtual IToolsContextQueriesAPI* GetContextQueriesAPI() { return QueriesAPI; }

	/**
	 * @return the context object store from the owning tools context.
	 */
	INTERACTIVETOOLSFRAMEWORK_API UContextObjectStore* GetContextObjectStore() const;



public:
	//
	// Standard Gizmos
	// 

	/**
	 * Register default gizmo types
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RegisterDefaultGizmos();

	/**
	 * Activate a new instance of the default 3-axis transformation Gizmo. RegisterDefaultGizmos() must have been called first.
	 * @param Owner optional void pointer to whatever "owns" this Gizmo. Allows Gizmo to later be deleted using DestroyAllGizmosByOwner()
	 * @param InstanceIdentifier optional client-defined *unique* string that can be used to locate this instance
	 * @return new Gizmo instance that has been created and initialized
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UCombinedTransformGizmo* Create3AxisTransformGizmo(void* Owner = nullptr, const FString& InstanceIdentifier = FString());

	/**
	 * Activate a new customized instance of the default 3-axis transformation Gizmo, with only certain elements included. RegisterDefaultGizmos() must have been called first.
	 * @param Elements flags that indicate which standard gizmo sub-elements should be included
	 * @param Owner optional void pointer to whatever "owns" this Gizmo. Allows Gizmo to later be deleted using DestroyAllGizmosByOwner()
	 * @param InstanceIdentifier optional client-defined *unique* string that can be used to locate this instance
	 * @return new Gizmo instance that has been created and initialized
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UCombinedTransformGizmo* CreateCustomTransformGizmo(ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());

	INTERACTIVETOOLSFRAMEWORK_API virtual UCombinedTransformGizmo* CreateCustomRepositionableTransformGizmo(ETransformGizmoSubElements Elements, void* Owner = nullptr, const FString& InstanceIdentifier = FString());

public:
	// builder identifiers for default gizmo types. Perhaps should have an API for this...
	static INTERACTIVETOOLSFRAMEWORK_API FString DefaultAxisPositionBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API FString DefaultPlanePositionBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API FString DefaultAxisAngleBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API FString DefaultThreeAxisTransformBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString CustomThreeAxisTransformBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API const FString CustomRepositionableThreeAxisTransformBuilderIdentifier;
	static INTERACTIVETOOLSFRAMEWORK_API FString DefaultScalableSphereBuilderIdentifier;

protected:
	/** set of Currently-active Gizmos */
	UPROPERTY()
	TArray<FActiveGizmo> ActiveGizmos;


protected:
	/** Current Context-Queries implementation */
	IToolsContextQueriesAPI* QueriesAPI;
	/** Current Transactions implementation */
	IToolsContextTransactionsAPI* TransactionsAPI;

	/** Current InputRouter (Context owns this) */
	UInputRouter* InputRouter;

	/** Current set of named GizmoBuilders */
	UPROPERTY()
	TMap<FString, TObjectPtr<UInteractiveGizmoBuilder>> GizmoBuilders;

	bool bDefaultGizmosRegistered = false;
	TSharedPtr<FCombinedTransformGizmoActorFactory> GizmoActorBuilder;
};
