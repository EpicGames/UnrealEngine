// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "InputBehaviorSet.h"
#include "InteractiveToolActionSet.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Shader.h"
#include "Templates/EnableIf.h"
#include "Templates/Function.h"
#include "Templates/Models.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "ToolContextInterfaces.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "InteractiveTool.generated.h"

class FCanvas;
class FInteractiveToolActionSet;
class FProperty;
class IToolsContextRenderAPI;
class UInputBehavior;
class UInteractiveTool;
class UInteractiveToolManager;
struct FPropertyChangedEvent;
struct FSlateBrush;

/** Passed to UInteractiveTool::Shutdown to indicate how Tool should shut itself down*/
UENUM(BlueprintType)
enum class EToolShutdownType : uint8
{
	/** Tool cleans up and exits. Pass this to tools that do not have Accept/Cancel options. */
	Completed = 0,
	/** Tool commits current preview to scene */
	Accept = 1,
	/** Tool discards current preview without modifying scene */
	Cancel = 2
};

/** Used in various ways by the tool manager and some optional tool interfaces to make shutdown responses configurable. */
enum class EToolShutdownReason : uint8
{
	/** The reason for the shutdown was left unspecified. */
	Unspecified = 0,
	/** Another tool is being activated without the user having explicitly shut down this one. */
	SwitchTool = 1
};

/**
 * FInteractiveToolInfo provides information about a tool (name, tooltip, etc)
 */
struct FInteractiveToolInfo
{
	/** Name of Tool. May be FText::Empty(), but will default to Tool->GetClass()->GetDisplayNameText() in InteractiveTool constructor */
	FText ToolDisplayName = FText::GetEmpty();
	/** Display message that can be used as a tool tip or to provide information about the tool. */
	FText ToolDisplayMessage = FText::GetEmpty();
	/** The icon to use for the tool ui command. */
	const FSlateBrush* ToolIcon = nullptr;
};


class FWatchablePropertySet
{
public:
	//
	// Property watching infrastructure
	//
	class FPropertyWatcher
	{
	public:
		virtual ~FPropertyWatcher() = default;
		virtual void CheckAndUpdate() = 0;
		virtual void SilentUpdate() = 0;
	};

	template <typename PropType>
	class TPropertyWatcher : public FPropertyWatcher
	{
	public:
		using FValueGetter = TFunction<PropType(void)>;
		using FChangedCallback = TFunction<void(const PropType&)>;
		using FNotEqualTestFunction = TFunction<bool(const PropType&, const PropType&)>;	// Define "!=" for PropType

		/**
		 * Describes a type having a "!=" comparasion operator. (Note: it would probably be cleaner to require operator==,
		 * however some types have operator!= but no operator==, and we do only use != in the code.)
		*/
		struct CInequalityComparable
		{
			template <typename T>
			auto Requires(bool& Result, const T& A, const T& B) -> decltype(
				Result = A != B
				);
		};

		// If PropType is CInequalityComparable, allow the two-argument constructor and default to using the type's 
		// existing != operator.
		// If PropType is not CInequalityComparable, the caller must use the three-argument constructor to provide a
		// "not equal" function.

		// Two-argument constructor, enabled only if PropType is CInequalityComparable
		template<typename Q = PropType, typename = typename TEnableIf<TModels_V<CInequalityComparable, Q>>::Type>
		TPropertyWatcher(const PropType& Property,
						 FChangedCallback OnChangedIn)
			: Cached(),
			GetValue([&Property]() {return Property; }),
			OnChanged(MoveTemp(OnChangedIn)),
			NotEqual([](const PropType& A, const PropType& B) { return A != B; })
		{}

		// Two-argument constructor, enabled only if PropType is CInequalityComparable
		template<typename Q = PropType, typename = typename TEnableIf<TModels_V<CInequalityComparable, Q>>::Type>
		TPropertyWatcher(FValueGetter GetValueIn,
						 FChangedCallback OnChangedIn)
			: Cached(),
			GetValue(MoveTemp(GetValueIn)),
			OnChanged(MoveTemp(OnChangedIn)),
			NotEqual([](const PropType& A, const PropType& B) { return A != B; })
		{}
	
		TPropertyWatcher(const PropType& Property,
						 FChangedCallback OnChangedIn,
						 FNotEqualTestFunction NotEqualIn)
			: Cached(),
			GetValue([&Property]() {return Property; }),
			OnChanged(MoveTemp(OnChangedIn)),
			NotEqual(MoveTemp(NotEqualIn))
		{}

		TPropertyWatcher(FValueGetter GetValueIn,
						 FChangedCallback OnChangedIn,
						 FNotEqualTestFunction NotEqualIn)
			: Cached(),
			GetValue(MoveTemp(GetValueIn)),
			OnChanged(MoveTemp(OnChangedIn)),
			NotEqual(MoveTemp(NotEqualIn))
		{}

		void CheckAndUpdate() final
		{
			PropType Value = GetValue();
			if ((!Cached.IsSet()) || NotEqual(Cached.GetValue(), Value))
			{
				Cached = Value;
				OnChanged(Cached.GetValue());
			}
		}

		void SilentUpdate() final
		{
			Cached = GetValue();
		}

	private:
		TOptional<PropType> Cached;
		FValueGetter GetValue;
		FChangedCallback OnChanged;
		FNotEqualTestFunction NotEqual;
	};

	FWatchablePropertySet() = default;
	FWatchablePropertySet(const FWatchablePropertySet&) = delete;
	FWatchablePropertySet& operator=(const FWatchablePropertySet&) = delete;

	void CheckAndUpdateWatched()
	{
		for ( auto& PropWatcher : PropertyWatchers )
		{
			PropWatcher->CheckAndUpdate();
		}
	}
	void SilentUpdateWatched()
	{
		for ( auto& PropWatcher : PropertyWatchers )
		{
			PropWatcher->SilentUpdate();
		}
	}
	/** 
	 * Silently updates just a single watcher, using an index gotten from WatchProperty.
	 * Useful when you want watching to still work for other properties changed in the
	 * same tick.
	 */
	void SilentUpdateWatcherAtIndex(int32 i)
	{
		check(i >= 0 && i < PropertyWatchers.Num());
		PropertyWatchers[i]->SilentUpdate();
	}

	/** @return Index of the watcher, which can be used in SilentUpdateWatcherAtIndex */
	template <typename PropType>
	int32 WatchProperty(const PropType& ValueIn,
					   typename TPropertyWatcher<PropType>::FChangedCallback OnChangedIn)
	{
		return PropertyWatchers.Emplace(MakeUnique<TPropertyWatcher<PropType>>(ValueIn, OnChangedIn));
	}
	/** @return Index of the watcher, which can be used in SilentUpdateWatcherAtIndex */
	template <typename PropType>
	int32 WatchProperty(typename TPropertyWatcher<PropType>::FValueGetter GetValueIn,
					   typename TPropertyWatcher<PropType>::FChangedCallback OnChangedIn)
	{
		return PropertyWatchers.Emplace(MakeUnique<TPropertyWatcher<PropType>>(GetValueIn, OnChangedIn));
	}

	/** @return Index of the watcher, which can be used in SilentUpdateWatcherAtIndex */
	template <typename PropType>
	int32 WatchProperty(const PropType& ValueIn,
						typename TPropertyWatcher<PropType>::FChangedCallback OnChangedIn,
						typename TPropertyWatcher<PropType>::FNotEqualTestFunction NotEqualsIn)
	{
		return PropertyWatchers.Emplace(MakeUnique<TPropertyWatcher<PropType>>(ValueIn, OnChangedIn, NotEqualsIn));
	}
	/** @return Index of the watcher, which can be used in SilentUpdateWatcherAtIndex */
	template <typename PropType>
	int32 WatchProperty(typename TPropertyWatcher<PropType>::FValueGetter GetValueIn,
					  typename TPropertyWatcher<PropType>::FChangedCallback OnChangedIn,
					  typename TPropertyWatcher<PropType>::FNotEqualTestFunction NotEqualsIn)
	{
		return PropertyWatchers.Emplace(MakeUnique<TPropertyWatcher<PropType>>(GetValueIn, OnChangedIn, NotEqualsIn));
	}

private:
	TArray<TUniquePtr<FPropertyWatcher>> PropertyWatchers;
};

/** This delegate is used by UInteractiveToolPropertySet */
DECLARE_MULTICAST_DELEGATE_TwoParams(FInteractiveToolPropertySetModifiedSignature, UObject*, FProperty*);


/**
 * A UInteractiveTool contains a set of UObjects that contain "properties" of the Tool, ie
 * the configuration flags, parameters, etc that control the Tool. Currently any UObject
 * can be added as a property set, however there is no automatic mechanism for those child
 * UObjects to notify the Tool when a property changes.
 *
 * If you make your property set UObjects subclasses of UInteractiveToolPropertySet, then
 * when the Tool Properties are changed *in the Editor*, the parent Tool will be automatically notified.
 * You can override UInteractiveTool::OnPropertyModified() to act on these notifications
 */
UCLASS(Transient, MinimalAPI)
class UInteractiveToolPropertySet : public UObject, public FWatchablePropertySet
{
	GENERATED_BODY()

public:
	/** @return the multicast delegate that is called when properties are modified */
	FInteractiveToolPropertySetModifiedSignature& GetOnModified()
	{
		return OnModified;
	}

	/** Return true if this property set is enabled. Enabled/Disable state is intended to be used to control things like visibility in UI/etc. */
	bool IsPropertySetEnabled() const
	{
		return bIsPropertySetEnabled;
	}

	//
	// Setting saving/serialization
	//
	/**
	 * Save and restore values of current Tool Properties between tool invocations. The standard usage of
	 * this setup is to call PropertySet->RestoreProperties() in the UInteractiveTool::Setup() implementation,
	 * and PropertySet->SaveProperties() in the UInteractiveTool::Shutdown() implementation.
	 *
	 * The default behaviour of these functions is to Save or Restore every property in the property set.  It is not
	 * necessary to save/restore all possible Properties (in many cases this would not make sense), so individual
	 * properties may be skipped by adding the "TransientToolProperty" tag to their metadata on a property by property basis.
	 *
	 * Property sets which need more exotic behaviour upon Save and Restore may override these routines
	 *
	 * GetDynamicPropertyCache() can be used to return an instance of the specified property set subclass which 
	 * may be used as a place to save/restore these properties by customized Save/Restore functions
	 * 
	 * Note: the current design of this system assumes that the CDO will keep the referenced objects alive.
	 * This assumption is incorrect in Runtime builds, and some external mechanism must be used to keep
	 * the elements in the CachedPropertiesMap alive.
	 */

	/**
	 * Save the values of this PropertySet with the given CacheIdentifier.
	 * The Tool parameter is currently ignored.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void SaveProperties(UInteractiveTool* SaveFromTool, const FString& CacheIdentifier = TEXT(""));

	/**
	 * Restore the values of the Property Set with the given CacheIdentifier.
	 * The Tool parameter is currently ignored.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier = TEXT(""));

protected:
	// Utility func used to implement the default Save/RestoreProperties funcs
	INTERACTIVETOOLSFRAMEWORK_API virtual void SaveRestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier, bool bSaving);

	/**
	 * GetDynamicPropertyCache return class-internal objects that subclasses can use to save/restore properties.
	 * @param CacheIdentifier multiple versions of the UInteractiveToolPropertySet can be stored, CacheIdentifier indicates which one to use
	 * @param bWasCreatedOut true is returned here if this is the first time the object was seen
	 * @return instance of the current subclass that can be used to save/restore values
	 */
	UE_DEPRECATED(5.8, "Use FindDynamicPropertyCache or FindOrAddDynamicPropertyCache instead.")
	INTERACTIVETOOLSFRAMEWORK_API TObjectPtr<UInteractiveToolPropertySet> GetDynamicPropertyCache(const FString& CacheIdentifier, bool& bWasCreatedOut);
	
	/**
	 * FindDynamicPropertyCache returns class-internal objects that subclasses can use to save/restore properties, or nullptr if
	 * the cache entry does not exist.
	 * @param CacheIdentifier multiple versions of the UInteractiveToolPropertySet can be stored, CacheIdentifier indicates which one to use
	 * @return instance of the current subclass that can be used to save/restore values, or nullptr if the entry does not exist.
	 */
	INTERACTIVETOOLSFRAMEWORK_API TObjectPtr<UInteractiveToolPropertySet> FindDynamicPropertyCache(const FString& CacheIdentifier) const;
	
	/**
	 * FindOrAddDynamicPropertyCache returns class-internal objects that subclasses can use to save/restore properties. Creates
	 * a cache entry if one does not yet exist.
	 * @param CacheIdentifier multiple versions of the UInteractiveToolPropertySet can be stored, CacheIdentifier indicates which one to use
	 * @return instance of the current subclass that can be used to save/restore values
	 */
	INTERACTIVETOOLSFRAMEWORK_API TObjectPtr<UInteractiveToolPropertySet> FindOrAddDynamicPropertyCache(const FString& CacheIdentifier) const;

public:
#if WITH_EDITOR
	/**
	  * Posts a message to the OnModified delegate with the modified FProperty
	  * @warning Please consider listening to OnModified instead of overriding this function
	  * @warning this function is currently only called in Editor (not at runtime)
	  */
	INTERACTIVETOOLSFRAMEWORK_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif

protected:
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TMap<FString, TObjectPtr<UInteractiveToolPropertySet>> CachedPropertiesMap;

	// Controls whether a property set is shown in the UI.  Transient so that disabling a PropertySet in one tool doesn't disable it in others.
	UPROPERTY(Transient, DuplicateTransient, SkipSerialization, meta=(TransientToolProperty))
	bool bIsPropertySetEnabled = true;

	friend class UInteractiveTool;	// so that tool can enable/disable

	FInteractiveToolPropertySetModifiedSignature OnModified;
};


/**
 * UInteractiveTool is the base class for all Tools in the InteractiveToolsFramework.
 * A Tool is is a "lightweight mode" that may "own" one or more Actors/Components/etc in
 * the current scene, may capture certain input devices or event streams, and so on.
 * The base implementation essentially does nothing but provide sane default behaviors.
 *
 * The BaseTools/ subfolder contains implementations of various kinds of standard
 * "tool behavior", like a tool that responds to a mouse click, etc, that can be
 * extended to implement custom behaviors.
 *
 * In the framework, you do not create instances of UInteractiveTool yourself.
 * You provide a UInteractiveToolBuilder implementation that can properly construct
 * an instance of your Tool, this is where for example default parameters would be set.
 * The ToolBuilder is registered with the ToolManager, and then UInteractiveToolManager::ActivateTool()
 * is used to kick things off.
 *
 * @todo callback/delegate for if/when .InputBehaviors changes
 * @todo callback/delegate for when tool properties change
 */
UCLASS(Transient, MinimalAPI)
class UInteractiveTool : public UObject, public IInputBehaviorSource
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UInteractiveTool();

	/**
	 * Called by ToolManager to initialize the Tool *after* ToolBuilder::BuildTool() has been called
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Setup();

	/**
	 * Called by ToolManager to shut down the Tool
	 * @param ShutdownType indicates how the tool should shutdown (ie Accept or Cancel current preview, etc)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown(EToolShutdownType ShutdownType);

	/**
	 * Allow the Tool to do any custom drawing (ie via PDI/RHI)
	 * @param RenderAPI Abstraction that provides access to Rendering in the current ToolsContext
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI);

	/**
	 * Allow the Tool to do any custom screen space drawing
	 * @param Canvas the FCanvas to use to do the drawing
	 * @param RenderAPI Abstraction that provides access to Rendering in the current ToolsContext
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD( FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI );

	/**
	 * Non overrideable func which does processing and calls the tool's OnTick
	 * @param DeltaTime the time delta since last tick
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime) final;

	/**
	 * @return ToolManager that owns this Tool
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveToolManager* GetToolManager() const;


	/**
	 * @return true if this Tool support being Cancelled, ie calling Shutdown(EToolShutdownType::Cancel)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasCancel() const;

	/**
	 * @return true if this Tool support being Accepted, ie calling Shutdown(EToolShutdownType::Accept)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasAccept() const;

	/**
	 * @return true if this Tool is currently in a state where it can be Accepted. This may be false if for example there was an error in the Tool.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanAccept() const;



	//
	// Input Behaviors support
	//

	/**
	 * Add an input behavior for this Tool. Typically only effective when called during tool Setup(), as
	 * the behavior set gets submitted to the input router after that call.
	 * @param Behavior behavior to add
	 * @param Source Optional pointer that could be used to identify the behavior for removal later.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddInputBehavior(UInputBehavior* Behavior, void* Source = nullptr);

	/**
	 * Remove all input behaviors that had the given source pointer set during their addition.
	 * Typically only effective when called during tool Setup(), as the behavior set gets submitted
	 * to the input router after that call.
	 * @param Source Identifying pointer
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RemoveInputBehaviorsBySource(void* Source);

	/**
	 * @return Current input behavior set.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual const UInputBehaviorSet* GetInputBehaviors() const;


	//
	// Property support
	//

	/**
	 * @return list of property UObjects for this tool (ie to add to a DetailsViewPanel, for example)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual TArray<UObject*> GetToolProperties(bool bEnabledOnly = true) const;

	/**
	 * OnPropertySetsModified is broadcast whenever the contents of the ToolPropertyObjects array is modified
	 */
	DECLARE_MULTICAST_DELEGATE(OnInteractiveToolPropertySetsModified);
	OnInteractiveToolPropertySetsModified OnPropertySetsModified;

	/**
	 * OnPropertyModifiedDirectlyByTool is broadcast whenever the ToolPropertyObjects array stays the same, but a property
	 * inside of one of the objects is changed internally by the tool. This allows any external display of such properties
	 * to properly update. In a DetailsViewPanel, for instance, it refreshes certain cached states such as edit condition
	 * states for other properties.
	 * 
	 * This should only broadcast when the tool itself is responsible for the change, so it typically isn't broadcast
	 * from the tool's OnPropertyModified function. 
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(OnInteractiveToolPropertyInternallyModified, UObject*);
	OnInteractiveToolPropertyInternallyModified OnPropertyModifiedDirectlyByTool;

	/**
	 * Automatically called by UInteractiveToolPropertySet.OnModified delegate to notify Tool of child property set changes
	 * @param PropertySet which UInteractiveToolPropertySet was modified
	 * @param Property which FProperty in the set was modified
	 */
	virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property)
	{
	}


protected:

	/** The current set of InputBehaviors provided by this Tool */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UInputBehaviorSet> InputBehaviors;

	/** The current set of Property UObjects provided by this Tool. May contain pointer to itself. */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UObject>> ToolPropertyObjects;

	/**
	 * Add a Property object for this Tool
	 * @param Property object to add
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddToolPropertySource(UObject* PropertyObject);

	/**
	 * Add a PropertySet object for this Tool
	 * @param PropertySet Property Set object to add
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddToolPropertySource(UInteractiveToolPropertySet* PropertySet);

	/**
	 * Remove a PropertySet object from this Tool. If found, will broadcast OnPropertySetsModified
	 * @param PropertySet property set to remove.
	 * @return true if PropertySet is found and removed
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool RemoveToolPropertySource(UInteractiveToolPropertySet* PropertySet);

	/**
	 * Replace a PropertySet object on this Tool with another property set. If replaced, will broadcast OnPropertySetsModified
	 * @param CurPropertySet property set to remove
	 * @param ReplaceWith property set to add
	 * @param bSetToEnabled if true, ReplaceWith property set is explicitly enabled (otherwise enable/disable state is unmodified)
	 * @return true if CurPropertySet is found and replaced
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool ReplaceToolPropertySource(UInteractiveToolPropertySet* CurPropertySet, UInteractiveToolPropertySet* ReplaceWith, bool bSetToEnabled = true);

	/**
	 * Enable/Disable a PropertySet object for this Tool. If found and state was modified, will broadcast OnPropertySetsModified
	 * @param PropertySet Property Set object to modify
	 * @param bEnabled whether to enable or disable
	 * @return true if PropertySet was found
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SetToolPropertySourceEnabled(UInteractiveToolPropertySet* PropertySet, bool bEnabled);

	/**
	 * Call after changing a propertyset internally in the tool to allow external views of the property
	 * set to update properly. This is meant as an outward notification mechanism, not a way to to
	 * pass along notifications, so don't call this if the property is changed externally (i.e., this
	 * should not usually be called from OnPropertyModified unless the tool adds changes of its own).
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void NotifyOfPropertyChangeByTool(UInteractiveToolPropertySet* PropertySet) const;

	enum EAcceptWarning
	{
		NoWarning,
		EmptyForbidden
	};

	/**
	 * Helper function to update a standard warning when we need to explain why "Accept" is disabled
	 * Note that calling this base implementation will clear any unrelated warnings.
	 * Note that this function is not automatically called by the base class.
	 *
	 * @param Warning Reason that tool cannot be accepted (or EAcceptWarning::NoWarning if no warning need be displayed)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void UpdateAcceptWarnings(EAcceptWarning Warning);

private:
	// Tracks whether the UpdateAcceptWarnings function showed a warning the last time it was called.
	// Used to avoid clearing the tool display message in cases where it was not set by this function.
	bool bLastShowedAcceptWarning = false;



protected:
	/**
	 * Allow the Tool to do any necessary processing on Tick
	 * @param DeltaTime the time delta since last tick
	 */
	virtual void OnTick(float DeltaTime){};



	//
	// Action support/system
	//
	// Your Tool subclass can register a set of "Actions" it can execute
	// by overloading RegisterActions(). Then external systems can use GetActionSet() to
	// find out what Actions your Tool supports, and ExecuteAction() to run those actions.
	//
public:
	/**
	 * Get the internal Action Set for this Tool. The action set is created and registered on-demand.
	 * @return pointer to initialized Action set
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FInteractiveToolActionSet* GetActionSet();

	/**
	 * Request that the Action identified by ActionID be executed.
	 * Default implementation forwards these requests to internal ToolActionSet.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ExecuteAction(int32 ActionID);


protected:
	/**
	 * Override this function to register the set of Actions this Tool supports, using FInteractiveToolActionSet::RegisterAction.
	 * Note that for the actions to be triggered, you will also need to add corresponding registration per tool
	 *  -- see Engine\Plugins\Editor\ModelingToolsEditorMode\Source\ModelingToolsEditorMode\Public\ModelingToolsActions.h for examples
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet);


private:
	/**
	 * Set of actions this Tool can execute. This variable is allocated on-demand.
	 * Use GetActionSet() instead of accessing this pointer directly!
	 */
	FInteractiveToolActionSet* ToolActionSet = nullptr;



	//
	// Tool Information (name, icon, help text, etc)
	//


public:
	/**
	 * @return ToolInfo structure for this Tool
	 */
	virtual FInteractiveToolInfo GetToolInfo() const
	{
		return DefaultToolInfo;
	}

	/**
	 * Replace existing ToolInfo with new data
	 */
	virtual void SetToolInfo(const FInteractiveToolInfo& NewInfo)
	{
		DefaultToolInfo = NewInfo;
	}

	/**
	 * Set Tool name
	 */
	virtual void SetToolDisplayName(const FText& NewName)
	{
		DefaultToolInfo.ToolDisplayName = NewName;
	}

	/**
	 * Set the Tool's Display Message
	 */
	virtual void SetToolDisplayMessage(const FText& NewDisplayMessage)
	{
		DefaultToolInfo.ToolDisplayMessage = NewDisplayMessage;
	}

	/**
	 * Set the Tool's Icon
	 */
	virtual void SetToolIcon(const FSlateBrush* NewIcon)
	{
		DefaultToolInfo.ToolIcon = NewIcon;
	}

private:
	/**
	 * ToolInfo for this Tool
	 */
	FInteractiveToolInfo DefaultToolInfo;



private:

	// InteractionMechanic needs to be able to talk to Tool internals, eg property sets, behaviors, etc
	friend class UInteractionMechanic;
};


