// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Templates/Function.h"

class UClass;
class UInteractiveTool;


/**
 * Standard Actions that can be shared across multiple Tools.
 * The enum values are used as FInteractiveToolAction.ActionID.
 * If you need to define your own values, start at BaseClientDefinedActionID
 */
enum class EStandardToolActions
{
	IncreaseBrushSize = 100,
	DecreaseBrushSize = 101,
	ToggleWireframe = 102,


	BaseClientDefinedActionID = 10000
};



/**
 * Controls whether a tool Action re-fires (by key repeat) while its shortcut key is held down
 */
enum class EToolActionKeyRepeatMode
{
	Disabled,
	Enabled
};


/**
 * FInteractiveToolAction is returned by a UInteractiveTool to represent
 * an "Action" the Tool can execute. 
 */
struct FInteractiveToolAction
{
	/** Which type of UInteractiveTool this Action can be applied to*/
	const UClass* ClassType;

	/** Identifier for this Action */
	int32 ActionID;

	/** Internal name for this Action */
	FString ActionName;
	/** Short name for this Action */
	FText ShortName;
	/** Descriptive name for this Action */
	FText Description;

	/** Suggested modifier keys for this Action */
	EModifierKey::Type DefaultModifiers;
	/** Suggested keybinding for this Action. */
	FKey DefaultKey;

	/** Whether to re-fire (by key repeat) this Action while the shortcut key is held down */
	EToolActionKeyRepeatMode KeyRepeatMode = EToolActionKeyRepeatMode::Disabled;

	/** Call this function to execute the Action */
	TFunction<void()> OnAction;


	FInteractiveToolAction()
	{
		ClassType = nullptr;
		ActionID = 0;
	}

	FInteractiveToolAction(const UClass* ClassTypeIn, int32 ActionIDIn,
		const FString& ActionNameIn, const FText& ShortNameIn, const FText& DescriptionIn,
		EModifierKey::Type DefaultModifiersIn, const FKey& DefaultKeyIn )
	{
		ClassType = ClassTypeIn;
		ActionID = ActionIDIn;
		ActionName = ActionNameIn;
		ShortName = ShortNameIn;
		Description = DescriptionIn;
		DefaultModifiers = DefaultModifiersIn;
		DefaultKey = DefaultKeyIn;
	}

};



/**
 * FInteractiveToolActionSet maintains a list of FInteractiveToolAction.
 * Each UInteractiveTool contains an instance of this class.
 */
class FInteractiveToolActionSet
{
public:

	/**
	 * Register an Action with the ActionSet. This function is intended to be called by
	 * UInteractiveTool::RegisterActions() implementations
	 */
	INTERACTIVETOOLSFRAMEWORK_API void RegisterAction(UInteractiveTool* Tool, int32 ActionID,
		const FString& ActionName, const FText& ShortUIName, const FText& DescriptionText,
		EModifierKey::Type Modifiers, const FKey& ShortcutKey,
		TFunction<void()> ActionFunction,
		EToolActionKeyRepeatMode KeyRepeatMode = EToolActionKeyRepeatMode::Disabled );

	/**
	 * Find an existing Action by ID
	 * @return located Action, or nullptr if not found
	 */
	INTERACTIVETOOLSFRAMEWORK_API const FInteractiveToolAction* FindActionByID(int32 ActionID) const;

	/**
	 * Return the internal list of registered Actions by adding to the OutActions array
	 */
	INTERACTIVETOOLSFRAMEWORK_API void CollectActions(TArray<FInteractiveToolAction>& OutActions) const;

	/**
	 * Execute the action identified by ActionID
	 */
	INTERACTIVETOOLSFRAMEWORK_API void ExecuteAction(int32 ActionID) const;

protected:
	TArray<FInteractiveToolAction> Actions;
};
