// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "Misc/AssertionMacros.h"
#include "ToolTargets/ToolTarget.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "InteractiveToolBuilder.generated.h"

class UInteractiveTool;
struct FToolBuilderState;



/**
 * A UInteractiveToolBuilder creates a new instance of an InteractiveTool (basically this is a Factory).
 * These are registered with the InteractiveToolManager, which calls BuildTool() if CanBuildTool() returns true.
 * In addition CanBuildTool() will be queried to (for example) enable/disable UI buttons, etc.
 * This is an abstract base class, you must subclass it in order to create your particular Tool instance
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UInteractiveToolBuilder : public UObject
{
	GENERATED_BODY()

public:

	/** 
	 * Check if, given the current scene state, a new instance of this builder's Tool can be created
	 * @param SceneState the current scene selection state, etc
	 * @return true if a new Tool instance can be created
	 */
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const
	{
		check(false);
		return false;
	}

	/** 
	 * Create a new instance of this builder's Tool
	 * @param SceneState the current scene selection state, etc
	 * @return a new instance of the Tool, or nullptr on error/failure
	 */
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const
	{
		check(false);
		return nullptr;
	}


	/**
	 * PostBuildTool is called by the ToolManager after a new Tool instance has been built.
	 * This allows clients to extend a ToolBuilder with additional customization/etc, instead of having to entirely replace the existing BuildTool()
	 * @param Tool the new Tool instance
	 * @param SceneState the scene state used to create the new Tool
	 */
	virtual void PostBuildTool(UInteractiveTool* Tool, const FToolBuilderState& SceneState) const
	{
		return;
	}

	/**
	 * PostBuildTool is called by the ToolManager after a new Tool instance has been built and Setup() has been called.
	 * This allows clients to make modifications to Tools like changing initial parameters without subclassing the Tool
	 * @param Tool the new Tool instance
	 * @param SceneState the scene state used to create the new Tool
	 */
	virtual void PostSetupTool(UInteractiveTool* Tool, const FToolBuilderState& SceneState) const
	{
		return;
	}
};

/**
 * A UInteractiveToolWithToolTargetsBuilder creates a new instance of an InteractiveTool that uses tool targets.
 * See ToolTarget.h for more information on tool targets and their usage. This class defines the common
 * interface(s) for defining the tool target requirements of the tool it builds.
 * This is an abstract base class, you must subclass it in order to create your particular Tool instance.
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UInteractiveToolWithToolTargetsBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

protected:
	/**
	 * Gives the target requirements of the associated tool. Usually, it is the tool builder
	 * will use this function in CanBuildTool and BuildTool to find and create any necessary targets.
	 */
	virtual const FToolTargetTypeRequirements& GetTargetRequirements() const
	{
		static FToolTargetTypeRequirements TypeRequirements; // Default initialized to no requirements.
		return TypeRequirements;
	}
};
