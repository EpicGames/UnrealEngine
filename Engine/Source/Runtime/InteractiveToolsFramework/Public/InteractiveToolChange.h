// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Misc/Change.h"
#include "InteractiveToolChange.generated.h"

class UInteractiveToolManager;
class UInteractiveGizmoManager;


/**
 * FToolCommandChange is a base class for command changes used by the Tools Framework.
 */
class FToolCommandChange : public FCommandChange
{
public:

	virtual FString ToString() const override { return TEXT("ToolCommandChange"); }
};





UINTERFACE(MinimalAPI)
class UToolContextTransactionProvider : public UInterface
{
	GENERATED_BODY()
};
/**
 * IToolContextTransactionProvider is a UInterface that defines several functions that InteractiveTool code
 * uses to interface with the higher-level transaction system. UInteractiveToolManager and UInteractiveGizmoManager
 * both implement this interface.
 */
class IToolContextTransactionProvider
{
	GENERATED_BODY()
public:
	/**
	 * Request that the Context open a Transaction, whatever that means to the current Context
	 * @param Description text description of this transaction (this is the string that appears on undo/redo in the UE Editor)
	 */
	virtual void BeginUndoTransaction(const FText& Description) = 0;

	/** Request that the Context close and commit the open Transaction */
	virtual void EndUndoTransaction() = 0;

	/** Request that the Context cancel and revert the open Transaction */
	virtual void CancelUndoTransaction() = 0;

	/**
	 * Forward an FChange object to the Context
	 * @param TargetObject the object that the FChange applies to
	 * @param Change the change object that the Context should insert into the transaction history
	 * @param Description text description of this change (this is the string that appears on undo/redo in the UE Editor)
	 */
	virtual void EmitObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description) = 0;
};




/**
 * IToolCommandChangeSource is a generic interface for generating a change object.
 * This is useful to 
 */
class IToolCommandChangeSource
{
public:
	virtual ~IToolCommandChangeSource() = default;

	virtual void BeginChange() = 0;

	// Generally assuming that user of this class will call something like EmitObjectChange(GetChangeTarget(), MoveTemp(EndChange()), GetChangeDescription())

	virtual TUniquePtr<FToolCommandChange> EndChange() = 0;
	virtual UObject* GetChangeTarget() = 0;
	virtual FText GetChangeDescription() = 0;
	
	virtual void CancelChange() = 0;
};






/**
 * Holds another Change and forwards Apply/Revert to it, with
 * calls to Before/After lambas, allowing client classes to
 * respond to a change without having to intercept it explicitly.
 * (Be very careful with these lambdas!)
 */
template<typename ChangeType>
class TWrappedToolCommandChange : public FToolCommandChange
{
public:
	TUniquePtr<ChangeType> WrappedChange;

	TUniqueFunction<void(bool bRevert)> BeforeModify;
	TUniqueFunction<void(bool bRevert)> AfterModify;

	virtual void Apply(UObject* Object) override
	{
		if (BeforeModify)
		{
			BeforeModify(false);
		}
		WrappedChange->Apply(Object);
		if (AfterModify)
		{
			AfterModify(false);
		}
	}

	virtual void Revert(UObject* Object) override
	{
		if (BeforeModify)
		{
			BeforeModify(true);
		}
		WrappedChange->Revert(Object);
		if (AfterModify)
		{
			AfterModify(true);
		}
	}

	virtual FString ToString() const override
	{
		return WrappedChange->ToString();
	}
};

