// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InputRouter.h"
#include "InteractiveToolChange.h"
#include "ToolContextInterfaces.h"
#include "InteractiveToolManager.generated.h"

class UInteractiveToolStorableSelection;
class UContextObjectStore;

/** A Tool can be activated on a particular input device, currently identified by a "side" */
UENUM()
enum class EToolSide
{
	/** Left-hand Tool, also used for Mouse */
	Left = 1,
	Mouse = 1,
	/** Right-hand Tool*/
	Right = 2,
};


/**
 * UInteractiveToolManager can emit change events for the active tool in various ways.
 * This allows different modes to control how tools activate/deactivate on undo/redo, which is necessary
 * because some modes (eg Modeling Mode) do not support redo "into" a Tool, while others require it (like Paint Mode)
 */
UENUM()
enum class EToolChangeTrackingMode
{
	/** Do not emit any Active Tool change events */
	NoChangeTracking = 1,
	/** When Activating a new Tool, emit a change that will cancel/deactivate that Tool on Undo, but not reactivate it on Redo */
	UndoToExit = 2,
	/** Full change tracking of active Tool. Note that on Activation when an existing Tool is auto-shutdown, two separate FChanges are emitted, wrapped in a single Transaction */
	FullUndoRedo = 3
};

/**
 * Determines how a currently active tool is shutdown when switching to another tool.
 */
UENUM()
enum class EToolManagerToolSwitchMode
{
	/* Accept if able, cancel if not (and complete if neither is possible) */
	AcceptIfAble,
	/* Cancel if able, complete if not */
	CancelIfAble,
	/* Like AcceptIfAble, except a tool can change the shutdown type by implementing IInteractiveToolShutdownQueryAPI */
	CustomizableAcceptIfAble,
	/* Like CancelIfAble, except a tool can change the shutdown type by implementing IInteractiveToolShutdownQueryAPI */
	CustomizableCancelIfAble,
};


/**
 * UInteractiveToolManager allows users of the tools framework to create and operate Tool instances.
 * For each Tool, a (string,ToolBuilder) pair is registered with the ToolManager.
 * Tools can then be activated via the string identifier.
 * 
 * Currently a single Tool can be active for each input device. So for mouse input a single
 * Tool is available and effectively a lightweight mode. The mouse uses the "Left" tool slot.
 * 
 * For VR controllers and touch input, a "Left" and "Right" tool can be active at the same time.  
 * @todo this is not fully supported yet
 * 
 * Tools are not directly created. Use SelectActiveToolType(side,string) to set the active ToolBuilder
 * on a given side, and then use ActivateTool() to create the new Tool instance.
 *
 */
UCLASS(Transient, MinimalAPI)
class UInteractiveToolManager : public UObject, public IToolContextTransactionProvider
{
	GENERATED_BODY()

protected:
	friend class UInteractiveToolsContext;		// to call Initialize/Shutdown

	INTERACTIVETOOLSFRAMEWORK_API UInteractiveToolManager();

	/** Initialize the ToolManager with the necessary Context-level state. UInteractiveToolsContext calls this, you should not. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IToolsContextQueriesAPI* QueriesAPI, IToolsContextTransactionsAPI* TransactionsAPI, UInputRouter* InputRouter);

	/** Shutdown the ToolManager. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

	/** Called immediately after a tool is built. Broadcasts OnToolPostBuild. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DoPostBuild(EToolSide Side, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& InBuilderState);
	
	/** Called immediately after a tool's Setup is called. Broadcasts OnToolPostSetup. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DoPostSetup(EToolSide Side, UInteractiveTool* InInteractiveTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& InBuilderState);

public:


	/**
	 * @return true if ToolManager is currently active, ie between Initialize() and Shutdown() 
	 */
	bool IsActive() const { return bIsActive; }


	//
	// Tool registration and Current Tool state
	//

	/**
	 * Register a new ToolBuilder
	 * @param Identifier string used to identify this Builder
	 * @param Builder new ToolBuilder instance
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RegisterToolType(const FString& Identifier, UInteractiveToolBuilder* Builder);

	/**
	 * Unregisters a ToolBuilder
	 * @param Identifier string used to identify this Builder
	 * @param Builder new ToolBuilder instance
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void UnregisterToolType(const FString& Identifier);

	/**
	 * Set active ToolBuilder for a ToolSide via string identifier
	 * @param Side which "side" should we set this Builder on
	 * @param Identifier name of ToolBuilder that was passed to RegisterToolType()
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SelectActiveToolType(EToolSide Side, const FString& Identifier);

	/**
	 * Get active ToolBuilder string identifier for a ToolSide
	 * @param Side which "side" should we get this Builder from 
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual const FString& GetActiveToolType(EToolSide Side) const;

	/**
	 * Check if a named Tool type can currently be activated on the given ToolSide
	 * @param Side which "side" you would like to active the tool on
	 * @param Identifier string name of the Tool type
	 * @return true if the Tool type could be activated
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanActivateTool(EToolSide eSide, FString Identifier);

	/**
	 * Try to activate a new Tool instance on the given Side. If another tool is already active, it
	 *  will be shut down according to ToolSwitchMode.
	 * @param Side which "side" you would like to active the tool on
	 * @return true if a new Tool instance was created and initialized
	 */	
	INTERACTIVETOOLSFRAMEWORK_API virtual bool ActivateTool(EToolSide Side);

	/**
	 * Check if there is an active Tool on the given Side
	 * @param Side which Side to check
	 * @return true if there is an active Tool on that side
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasActiveTool(EToolSide Side) const;

	/**
	 * Get the current setting for how to deactivate an already-active tool if switching directly to
	 * another tool.
	 */
	virtual EToolManagerToolSwitchMode GetToolSwitchMode() const { return ToolSwitchMode; }

	/**
	 * Set the behavior for how to deactivate an already-active tool if ActivateTool is called while
	 * HasActiveTool is true.
	 * 
	 * Note: a higher level system could choose to enforce its preferences for this behavior by querying 
	 * HasActiveTool and making sure to shut down the previous tool before calling ActivateTool.
	 * However it is convenient to be able to set the behavior at the tool manager level.
	 */
	virtual void SetToolSwitchMode(EToolManagerToolSwitchMode ToolSwitchModeIn) { ToolSwitchMode = ToolSwitchModeIn; }

	/**
	 * @return true if there are any active tools
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasAnyActiveTool() const;


	/**
	 * Get pointer to active Tool on a given side
	 * @param Side which Side is being requested
	 * @return pointer to Tool instance active on that Side, or nullptr if no such Tool exists
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveTool* GetActiveTool(EToolSide Side);

	/**
	 * Get pointer to active Tool Builder on a given side
	 * @param Side which Side is being requested
	 * @return pointer to Tool Builder instance active on that Side, or nullptr if no such ToolBuilder exists
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UInteractiveToolBuilder* GetActiveToolBuilder(EToolSide Side);

	/**
	 * Get name of registered ToolBuilder that created active tool for given side, or empty string if no tool is active
	 * @param Side which Side is being requested
	 * @return name of tool, or empty string if no tool is active
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual FString GetActiveToolName(EToolSide Side);

	/**
	 * Check if an active Tool on the given Side can be Accepted in its current state
	 * @param Side which Side to check
	 * @return true if there is an active Tool and it returns true from HasAccept() and CanAccept()
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanAcceptActiveTool(EToolSide Side);

	/**
	 * Check if an active Tool on the given Side can be Canceled
	 * @param Side which Side to check
	 * @return true if there is an active Tool and it returns true from HasCancel()
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanCancelActiveTool(EToolSide Side);

	/**
	 * Shut down an active Tool on the given side
	 * @param Side which "side" you would like to shut down
	 * @param ShutdownType how should the tool be terminated (eg Accept/Cancel)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DeactivateTool(EToolSide Side, EToolShutdownType ShutdownType);

	/**
	 * Determines the preferred shutdown type for the active tool based on the tool's type and the tool manager's configuration.
	 */
	INTERACTIVETOOLSFRAMEWORK_API EToolShutdownType GetPreferredShutdownType(EToolSide Side,
		EToolShutdownReason ShutdownReason = EToolShutdownReason::Unspecified);

	/**
	 * Configure how tool changes emit change events. See EToolChangeTrackingMode for details.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void ConfigureChangeTrackingMode(EToolChangeTrackingMode ChangeMode);

	/**
	 * Returns how tool changes emit change events. See EToolChangeTrackingMode for details.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual EToolChangeTrackingMode GetChangeTrackingMode() const;

	//
	// Functions that Tools can call to interact with Transactions API
	//
	
	/** Post a message via the Transactions API */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DisplayMessage(const FText& Message, EToolMessageLevel Level);

	/** Request an Invalidation via the Transactions API (ie to cause a repaint, etc) */
	INTERACTIVETOOLSFRAMEWORK_API virtual void PostInvalidation();

	/**
	 * Request that the Context open a Transaction, whatever that means to the current Context
	 * @param Description text description of this transaction (this is the string that appears on undo/redo in the UE Editor)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void BeginUndoTransaction(const FText& Description);

	/** Request that the Context close and commit the open Transaction */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EndUndoTransaction();

	/** Request that the Context cancel and revert the open Transaction */
	INTERACTIVETOOLSFRAMEWORK_API virtual void CancelUndoTransaction() override;

	/**
	 * Forward an FChange object to the Context
	 * @param TargetObject the object that the FChange applies to
	 * @param Change the change object that the Context should insert into the transaction history
	 * @param Description text description of this change (this is the string that appears on undo/redo in the UE Editor)
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EmitObjectChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description );


	/**
	 * Forward an FChange object to the Context
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool RequestSelectionChange(const FSelectedObjectsChangeList& SelectionChange);

	//
	// State control  (@todo: have the Context call these? not safe for anyone to call)
	//

	/** Tick any active Tools. Called by UInteractiveToolsContext */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Tick(float DeltaTime);

	/** Render any active Tools. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Render(IToolsContextRenderAPI* RenderAPI);

	/** Let active Tools do their screen space drawing. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	/**
	 * A Tool (or other code) can call this function to request that the Tool be deactivated.
	 * The Tool must be active. By default, ::DeactivateTool() wil be called. However if some
	 * external code has bound to the OnToolShutdownRequest delegate, the request will be
	 * forwarded to that function first (::DeactivateTool() will still be called if it returns false)
	 * 
	 * If this function is called during InteractiveTool::Setup(), the Tool invocation will be aborted,
	 * ie the Tool will be immediately shutdown with EToolShutdownType::Cancel, and never become "Active"/etc. 
	 * Note also that in this case the shutdown request *will not* be forwarded to the OnToolShutdownRequest delegate.
	 * 
	 * @param bShowUnexpectedShutdownMessage if this was an unexpected shutdown, passing true here, along with a non-empty UnexpectedShutdownMessage, will result in OnToolUnexpectedShutdownMessage being called
	 * @param UnexpectedShutdownMessage message provided by the caller meant to explain why the Tool has unexpectedly been shut down
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool PostActiveToolShutdownRequest(
		UInteractiveTool* Tool, 
		EToolShutdownType ShutdownType,
		bool bShowUnexpectedShutdownMessage = false,
		const FText& UnexpectedShutdownMessage = FText() );

	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnToolShutdownRequest, UInteractiveToolManager*, UInteractiveTool*, EToolShutdownType);
	/**
	 * If bound, OnToolShutdownRequest is called by PostActiveToolShutdownRequest to optionally handle
	 * requests to shut down a Tool (which may be sent by the Tool itself). Return true to indicate
	 * that the request will be handled, otherwise ::DeactivateTool() will be called
	 */
	FOnToolShutdownRequest OnToolShutdownRequest;
	
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnToolUnexpectedShutdownMessage, UInteractiveToolManager*, UInteractiveTool*, const FText& Message, bool bWasDuringToolSetup);

	/**
	 * PostActiveToolShutdownRequest() will broadcast this delegate with a Message if bShowUnexpectedShutdownMessage=true.
	 * If the shutdown was during Tool Setup/startup, bWasDuringToolSetup will be passed as true
	 */
	FOnToolUnexpectedShutdownMessage OnToolUnexpectedShutdownMessage;
	

	//
	// access to APIs, etc
	//
	
	/** @return current IToolsContextQueriesAPI */
	virtual IToolsContextQueriesAPI* GetContextQueriesAPI() { return QueriesAPI; }

	/** @return current IToolsContextTransactionsAPI */
	virtual IToolsContextTransactionsAPI* GetContextTransactionsAPI() final { return TransactionsAPI; }

	INTERACTIVETOOLSFRAMEWORK_API UInteractiveGizmoManager* GetPairedGizmoManager();

	/**
	 * @return the context object store from the owning tools context.
	 */
	INTERACTIVETOOLSFRAMEWORK_API UContextObjectStore* GetContextObjectStore() const;

public:
	/** Currently-active Left Tool, or null if no Tool is active */
	UPROPERTY()
	TObjectPtr<UInteractiveTool> ActiveLeftTool;

	/** Currently-active Right Tool, or null if no Tool is active */
	UPROPERTY()
	TObjectPtr<UInteractiveTool> ActiveRightTool;




public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FToolManagerToolStartedSignature, UInteractiveToolManager*, UInteractiveTool*);
	FToolManagerToolStartedSignature OnToolStarted;

	DECLARE_MULTICAST_DELEGATE_FiveParams(FToolManagerToolPostBuildSignature, UInteractiveToolManager*, EToolSide, UInteractiveTool*, UInteractiveToolBuilder*, const FToolBuilderState&);
	FToolManagerToolPostBuildSignature OnToolPostBuild;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FToolManagerToolPostSetupSignature, UInteractiveToolManager*, EToolSide, UInteractiveTool*);
	FToolManagerToolPostSetupSignature OnToolPostSetup;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FToolManagerToolEndedSignature, UInteractiveToolManager*, UInteractiveTool*);
	FToolManagerToolEndedSignature OnToolEnded;

	// Variant of OnToolEnded that also reports the EToolShutdownType
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FToolManagerToolCancelledSignature, UInteractiveToolManager*, UInteractiveTool*, EToolShutdownType);
	FToolManagerToolCancelledSignature OnToolEndedWithStatus;


protected:
	/** Pointer to current Context-Queries implementation */
	IToolsContextQueriesAPI* QueriesAPI;
	/** Pointer to current Transactions implementation */
	IToolsContextTransactionsAPI* TransactionsAPI;

	/** Pointer to current InputRouter (Context owns this) */
	UInputRouter* InputRouter;

	/** This flag is set to true on Initialize() and false on Shutdown(). */
	bool bIsActive = false;

	// This bool is for handling the case where a Tool posts a shutdown request during it's Setup() call.
	// In that case we will terminate the Tool immediately
	bool bInToolSetup = false;
	bool bToolRequestedTerminationDuringSetup = false;


	/** Current set of named ToolBuilders */
	UPROPERTY()
	TMap<FString, TObjectPtr<UInteractiveToolBuilder>> ToolBuilders;

	/** Currently-active Left ToolBuilder */
	FString ActiveLeftBuilderName;
	UInteractiveToolBuilder* ActiveLeftBuilder;
	/** Currently-active Right ToolBuilder */
	FString ActiveRightBuilderName;
	UInteractiveToolBuilder* ActiveRightBuilder;

	EToolChangeTrackingMode ActiveToolChangeTrackingMode;
	EToolManagerToolSwitchMode ToolSwitchMode = EToolManagerToolSwitchMode::CustomizableAcceptIfAble;

	FString ActiveLeftToolName;
	FString ActiveRightToolName;
	bool bInToolShutdown = false;

	UE_DEPRECATED(5.5, "bActiveToolMadeSelectionStoreRequest is no longer used.")
	bool bActiveToolMadeSelectionStoreRequest = false;

	INTERACTIVETOOLSFRAMEWORK_API virtual bool ActivateToolInternal(EToolSide Side);
	INTERACTIVETOOLSFRAMEWORK_API virtual void DeactivateToolInternal(EToolSide Side, EToolShutdownType ShutdownType);

	friend class FBeginToolChange;
	friend class FActivateToolChange;
};




/**
 * FBeginToolChange is used by UInteractiveToolManager to back out of a Tool on Undo.
 * No action is taken on Redo, ie we do not re-start the Tool on Redo.
 */
class FBeginToolChange : public FToolCommandChange
{
public:
	INTERACTIVETOOLSFRAMEWORK_API virtual void Apply(UObject* Object) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void Revert(UObject* Object) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasExpired(UObject* Object) const override;

	INTERACTIVETOOLSFRAMEWORK_API virtual FString ToString() const override;
};

/**
 * FActivateToolChange is used by UInteractiveToolManager to change the active tool.
 * This Change has two modes, either activating or deactivating.
 */
class FActivateToolChange : public FToolCommandChange
{
public:
	EToolSide Side;
	FString ToolType;
	bool bIsDeactivate = false;
	EToolShutdownType ShutdownType;

	FActivateToolChange(EToolSide SideIn, FString ToolTypeIn)
		: Side(SideIn), ToolType(ToolTypeIn), bIsDeactivate(false) {}
	FActivateToolChange(EToolSide SideIn, FString ToolTypeIn, EToolShutdownType ShutdownTypeIn)
		: Side(SideIn), ToolType(ToolTypeIn), bIsDeactivate(true), ShutdownType(ShutdownTypeIn) {}

	INTERACTIVETOOLSFRAMEWORK_API virtual void Apply(UObject* Object) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void Revert(UObject* Object) override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasExpired(UObject* Object) const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FString ToString() const override;
};





/**
 * FToolChangeWrapperChange wraps an FChange emitted by an InteractiveTool, allowing
 * us to Expire the change without each FChange implementation needing to handle this explicitly.
 */
class FToolChangeWrapperChange : public FToolCommandChange
{
public:
	TWeakObjectPtr<UInteractiveToolManager> ToolManager;
	TWeakObjectPtr<UInteractiveTool> ActiveTool;
	TUniquePtr<FToolCommandChange> ToolChange;

	INTERACTIVETOOLSFRAMEWORK_API virtual void Apply(UObject* Object) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual void Revert(UObject* Object) override;

	INTERACTIVETOOLSFRAMEWORK_API virtual bool HasExpired(UObject* Object) const override;

	INTERACTIVETOOLSFRAMEWORK_API virtual FString ToString() const override;
};
