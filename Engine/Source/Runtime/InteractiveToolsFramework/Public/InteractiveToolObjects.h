// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "GameFramework/Actor.h"
#include "InteractiveToolObjects.generated.h"


/**
 * AInternalToolFrameworkActor is a base class for internal Actors that the
 * ToolsFramework may spawn (for gizmos, mesh previews, etc). These Actors
 * may need special-case handling, for example to prevent the user from
 * selecting and deleting them. 
 */
UCLASS(Transient, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType, MinimalAPI)
class AInternalToolFrameworkActor : public AActor
{
	GENERATED_BODY()
private:

public:

	/** Controls whether this InternalToolFrameworkActor can be selected in the Editor. */
	UPROPERTY()
	bool bIsSelectableInEditor = false;

#if WITH_EDITOR
	virtual bool IsSelectable() const override
	{
		return bIsSelectableInEditor;
	}
#endif
};





// UInterface for IToolFrameworkComponent
UINTERFACE(MinimalAPI)
class UToolFrameworkComponent : public UInterface
{
	GENERATED_BODY()
};

// IToolFrameworkComponent is an interface we can attach to custom Components used in the Tools framework.
// Currently this interface is only used to identify such Components (to prevent certain Actors from being deleted)
class IToolFrameworkComponent
{
	GENERATED_BODY()

public:
};

