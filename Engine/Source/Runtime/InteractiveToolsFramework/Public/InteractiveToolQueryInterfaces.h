// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h" // EToolShutdownType, EToolShutdownReason
#include "UObject/Interface.h"
#include "InteractiveToolQueryInterfaces.generated.h"


//
// Below are various interfaces that a UInteractiveTool can implement to allow
// higher-level code (eg like an EdMode) to query into the Tool.
//



// UInterface for IInteractiveToolCameraFocusAPI
UINTERFACE(MinimalAPI)
class UInteractiveToolCameraFocusAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * IInteractiveToolCameraFocusAPI provides two functions that can be
 * used to extract "Focus" / "Region of Interest" information about an
 * active Tool:
 * 
 * GetWorldSpaceFocusBox() - provides a bounding box for an "active region" if one is known.
 *   An example of using the FocusBox would be to center/zoom the camera in a 3D viewport
 *   onto this box when the user hits a hotkey (eg 'f' in the Editor).
 *   Should default to the entire active object, if no subregion is available.
 * 
 * GetWorldSpaceFocusPoint() - provides a "Focus Point" at the cursor ray if one is known.
 *   This can be used to (eg) center the camera at the focus point.
 * 
 * The above functions should not be called unless the corresponding SupportsX() function returns true.
 */
class IInteractiveToolCameraFocusAPI
{
	GENERATED_BODY()
public:

	/**
	 * @return true if the implementation can provide a Focus Box
	 */
	virtual bool SupportsWorldSpaceFocusBox() { return false; }
	
	/**
	 * @return the current Focus Box
	 */
	virtual FBox GetWorldSpaceFocusBox() { return FBox(); }

	/**
	 * @return true if the implementation can provide a Focus Point
	 */
	virtual bool SupportsWorldSpaceFocusPoint() { return false; }

	/**
	 * @param WorldRay 3D Ray that should be used to find the focus point, generally ray under cursor
	 * @param PointOut computed Focus Point
	 * @return true if a Focus Point was found, can return false if (eg) the ray missed the target objects
	 */
	virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) { return false; }


};







// UInterface for IInteractiveToolNestedAcceptCancelAPI
UINTERFACE(MinimalAPI)
class UInteractiveToolNestedAcceptCancelAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * IInteractiveToolNestedAcceptCancelAPI provides an API for a Tool to publish
 * intent and ability to Accept or Cancel sub-operations. For example in a Tool
 * that has an editable active Selection, we might want the Escape hotkey to
 * Clear any active selection, and then on a second press, to Cancel the Tool. 
 * This API allows a Tool to say "I can consume a Cancel action", and similarly
 * for Accept (although this is much less common).
 */
class IInteractiveToolNestedAcceptCancelAPI
{
	GENERATED_BODY()
public:

	/**
	 * @return true if the implementor of this API may be able to consume a Cancel action
	 */
	virtual bool SupportsNestedCancelCommand() { return false; }

	/**
	 * @return true if the implementor of this API can currently consume a Cancel action
	 */
	virtual bool CanCurrentlyNestedCancel() { return false; }

	/**
	 * Called by Editor levels to tell the implementor (eg Tool) to execute a nested Cancel action
	 * @return true if the implementor consumed the Cancel action
	 */
	virtual bool ExecuteNestedCancelCommand() { return false; }



	/**
	 * @return true if the implementor of this API may be able to consume an Accept action
	 */
	virtual bool SupportsNestedAcceptCommand() { return false; }

	/**
	 * @return true if the implementor of this API can currently consume an Accept action
	 */
	virtual bool CanCurrentlyNestedAccept() { return false; }

	/**
	 * Called by Editor levels to tell the implementor (eg Tool) to execute a nested Accept action
	 * @return true if the implementor consumed the Accept action
	 */
	virtual bool ExecuteNestedAcceptCommand() { return false; }


};

// UInterface for IInteractiveToolShutdownQueryAPI
UINTERFACE(MinimalAPI)
class UInteractiveToolShutdownQueryAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * Allows an interactive tool to influence the way it is shut down, if the tool host supports
 * that kind of customization. This can be helpful, for example, if your tool prefers a specific
 * shutdown type in various situations, but a tool can't rely on this interface being queried unless
 * it knows that it will only be used by systems that respect it. A simple interactive tools framework
 * context implementation does not need to bother querying the tool on its preferences (aside from the
 * already existing CanAccept method on the actual tool object).
 * 
 * Note that there are different systems that might choose to query this interface. The tool manager
 * might want to query it if its ToolSwitchMode is set to be customizable, or the mode or mode toolkit
 * might query it when shutting down the tool in various situations.
 */
class IInteractiveToolShutdownQueryAPI
{
	GENERATED_BODY()
public:

	enum class UE_DEPRECATED(5.7, "EShutdownReason has moved to InteractiveTool.h (under the name EToolShutdownReason).") EShutdownReason
	{
		// Another tool is being activated without the user having explicitly shut down this one.
		SwitchTool
	};

	/**
	 * Given a shutdown situation, tells what kind of shutdown method the tool might prefer.
	 * @param ShutdownReason Information about the shutdown type
	 * @param StandardShutdownType Shutdown type that the host would use if it weren't giving the tool a
	 *   chance to customize the behavior.
	 * @return Shutdown type that the tool would prefer that the host use in this situation.
	 */
	UE_DEPRECATED(5.7, "Prefer the GetPreferredShutdownType variant which uses EToolShutdownReason.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual EToolShutdownType GetPreferredShutdownType(EShutdownReason ShutdownReason, EToolShutdownType StandardShutdownType) const
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return StandardShutdownType;
	}
	
	/**
	 * Given a shutdown situation, tells what kind of shutdown method the tool might prefer.
	 * @param ShutdownReason Information about the shutdown type
	 * @param StandardShutdownType Shutdown type that the host would use if it weren't giving the tool a
	 *   chance to customize the behavior.
	 * @return Shutdown type that the tool would prefer that the host use in this situation.
	 */
	virtual EToolShutdownType GetPreferredShutdownType(EToolShutdownReason ShutdownReason, EToolShutdownType StandardShutdownType) const
	{
		if (ShutdownReason == EToolShutdownReason::SwitchTool)
		{
			// todo: remove this special case when removing the deprecated variant of GetPreferredShutdownType.
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return GetPreferredShutdownType(EShutdownReason::SwitchTool, StandardShutdownType);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	
		return StandardShutdownType;
	}

	//~ Not currently supported, but someday we might let tools query the user with an "are you sure"
	//~ type of message, and allow the tool to ask to NOT be shut down after all. This is one way
	//~ we might implement the ability for tools to ask for this:
	//~ struct FShutdownUserQuery
	//~ {
	//~ 	FString MessageToUser;
	//~ 	EAppMsgType::Type QueryType;
	//~ 	TFunction<void(EAppReturnType::Type UserResponse, bool& bStillShutdownOut, EToolShutdownType& ShutdownTypeOut)> ResponseHandler;
	//~ };
	//~ TOptional<FShutdownUserQuery> GetShutdownUserQuery() { return TOptional<FShutdownUserQuery>(); }
};


// UInterface for IInteractiveToolExclusiveToolAPI
UINTERFACE(MinimalAPI)
class UInteractiveToolExclusiveToolAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * IInteractiveToolExclusiveToolAPI provides an API to inform the
 * ToolManager about tool exclusivity. An exclusive tool prevents other
 * tools from building & activating while the tool is active. This is
 * useful in scenarios where tools want to enforce an explicit Accept,
 * Cancel or Complete user input to exit the tool.
 */
class IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()
};



// UInterface for IInteractiveToolEditorGizmoAPI
UINTERFACE(MinimalAPI)
class UInteractiveToolEditorGizmoAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * IInteractiveToolEditorGizmoAPI provides an API to indicate whether
 * the standard editor gizmos can be enabled while this tool is active.
 */
class IInteractiveToolEditorGizmoAPI
{
	GENERATED_BODY()
public:

	/**
	 * @return true if the tool implementing this API allows the editor gizmos to be enabled while the tool is active
	 */
	virtual bool GetAllowStandardEditorGizmos() { return false; }
};



// UInterface for IInteractiveToolManageGeometrySelectionAPI
UINTERFACE(MinimalAPI)
class UInteractiveToolManageGeometrySelectionAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * Provides an API to allow a tool to report how it has affected (updated or invalidated) geometry selections on the tool's targets
 */
class IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:

	/**
	 * @return true if the tool implementing this API has not updated the geometry selection or modified geometry to invalidate any previous geometry selection, i.e. has not removed/added vertices/edges/triangles
	 */
	virtual bool IsInputSelectionValidOnOutput() { return false; }
};
