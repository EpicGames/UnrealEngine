// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "Templates/Function.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InteractiveToolsContext.generated.h"

class UToolGroupManager;
class FText;
class IToolsContextQueriesAPI;
class IToolsContextTransactionsAPI;
class UContextObjectStore;
class UInputRouter;
class UInteractiveGizmoManager;
class UToolTargetManager;


/**
 * InteractiveToolsContext owns the core parts of an Interactive Tools Framework implementation - the
 * InputRouter, ToolManager, GizmoManager, TargetManager, ToolGroupManager, and ContextStore. In the simplest
 * use case, UInteractiveToolsContext is just a top-level container that will keep the various UObjects
 * alive, and provide an easy way to access them. However in a more complex situation it may be
 * desirable to subclass and extend the ToolsContext. For example, UEdModeInteractiveToolsContext allows
 * a ToolsContext to live within a UEdMode.
 */
UCLASS(Transient, MinimalAPI)
class UInteractiveToolsContext : public UObject
{
	GENERATED_BODY()
	
public:
	INTERACTIVETOOLSFRAMEWORK_API UInteractiveToolsContext();

	// FContextInitInfo is used by Initialize() to pass information to the various creator functions below.
	// See ::UInteractiveToolsContext() and ::Initialize() for how these fields are used/initialized.
	// In particular, InputRouter will not be set until after the UInputRouter has been created
	struct FContextInitInfo
	{
		UInteractiveToolsContext* ToolsContext = nullptr;
		IToolsContextQueriesAPI* QueriesAPI = nullptr;
		IToolsContextTransactionsAPI* TransactionsAPI = nullptr;
		UInputRouter* InputRouter = nullptr;
	};

	//
	// Replace the internal functions that are called to create/destroy the sub-objects owned by the ITC.
	// By doing this, clients of the ITF can provide their own subclass implementations, or do other
	// custom setup/teardown as necessary. See comments above CreateInputRouterFunc below
	//
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCreateInputRouterFunc(TUniqueFunction<UInputRouter* (const FContextInitInfo&)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCreateToolManagerFunc(TUniqueFunction<UInteractiveToolManager* (const FContextInitInfo&)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCreateToolTargetManagerFunc(TUniqueFunction<UToolTargetManager* (const FContextInitInfo&)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCreateGizmoManagerFunc(TUniqueFunction<UInteractiveGizmoManager* (const FContextInitInfo&)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCreateContextStoreFunc(TUniqueFunction<UContextObjectStore* (const FContextInitInfo&)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetCreateToolGroupManagerFunc(TUniqueFunction<UToolGroupManager* (const FContextInitInfo&)> Func);

	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShutdownInputRouterFunc(TUniqueFunction<void(UInputRouter*)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShutdownToolManagerFunc(TUniqueFunction<void(UInteractiveToolManager*)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShutdownToolTargetManagerFunc(TUniqueFunction<void(UToolTargetManager*)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShutdownGizmoManagerFunc(TUniqueFunction<void(UInteractiveGizmoManager*)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShutdownContextStoreFunc(TUniqueFunction<void(UContextObjectStore*)> Func);
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetShutdownToolGroupManagerFunc(TUniqueFunction<void(UToolGroupManager*)> Func);

	/** 
	 * Initialize the Context. This creates the InputRouter, ToolManager, GizmoManager, TargetManager, and ContextStore
	 * @param QueriesAPI client-provided implementation of the API for querying the higher-evel scene state
	 * @param TransactionsAPI client-provided implementation of the API for publishing events and transactions
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize(IToolsContextQueriesAPI* QueriesAPI, IToolsContextTransactionsAPI* TransactionsAPI);

	/** Shutdown Context by destroying InputRouter and ToolManager */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

	INTERACTIVETOOLSFRAMEWORK_API virtual void DeactivateActiveTool(EToolSide WhichSide, EToolShutdownType ShutdownType);
	INTERACTIVETOOLSFRAMEWORK_API virtual void DeactivateAllActiveTools(EToolShutdownType ShutdownType);

	INTERACTIVETOOLSFRAMEWORK_API bool CanStartTool(EToolSide WhichSide, const FString& ToolTypeIdentifier) const;
	INTERACTIVETOOLSFRAMEWORK_API bool HasActiveTool(EToolSide WhichSide) const;
	INTERACTIVETOOLSFRAMEWORK_API FString GetActiveToolName(EToolSide WhichSide) const;
	INTERACTIVETOOLSFRAMEWORK_API bool ActiveToolHasAccept(EToolSide WhichSide) const;
	INTERACTIVETOOLSFRAMEWORK_API bool CanAcceptActiveTool(EToolSide WhichSide) const;
	INTERACTIVETOOLSFRAMEWORK_API bool CanCancelActiveTool(EToolSide WhichSide) const;
	INTERACTIVETOOLSFRAMEWORK_API bool CanCompleteActiveTool(EToolSide WhichSide) const;
	INTERACTIVETOOLSFRAMEWORK_API bool StartTool(EToolSide WhichSide, const FString& ToolTypeIdentifier);
	INTERACTIVETOOLSFRAMEWORK_API void EndTool(EToolSide WhichSide, EToolShutdownType ShutdownType);
	INTERACTIVETOOLSFRAMEWORK_API bool IsToolActive(EToolSide WhichSide, const FString ToolIdentifier) const;

public:
	// forwards message to OnToolNotificationMessage delegate
	INTERACTIVETOOLSFRAMEWORK_API virtual void PostToolNotificationMessage(const FText& Message);
	INTERACTIVETOOLSFRAMEWORK_API virtual void PostToolWarningMessage(const FText& Message);

	DECLARE_MULTICAST_DELEGATE_OneParam(FToolsContextToolNotification, const FText&);
	FToolsContextToolNotification OnToolNotificationMessage;
	FToolsContextToolNotification OnToolWarningMessage;

public:
	/** current UInputRouter for this Context */
	UPROPERTY()
	TObjectPtr<UInputRouter> InputRouter;	

	/** current UToolTargetManager for this Context */
	UPROPERTY()
	TObjectPtr<UToolTargetManager> TargetManager;

	/** current UInteractiveToolManager for this Context */
	UPROPERTY()
	TObjectPtr<UInteractiveToolManager> ToolManager;	

	/** current UInteractiveGizmoManager for this Context */
	UPROPERTY()
	TObjectPtr<UInteractiveGizmoManager> GizmoManager;

	/** current UToolGroupManager for this Context */
	UPROPERTY()
	TObjectPtr<UToolGroupManager> ToolGroupManager;
	
	/** 
	 * Current Context Object Store for this Context.
	 * Stores arbitrary objects which share data or expose APIs across interactive tools and managers belonging to this context.
	 */
	UPROPERTY()
	TObjectPtr<UContextObjectStore> ContextObjectStore;

protected:

	// Initialize() calls these functions to create the main child objects needed to operate
	// the Tools Framework - InputRouter, ToolManager, GizmoManager, TargetManager, ContextStore.
	// Default implementations are set up in the UInteractiveToolsContext() constructor, however
	// users of the framework are free to replace these with subclasses, or do more complex
	// initialization, by replacing these functions with their own versions before calling Initialize().
	// (the same could be accomplished by subclassing and overriding Initialize(), however using these
	//  lambda functions will be a safer alternative in many cases)
	TUniqueFunction<UInputRouter* (const FContextInitInfo&)> CreateInputRouterFunc;
	TUniqueFunction<UInteractiveToolManager* (const FContextInitInfo&)> CreateToolManagerFunc;
	TUniqueFunction<UToolTargetManager* (const FContextInitInfo&)> CreateToolTargetManagerFunc;
	TUniqueFunction<UInteractiveGizmoManager* (const FContextInitInfo&)> CreateGizmoManagerFunc;
	TUniqueFunction<UContextObjectStore* (const FContextInitInfo&)> CreateContextStoreFunc;
	TUniqueFunction<UToolGroupManager* (const FContextInitInfo&)> CreateToolGroupManagerFunc;

	// Analogous to the CreateX() functions above, these function are called by Shutdown()
	// to terminate and clean up after the various elements. 
	TUniqueFunction<void(UInputRouter*)> ShutdownInputRouterFunc;
	TUniqueFunction<void(UInteractiveToolManager*)> ShutdownToolManagerFunc;
	TUniqueFunction<void(UToolTargetManager*)> ShutdownToolTargetManagerFunc;
	TUniqueFunction<void(UInteractiveGizmoManager*)> ShutdownGizmoManagerFunc;
	TUniqueFunction<void(UContextObjectStore*)> ShutdownContextStoreFunc;
	TUniqueFunction<void(UToolGroupManager*)> ShutdownToolGroupManagerFunc;


	// todo: deprecate and remove this, can now be accomplished via CreateToolManagerFunc()
	UPROPERTY()
	TSoftClassPtr<UInteractiveToolManager> ToolManagerClass;
};
