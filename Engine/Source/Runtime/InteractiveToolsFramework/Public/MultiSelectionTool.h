// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveTool.h"
#include "ComponentSourceInterfaces.h"
#include "ToolTargets/ToolTarget.h"
#include "InteractiveToolQueryInterfaces.h"

#include "MultiSelectionTool.generated.h"

UCLASS(Transient, MinimalAPI)
class UMultiSelectionTool : public UInteractiveTool, public IInteractiveToolCameraFocusAPI
{
GENERATED_BODY()
public:
	void SetTargets(TArray<TObjectPtr<UToolTarget>> TargetsIn)
	{
		Targets = MoveTemp(TargetsIn);
	}

	/**
	 * @return true if all ComponentTargets of this tool are still valid
	 */
	virtual bool AreAllTargetsValid() const
	{
		for (const TObjectPtr<UToolTarget>& Target : Targets)
		{
			if (Target->IsValid() == false)
			{
				return false;
			}
		}
		return true;
	}


public:
	virtual bool CanAccept() const override
	{
		return AreAllTargetsValid();
	}

protected:
	UPROPERTY()
	TArray<TObjectPtr<UToolTarget>> Targets{};

public:
	// IInteractiveToolCameraFocusAPI implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SupportsWorldSpaceFocusBox() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FBox GetWorldSpaceFocusBox() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SupportsWorldSpaceFocusPoint() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) override;
};


