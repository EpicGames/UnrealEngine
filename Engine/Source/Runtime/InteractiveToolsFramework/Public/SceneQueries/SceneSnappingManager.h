// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Misc/Optional.h"
#include "Math/Vector.h"
#include "Math/Ray.h"
#include "Engine/HitResult.h"
#include "ToolContextInterfaces.h"

#include "SceneSnappingManager.generated.h"

class AActor;
class ISnappingPolicy;
class UActorComponent;
class UInteractiveGizmoManager;
class UInteractiveToolManager;
class UPrimitiveComponent;
struct FSceneSnapQueryRequest;
struct FSceneSnapQueryResult;

struct FSceneQueryVisibilityFilter
{
	/** Optional: components to consider invisible even if they aren't. */
	const TArray<const UPrimitiveComponent*>* ComponentsToIgnore = nullptr;

	/** Optional: components to consider visible even if they aren't. */
	const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr;

	/** @return true if the Component is currently configured as visible (does not consider ComponentsToIgnore or InvisibleComponentsToInclude lists) */
	INTERACTIVETOOLSFRAMEWORK_API bool IsVisible(const UPrimitiveComponent* Component) const;
};

/** Types of Hit Targets that a caller may want to run hit queries against. */
UENUM()
enum class ESceneHitQueryTargetType : uint8
{
	None = 0,
	/** Consider any mesh vertex */
	MeshVertex = 1,
	/** Consider any mesh edge */
	MeshEdge = 2,
	/** Hit any primitive with a surface representation */
	PrimitiveSurface = 4,
	/** Any primitive with a transform representation  */
	PrimitiveTransform = 8,
	/** As defined by an FName */
	Custom = 16,

	All = MeshVertex | MeshEdge | PrimitiveSurface | PrimitiveTransform | Custom
};
ENUM_CLASS_FLAGS(ESceneHitQueryTargetType);


/**
* Configuration variables for a USceneSnappingManager hit query request.
*/
struct FSceneHitQueryRequest
{
	/** Scene query ray. */
	FRay WorldRay;

	/** What does the caller want to try to hit. */
	ESceneHitQueryTargetType TargetTypes = ESceneHitQueryTargetType::All;

	/** The name of the custom TargetTypes, if applicable. */
	TSet<FName> CustomTargetTypes;

	bool bWantHitGeometryInfo = false;

	FSceneQueryVisibilityFilter VisibilityFilter;
};


/**
* Computed result of a USceneSnappingManager hit query request
*/
struct FSceneHitQueryResult
{
	/** Actor that owns hit target */
	AActor* TargetActor = nullptr;
	/** Component that owns hit target */
	UPrimitiveComponent* TargetComponent = nullptr;

	/** hit position*/
	FVector3d Position = FVector3d::Zero();
	/** hit normal */
	FVector3d Normal = FVector3d::UnitZ();

	/** integer ID of triangle that was hit */
	int HitTriIndex = -1;
	/** Vertices of triangle that was hit (for debugging, may not be set) */
	FVector3d TriVertices[3];

	FHitResult HitResult;

	INTERACTIVETOOLSFRAMEWORK_API void InitializeHitResult(const FSceneHitQueryRequest& FromRequest);
};




/** Types of Snap Queries that a USceneSnappingManager may support */
UENUM()
enum class ESceneSnapQueryType : uint8
{
	/** Position/Translation snapping */
	Position = 1,
	/** Full rotation snapping (which could affect the axis of rotation) */
	Rotation = 2,
	/** Snaps the rotation angle only (in degrees). */
	RotationAngle = 3,
	/** Scale snapping */
	Scale = 4,
	/** Transform snapping (ie. snap to surface is both position and rotation) */
	Transform = 5
};


/** Types of Snap Targets that a caller may want to run snap queries against. */
UENUM()
enum class ESceneSnapQueryTargetType : uint8
{
	None = 0,
	/** Consider any mesh vertex */
	MeshVertex = 1,
	/** Consider any mesh edge */
	MeshEdge = 2,
	/** Grid Snapping */
	Grid = 4,
	/** Any object with a transform representation  */
	ObjectTransform = 8,
	/** As defined by an FName */
	Custom = 16,

	All = MeshVertex | MeshEdge | Grid | ObjectTransform | Custom
};
ENUM_CLASS_FLAGS(ESceneSnapQueryTargetType);

/** Describes the result of the attempted snap operation.
 * Note that this differs to FSceneSnapQueryResult in that it only
 * indicates the success or failure of the snap, not the resulting data. */
UENUM()
enum class ESceneSnapQueryTargetResult : uint8
{
	/** Snapping was valid and attempted, but did not snap to target. */
	NotSnapped,

	/** Snapping was valid and attempted, and did snap to target. This may indicate no further snapping operations should be attempted for the given interaction. */
	Snapped,

	/** Snapping is valid and supported, but is disabled (usually due to a user setting). */
	Disabled,

	/** The snapping query isn't valid for this Query Target. */
	Unsupported
};

/** A custom query target implementation, allowing extensibility and handling
 * of custom targets not defined in the ESceneSnapQueryTargetType enum. */
class FSceneSnapQueryTargetHandler
{
public:
	virtual ~FSceneSnapQueryTargetHandler() = default;

	virtual INTERACTIVETOOLSFRAMEWORK_API ESceneSnapQueryTargetResult SnapPosition(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;
	virtual ESceneSnapQueryTargetResult SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapPositionAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual INTERACTIVETOOLSFRAMEWORK_API ESceneSnapQueryTargetResult SnapRotation(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;
	virtual ESceneSnapQueryTargetResult SnapRotation(const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList = EAxisList::All) const { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapRotationAxisAngle(const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual INTERACTIVETOOLSFRAMEWORK_API ESceneSnapQueryTargetResult SnapScale(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const;
	virtual ESceneSnapQueryTargetResult SnapScale(const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList = EAxisList::All) const { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapScaleAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual ESceneSnapQueryTargetType GetTargetTypes() const { return ESceneSnapQueryTargetType::Custom; }

	/** Returns true if the given query type can -ever- be supported, regardless of any variable state. */
	virtual bool IsQueryTypeSupported(const ESceneSnapQueryType InQueryType) const { return false; }

	bool operator==(const FSceneSnapQueryTargetHandler& InOther) const
	{
		return TargetName == InOther.TargetName;
	}

	bool operator !=(const FSceneSnapQueryTargetHandler& InOther) const
	{
		return TargetName != InOther.TargetName;
	}

public:
	/** Unique query target name, ie. "Grid" */
	FName TargetName;
};

inline uint32 GetTypeHash(const FSceneSnapQueryTargetHandler& InSnapQueryTargetHandler)
{
	return GetTypeHash(InSnapQueryTargetHandler.TargetName);
}

/** A custom query target implementation with typed parameters, allowing extensibility and handling
 * of custom targets not defined in the ESceneSnapQueryTargetType enum. */
template <typename TargetRequestParametersType UE_REQUIRES(std::is_class_v<TargetRequestParametersType>)>
class TSceneSnapQueryTargetHandler : public FSceneSnapQueryTargetHandler
{
public:
	virtual ESceneSnapQueryTargetResult SnapPosition(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const override { return FSceneSnapQueryTargetHandler::SnapPosition(InRequest, OutResults); }
	virtual ESceneSnapQueryTargetResult SnapPosition(const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const override { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapPosition(const TargetRequestParametersType& InTargetRequest, const FVector& InPosition, FVector& OutSnappedPosition, const EAxisList::Type InAxisList = EAxisList::All) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual ESceneSnapQueryTargetResult SnapPositionAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const override { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapPositionAxis(const TargetRequestParametersType& InTargetRequest, const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual ESceneSnapQueryTargetResult SnapRotation(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const override { return FSceneSnapQueryTargetHandler::SnapRotation(InRequest, OutResults); }
	virtual ESceneSnapQueryTargetResult SnapRotation(const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList = EAxisList::All) const override { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapRotation(const TargetRequestParametersType& InTargetRequest, const FQuat& InRotation, FQuat& OutSnappedRotation, const EAxisList::Type InAxisList = EAxisList::All) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual ESceneSnapQueryTargetResult SnapRotationAxisAngle(const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const override { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapRotationAxisAngle(const TargetRequestParametersType& InTargetRequest, const double& InAngle, double& OutSnappedAngle, const EAxisList::Type InAxis) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual ESceneSnapQueryTargetResult SnapScale(const FSceneSnapQueryRequest& InRequest, TArray<FSceneSnapQueryResult>& OutResults) const override { return FSceneSnapQueryTargetHandler::SnapScale(InRequest, OutResults); }
	virtual ESceneSnapQueryTargetResult SnapScale(const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList = EAxisList::All) const override { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapScale(const TargetRequestParametersType& InTargetRequest, const FVector& InScale, FVector& OutSnappedScale, const EAxisList::Type InAxisList = EAxisList::All) const { return ESceneSnapQueryTargetResult::Unsupported; }

	virtual ESceneSnapQueryTargetResult SnapScaleAxis(const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const override { return ESceneSnapQueryTargetResult::Unsupported; }
	virtual ESceneSnapQueryTargetResult SnapScaleAxis(const TargetRequestParametersType& InTargetRequest, const double InAxisValue, double& OutSnappedAxisValue, const EAxisList::Type InAxis) const { return ESceneSnapQueryTargetResult::Unsupported; }
};

/**
 * Configuration variables for a USceneSnappingManager snap query request.
 */
struct FSceneSnapQueryRequest
{
	/** Coordinate space of the request data */
	EToolContextCoordinateSystem RequestCoordinateSpace;

	/** Coordinate transform, which may be the same as Transform but could also be a screen-space plane. */
	FTransform CoordinateTransform = FTransform::Identity;

	/** Optionally specify the desired result coordinate space, which can prevent unnecessary transforms. Defaults to the requested RequestCoordinateSpace */
	TOptional<EToolContextCoordinateSystem> ResultCoordinateSpace;

	/** What type of snap query geometry is this */
	ESceneSnapQueryType RequestType = ESceneSnapQueryType::Position;

	/** What does caller want to try to snap to */
	ESceneSnapQueryTargetType TargetTypes = ESceneSnapQueryTargetType::Grid;

	/** The name of the custom TargetTypes, if applicable. */
	TSet<FName> CustomTargetTypes;

	/** Optional explicitly specified position grid */
	TOptional<FVector> GridSize{};

	/** Optional explicitly specified rotation grid */
	TOptional<FRotator> RotGridSize{};

	/** Optional explicitly specified scale grid */
	TOptional<FVector> ScaleGridSize{};

	/** Snap input transform in absolute coordinates (not delta) */
	FTransform Transform = FTransform::Identity;

	/** Snap input position */
	FVector Position = FVector::ZeroVector;

	/**
	 *  When considering if one point is close enough to another point for snapping purposes, they
	 *  must deviate less than this number of degrees (in visual angle) to be considered an acceptable snap position.
	 */
	float VisualAngleThresholdDegrees = 15.0;

	/** Snap input rotation delta */
	FQuat DeltaRotation = FQuat(EForceInit::ForceInitToZero);

	/** Snap input rotation angle */
	double RotationAngle = 0.0;

	/** Snap input axis constraints, defaults to all (unconstrained) */
	EAxisList::Type AxisList = EAxisList::All;

	/** Snap input scale */
	FVector Scale = FVector::OneVector;

	/** Scene query ray, required by some snapping operations. */
	FRay WorldRay;

	/** Optional: components to consider invisible even if they aren't. */
	const TArray<const UPrimitiveComponent*>* ComponentsToIgnore = nullptr;

	/** Optional: components to consider visible even if they aren't. */
	const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr;

	/** @return The ResultCoordinateSpace if set, otherwise the RequestCoordinateSpace */
	INTERACTIVETOOLSFRAMEWORK_API EToolContextCoordinateSystem GetResultCoordinateSpace() const;
};


/**
 * Computed result of a USceneSnappingManager snap query request
 */
struct FSceneSnapQueryResult
{
	/** (Optionally set) Coordinate space of the result data */
	TOptional<EToolContextCoordinateSystem> CoordinateSpace;

	/** Actor that owns snap target */
	AActor* TargetActor = nullptr;
	/** Component that owns snap target */
	UActorComponent* TargetComponent = nullptr;
	/** What kind of geometric element(s) were snapped to */
	ESceneSnapQueryTargetType TargetType = ESceneSnapQueryTargetType::None;

	/** Snap position (may not be set depending on query types) */
	FVector Position = FVector::ZeroVector;
	/** Snap normal (may not be set depending on query types) */
	FVector Normal;
	/** Snap rotation delta (may not be set depending on query types) */
	FQuat   DeltaRotation = FQuat::Identity;
	/** Snap rotation angle (may not be set depending on query types) */
	double  RotationAngle = 0.0;
	/** Snap scale (may not be set depending on query types) */
	FVector Scale = FVector::ZeroVector;

	/** Vertices of triangle that contains result (for debugging, may not be set) */
	FVector TriVertices[3];
	/** Vertex/Edge index we snapped to in triangle */
	int TriSnapIndex;

	/** @return The CoordinateSpace if set, otherwise the ResultCoordinateSpace from the provided request */
	INTERACTIVETOOLSFRAMEWORK_API EToolContextCoordinateSystem GetCoordinateSpace(const FSceneSnapQueryRequest& InFromRequest) const;
};


/**
 * USceneSnappingManager is intended to be used as a base class for a Snapping implementation
 * stored in the ContextObjectStore of an InteractiveToolsContext. ITF classes like Tools and Gizmos
 * can then access this object and run snap queries via the various API functions.
 * 
 * USceneSnappingManager::Find() can be used to look up a registered USceneSnappingManager, if one is available
 * 
 * See UModelingSceneSnappingManager for a sample implementation.
 */
UCLASS(MinimalAPI)
class USceneSnappingManager : public UObject
{
	GENERATED_BODY()
public:

	/**
	* Try to find a Hit Object in the scene that satisfies the Hit Query
	* @param Request hit query configuration
	* @param Results hit query result
	* @return true if any valid hit target was found
	* @warning implementations are not required (and may not be able) to support hit testing
	*/
	virtual bool ExecuteSceneHitQuery(const FSceneHitQueryRequest& Request, FSceneHitQueryResult& ResultOut) const
	{
		return false;
	}

	/**
	* Try to find a Hit Object in the scene that satisfies the Hit Query, returning all candidate results
	* @param Request hit query configuration
	* @param Results hit query results
	* @return true if any valid hit target was found
	* @warning implementations are not required (and may not be able) to support hit testing
	*/
	virtual bool ExecuteSceneHitQuery(const FSceneHitQueryRequest& Request, TArray<FSceneHitQueryResult>& ResultsOut) const
	{
		return false;
	}

	/**
	* Try to find Snap Targets/Results in the scene that satisfy the Snap Query.
	* @param Request snap query configuration
	* @param Results list of potential snap results
	* @return true if any valid snap target/result was found
	* @warning implementations are not required (and may not be able) to support snapping
	*/
	virtual bool ExecuteSceneSnapQuery(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& ResultsOut) const
	{
		return false;
	}


public:
	/**
	 * @return existing USceneSnappingManager registered in context store via the ToolManager, or nullptr if not found
	 */
	static INTERACTIVETOOLSFRAMEWORK_API USceneSnappingManager* Find(UInteractiveToolManager* ToolManager);

	/**
	 * @return existing USceneSnappingManager registered in context store via the ToolManager, or nullptr if not found
	 */
	static INTERACTIVETOOLSFRAMEWORK_API USceneSnappingManager* Find(UInteractiveGizmoManager* GizmoManager);
};
