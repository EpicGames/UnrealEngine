// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SelectionSet.generated.h"

class USelectionSet;

/** This delegate is used by UInteractiveToolPropertySet */
DECLARE_MULTICAST_DELEGATE_OneParam(FSelectionSetModifiedSignature, USelectionSet*);


/**
 * USelectionSet is a base class for Selection objects. 
 * For example the UMeshSelectionSet implementation stores information on selected
 * triangles, vertices, etc. 
 */
UCLASS(Transient, MinimalAPI)
class USelectionSet : public UObject
{
	GENERATED_BODY()

protected:
	FSelectionSetModifiedSignature OnModified;


public:

	/** @return the multicast delegate that is called when properties are modified */
	FSelectionSetModifiedSignature& GetOnModified()
	{
		return OnModified;
	}

	/** posts a message to the OnModified delegate with the modified FProperty */
	void NotifySelectionSetModified()
	{
		OnModified.Broadcast(this);
	}

};




enum class EMeshSelectionElementType
{
	Vertex = 0,
	Face = 1,
	Edge = 2,
	Group = 3
};



/**
 * UMeshSelectionSet is an implementation of USelectionSet that represents selections on indexed meshes.
 * Vertices, Edges, Faces, and Groups can be selected.
 */
UCLASS(Transient, MinimalAPI)
class UMeshSelectionSet : public USelectionSet
{
	GENERATED_BODY()

public:
	INTERACTIVETOOLSFRAMEWORK_API UMeshSelectionSet();

	UPROPERTY()
	TArray<int32> Vertices;

	UPROPERTY()
	TArray<int32> Edges;

	UPROPERTY()
	TArray<int32> Faces;

	UPROPERTY()
	TArray<int32> Groups;


public:
	/** @return current selected elements of the given ElementType */
	INTERACTIVETOOLSFRAMEWORK_API TArray<int>& GetElements(EMeshSelectionElementType ElementType);

	/** @return current selected elements of the given ElementType */
	INTERACTIVETOOLSFRAMEWORK_API const TArray<int>& GetElements(EMeshSelectionElementType ElementType) const;

	/**
	 * Add list of Indices to the current selection for the given ElementType
	 * @warning no duplicate detection is currently done
	 */
	INTERACTIVETOOLSFRAMEWORK_API void AddIndices(EMeshSelectionElementType ElementType, const TArray<int32>& Indices);

	/**
	 * Add list of Indices to the current selection for the given ElementType
	 * @warning no duplicate detection is currently done
	 */
	INTERACTIVETOOLSFRAMEWORK_API void AddIndices(EMeshSelectionElementType ElementType, const TSet<int32>& Indices);


	/**
	 * Remove list of Indices from the current selection for the given ElementType
	 */
	INTERACTIVETOOLSFRAMEWORK_API void RemoveIndices(EMeshSelectionElementType ElementType, const TArray<int32>& Indices);

	/**
	 * Remove list of Indices from the current selection for the given ElementType
	 */
	INTERACTIVETOOLSFRAMEWORK_API void RemoveIndices(EMeshSelectionElementType ElementType, const TSet<int32>& Indices);
};
