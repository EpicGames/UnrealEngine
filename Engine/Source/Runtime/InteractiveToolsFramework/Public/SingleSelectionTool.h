// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveTool.h"
#include "ComponentSourceInterfaces.h"
#include "ToolTargets/ToolTarget.h"
#include "InteractiveToolQueryInterfaces.h"

#include "SingleSelectionTool.generated.h"

UCLASS(Transient, MinimalAPI)
class USingleSelectionTool : public UInteractiveTool, public IInteractiveToolCameraFocusAPI
{
GENERATED_BODY()
public:
	/**
	 * Set the ToolTarget for the Tool
	 */
	virtual void SetTarget(UToolTarget* TargetIn)
	{
		Target = TargetIn;
	}

	/**
	 * @return the ToolTarget for the Tool
	 */
	virtual UToolTarget* GetTarget()
	{
		return Target;
	}

	/**
	 * @return true if all ToolTargets of this tool are still valid
	 */
	virtual bool AreAllTargetsValid() const
	{
		return Target ? Target->IsValid() : false;
	}


public:
	virtual bool CanAccept() const override
	{
		return AreAllTargetsValid();
	}

protected:
	UPROPERTY()
	TObjectPtr<UToolTarget> Target;



public:
	// IInteractiveToolCameraFocusAPI implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SupportsWorldSpaceFocusBox() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FBox GetWorldSpaceFocusBox() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool SupportsWorldSpaceFocusPoint() override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) override;

};
