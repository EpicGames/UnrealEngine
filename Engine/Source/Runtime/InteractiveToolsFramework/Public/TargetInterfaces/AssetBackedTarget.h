// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/Interface.h"
#include "ToolTargets/ToolTarget.h"

#include "AssetBackedTarget.generated.h"

UINTERFACE(MinimalAPI)
class UAssetBackedTarget : public UInterface
{
	GENERATED_BODY()
};

class IAssetBackedTarget
{
	GENERATED_BODY()

public:
	/**
	 * @return the underlying source asset for this Target.
	 */
	virtual UObject* GetSourceData() const = 0;

	virtual bool HasSameSourceData(UToolTarget* OtherTarget) const
	{
		IAssetBackedTarget* OtherAsset = Cast<IAssetBackedTarget>(OtherTarget);
		return OtherAsset && GetSourceData() == OtherAsset->GetSourceData();
	}
};
