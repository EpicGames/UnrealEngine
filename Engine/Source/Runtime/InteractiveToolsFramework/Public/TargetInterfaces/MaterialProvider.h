// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "MaterialProvider.generated.h"

class UMaterialInterface;

/**
 * FComponentMaterialSet is the set of materials assigned to a component (ie Material Slots on a StaticMesh)
 */
struct FComponentMaterialSet
{
	TArray<UMaterialInterface*> Materials;

	bool operator!=(const FComponentMaterialSet& Other) const
	{
		return Materials != Other.Materials;
	}
};

UINTERFACE(MinimalAPI)
class UMaterialProvider : public UInterface
{
	GENERATED_BODY()
};

class IMaterialProvider
{
	GENERATED_BODY()

public:

	/** @return number of material indices in use by this Component */
	virtual int32 GetNumMaterials() const = 0;

	/**
	 * Get pointer to a Material provided by this Source
	 * @param MaterialIndex index of the material
	 * @return MaterialInterface pointer, or null if MaterialIndex is invalid
	 */
	virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const = 0;

	/**
	 * Get material set provided by this source
	 * @param MaterialSetOut returned material set
	 * @param bAssetMaterials Prefer the underlying asset materials. This may be ignored by targets
	 *  to which this is not relevant.
	 */
	virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials = false) const = 0;

	/**
	 * Commit an update to the material set. This may generate a transaction.
	 *
	 * Note that a target may not allow applying the material to an underlying asset, or may not even
	 * allow having the material applied to the instance, in which case the function will return false
	 * without doing anything.
	 *
	 * TODO: We may decide that we want some functions to query whether the object allows
	 * one or the other, or we may break this out into a separate interface. We'll wait to
	 * see when this actually becomes necessary to see the most convenient way to go at that
	 * point.
	 *
	 * @param MaterialSet new list of materials
	 * @return true if successful
	 */
	virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) = 0;
};
