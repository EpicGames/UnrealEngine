// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshDescription.h"
#include "UObject/Interface.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "MeshDescriptionCommitter.generated.h"

UINTERFACE(MinimalAPI)
class UMeshDescriptionCommitter : public UInterface
{
	GENERATED_BODY()
};

class IMeshDescriptionCommitter
{
	GENERATED_BODY()

public:

	/**
	 * Commits the given FMeshDescription.
	 */
	virtual bool CommitMeshDescription(const FMeshDescription& Mesh, const FCommitMeshParameters& CommitParams = FCommitMeshParameters() )
	{
		// It seems reasonable to have this function, but we'll go ahead and give a default implementation
		// if users want to just implement the other one.
		bool bSuccess = false;
		CommitMeshDescription([&](const FCommitterParams& CommitParams)
		{
			if (CommitParams.MeshDescriptionOut)
			{
				*CommitParams.MeshDescriptionOut = Mesh;
				bSuccess = true;
			}
		}, CommitParams);
		return bSuccess;
	}


	/**
	 * Commits the given FMeshDescription.
	 */
	virtual bool CommitMeshDescription(FMeshDescription&& Mesh, const FCommitMeshParameters& CommitParams = FCommitMeshParameters())
	{
		// It seems reasonable to have this function, but we'll go ahead and give a default implementation
		// if users want to just implement the other one.
		bool bSuccess = false;
		CommitMeshDescription([&](const FCommitterParams& CommitParams)
		{
			if (CommitParams.MeshDescriptionOut)
			{
				*CommitParams.MeshDescriptionOut = MoveTemp(Mesh);
				bSuccess = true;
			}
		}, CommitParams);
		return bSuccess;
	}

	/**
	 * @return a mapping from material ID to polygon group ID. If empty, an identity mapping is assumed.
	 */
	virtual TArray<int32> GetPolygonGroupToMaterialIndexMap() const
	{
		return TArray<int32>();
	}

protected:

	struct FCommitterParams
	{
		/** 
		* Mesh description that should be populated/updated by the passed-in function and which
		* will be committed to the target.
		*/
		FMeshDescription* MeshDescriptionOut  = nullptr;
	};
	using FCommitter = TFunction<void(const FCommitterParams&)>;

	/** 
	* Commit a mesh description. The mesh description to be committed will be passed to the
	* given function as a parameter, and it is up to the function to update it properly.
	*
	* @param Committer A function that takes in const IMeshDescriptionCommitter::FCommitParams&
	*  and populates the FMeshDescription pointed to by the MeshDescription pointer inside.
	*/
	virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams = FCommitMeshParameters()) = 0;

};
