// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"
#include "MeshDescription.h"

#include "MeshDescriptionProvider.generated.h"

UINTERFACE(MinimalAPI)
class UMeshDescriptionProvider : public UInterface
{
	GENERATED_BODY()
};

class IMeshDescriptionProvider
{
	GENERATED_BODY()

public:
	/**
	 * Access the MeshDescription available through this Provider. Note that this MeshDescription may or may not 
	 * be owned by the provider and should not be modified directly. Use IMeshDescriptionCommitter for writes.
	 * @return pointer to MeshDescription 
	 */
	virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) = 0;

	/**
	 * Returns an empty mesh description appropriate for the provider, i.e. configured with appropriate mesh
	   attributes but otherwise devoid of topology or element data.

	   Note: Some of our code expects at least FStaticMeshAttributes to be registered with the provided
	   mesh description, and will break if some of the attributes that FStaticMeshAttributes uses are not present.
	   The only reason we don't provide a default implementation here that is identical to the one used in
	   StaticMeshToolTarget.cpp is to avoid a dependency on the static mesh description module in ITF.
	 */
	virtual FMeshDescription GetEmptyMeshDescription() = 0;

	/**
	 * Get a copy of the MeshDescription available through this Provider. 
	 */
	virtual FMeshDescription GetMeshDescriptionCopy(const FGetMeshParameters& GetMeshParams)
	{
		return *GetMeshDescription(GetMeshParams);
	}

	/**
	 * For providers that have LODs (i.e. if SupportsLODs returns true), returns an array of all available LODs. Otherwise, returns an array with just the Default LOD.
	 * @param bSkipAutoGenerated	Skip auto-generated LODs, which have no associated source Mesh Description
	 */
	virtual TArray<EMeshLODIdentifier> GetAvailableLODs(bool bSkipAutoGenerated = true) const
	{
		TArray<EMeshLODIdentifier> DefaultLODArray;
		DefaultLODArray.Add(EMeshLODIdentifier::Default);
		return DefaultLODArray;
	}

	/**
	 * For providers that have LODs (i.e. if SupportsLODs returns true), returns the LOD that GetMeshDescription/etc will return. Otherwise, returns Default LOD.
	 */
	virtual EMeshLODIdentifier GetMeshDescriptionLOD() const
	{
		return EMeshLODIdentifier::Default;
	}

	/**
	 * @return true if the provider is able to provide different Mesh Descriptions for different requested LODs.  If so, GetAvailableLODs() will return the available LODs.
	 */
	virtual bool SupportsLODs() const
	{
		return false;
	}

	/**
	 * @return a mapping from section ID to material ID. If empty, an identity mapping is assumed.
	 */
	virtual TArray<int32> GetPolygonGroupToMaterialIndexMap() const
	{
		return TArray<int32>();
	}

};
