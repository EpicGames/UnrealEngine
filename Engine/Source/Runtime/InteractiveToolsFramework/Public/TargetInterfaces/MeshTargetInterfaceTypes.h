// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
* EMeshLODIdentifier is used to identify the LOD of a mesh for reading/writing.
*/
enum class EMeshLODIdentifier
{
	LOD0 = 0,
	LOD1 = 1,
	LOD2 = 2,
	LOD3 = 3,
	LOD4 = 4,
	LOD5 = 5,
	LOD6 = 6,
	LOD7 = 7,

	HiResSource = 100,	// HiRes source mesh is optional, and will fall back to LOD0 if not available

	Default = 1000,		// use whatever is the "standard" LOD - generally LOD0
	MaxQuality = 1001	// use HiRes source mesh if available, or LOD0 otherwise
};
typedef EMeshLODIdentifier EStaticMeshEditingLOD;

/**
 * FGetMeshParameters is used by ToolTarget Interfaces/Implementations that support returning a mesh, to allow clients
 * to specify various options like a specific LOD of a mesh, which attributes are required, etc.
 */
struct FGetMeshParameters
{
	FGetMeshParameters() = default;
	explicit FGetMeshParameters(bool bHaveRequestLOD, EMeshLODIdentifier RequestLOD)
		: bHaveRequestLOD(bHaveRequestLOD), RequestLOD(RequestLOD)
	{}

	/** If true, RequestLOD specifies which mesh LOD should be returned, if available. If not available, behavior is implementation-dependent. */
	bool bHaveRequestLOD = false;
	/** Specify which LOD to fetch from a mesh-containing ToolTarget. Ignored unless bHaveRequestLOD == true. */
	EMeshLODIdentifier RequestLOD = EMeshLODIdentifier::Default;

	/** 
	 * If true, returned Mesh should have Tangents available. In some cases (eg StaticMesh) this may require that the 
	 * ToolTarget implementation computes the Tangents, which may only be possible on a copy of the mesh
	 */
	bool bWantMeshTangents = false;
};


/**
 * FCommitMeshParameters is used by ToolTarget Interfaces/Implementations that support setting a mesh (eg on a StaticMesh Asset),
 * to allow the client to specify various options
 */
struct FCommitMeshParameters
{
	FCommitMeshParameters() = default;
	explicit FCommitMeshParameters(bool bHaveTargetLOD, EMeshLODIdentifier TargetLOD) :
		bHaveTargetLOD(bHaveTargetLOD), TargetLOD(TargetLOD)
	{}

	/** If true, TargetLOD specifies which mesh LOD should be set/updated */
	bool bHaveTargetLOD = false;
	/** Specify which LOD to set in a mesh-containing ToolTarget. Ignored unless bHaveTargetLOD == true. */
	EMeshLODIdentifier TargetLOD = EMeshLODIdentifier::Default;
};
