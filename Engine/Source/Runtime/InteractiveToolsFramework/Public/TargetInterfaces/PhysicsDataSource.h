// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "PhysicsDataSource.generated.h"

class UBodySetup;
class IInterface_CollisionDataProvider;

UINTERFACE(MinimalAPI)
class UPhysicsDataSource : public UInterface
{
	GENERATED_BODY()
};

/**
 * IPhysicsDataSource is a ToolTarget Interface that provides read/write access to physics-related data structures.
 */
class IPhysicsDataSource
{
	GENERATED_BODY()

public:

	/**
	 * @return The UBodySetup for this physics data source. If Nullptr, no physics data exists or is available.
	 */
	virtual UBodySetup* GetBodySetup() const = 0;


	/**
	* @return The CollisionDataProvider for this physics data source. If Nullptr, no physics data exists or is available.
	*/
	virtual IInterface_CollisionDataProvider* GetComplexCollisionProvider() const = 0;

};
