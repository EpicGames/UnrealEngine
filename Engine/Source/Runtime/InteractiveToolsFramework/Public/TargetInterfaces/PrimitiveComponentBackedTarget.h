// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"

#include "PrimitiveComponentBackedTarget.generated.h"

class UPrimitiveComponent;
class UActorComponent;
class AActor;

struct FHitResult;


UINTERFACE(MinimalAPI)
class UPrimitiveComponentBackedTarget : public USceneComponentBackedTarget
{
	GENERATED_BODY()
};

class IPrimitiveComponentBackedTarget : public ISceneComponentBackedTarget
{
	GENERATED_BODY()

public:

	/** @return the Component this is a Source for */
	virtual UPrimitiveComponent* GetOwnerComponent() const = 0;

	/**
	 * Compute ray intersection with geometry this Source is providing (if any)
	 * @param WorldRay ray in world space
	 * @param OutHit hit test data
	 * @return true if ray intersected Component
	 */
	virtual bool HitTestComponent(const FRay& WorldRay, FHitResult& OutHit) const = 0;
};
