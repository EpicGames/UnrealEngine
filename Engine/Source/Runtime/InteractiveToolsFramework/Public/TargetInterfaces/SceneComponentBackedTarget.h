// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "SceneComponentBackedTarget.generated.h"

class USceneComponent;
class AActor;

struct FHitResult;

UINTERFACE(MinimalAPI)
class USceneComponentBackedTarget : public UInterface
{
	GENERATED_BODY()
};

class ISceneComponentBackedTarget
{
	GENERATED_BODY()

public:

	/** @return the ActorComponent this is a Source for */
	virtual USceneComponent* GetOwnerSceneComponent() const = 0;

	/** @return the Actor that owns this Component */
	virtual AActor* GetOwnerActor() const = 0;

	/**
	 * Set the visibility of the Component associated with this Source (ie to hide during Tool usage)
	 * @param bVisible desired visibility
	 */
	virtual void SetOwnerVisibility(bool bVisible) const = 0;

	/**
	 * @return the transform on this component
	 * @todo Do we need to return a list of transforms here?
	 */
	virtual FTransform GetWorldTransform() const = 0;
};

