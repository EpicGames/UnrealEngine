// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/SkeletalMesh.h"
#include "TargetInterfaces/AssetBackedTarget.h"

#include "SkeletalMeshBackedTarget.generated.h"

UINTERFACE(MinimalAPI)
class USkeletalMeshBackedTarget : public UAssetBackedTarget
{
	GENERATED_BODY()
};

class ISkeletalMeshBackedTarget : public IAssetBackedTarget
{
	GENERATED_BODY()

	public:
	/**
	* @return the underlying source asset for this Target.
	*/
	virtual UObject* GetSourceData() const override
	{
		return GetSkeletalMesh();
	}

	/**
	* @return the underlying source USkeletalMesh asset for this Target
	*/
	virtual USkeletalMesh* GetSkeletalMesh() const = 0;
};
