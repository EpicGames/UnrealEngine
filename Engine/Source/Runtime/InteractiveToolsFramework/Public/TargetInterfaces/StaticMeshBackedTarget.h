// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "UObject/Interface.h"
#include "TargetInterfaces/AssetBackedTarget.h"

#include "StaticMeshBackedTarget.generated.h"

UINTERFACE(MinimalAPI)
class UStaticMeshBackedTarget : public UAssetBackedTarget
{
	GENERATED_BODY()
};

class IStaticMeshBackedTarget : public IAssetBackedTarget
{
	GENERATED_BODY()

public:
	/**
	 * @return the underlying source asset for this Target.
	 */
	virtual UObject* GetSourceData() const override
	{
		return GetStaticMesh();
	}

	/**
	 * @return the underlying source StaticMesh asset for this Target
	 */
	virtual UStaticMesh* GetStaticMesh() const = 0;
};
