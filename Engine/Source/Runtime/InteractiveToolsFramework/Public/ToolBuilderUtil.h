// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/Count.h"
#include "Algo/Find.h"
#include "GameFramework/Actor.h"
#include "Engine/Selection.h"
#include "InteractiveToolBuilder.h"

/**
* Helper functions that can be used in InteractiveToolBuilder implementations
*/
namespace ToolBuilderUtil
{
	/** Count number of selected components that pass predicate. If Component selection is not empty, returns that count, otherwise counts in all selected Actors */
	INTERACTIVETOOLSFRAMEWORK_API
	int CountComponents(const FToolBuilderState& InputState, const TFunction<bool(UActorComponent*)>& Predicate);

	/** First first available component that passes predicate. Searches Components selection list first, then all Actors */
	INTERACTIVETOOLSFRAMEWORK_API
	UActorComponent* FindFirstComponent(const FToolBuilderState& InputState, const TFunction<bool(UActorComponent*)>& Predicate);

	/** First all components that passes predicate. Searches Components selection list first, then all Actors */
	INTERACTIVETOOLSFRAMEWORK_API
	TArray<UActorComponent*> FindAllComponents(const FToolBuilderState& InputState, const TFunction<bool(UActorComponent*)>& Predicate);

	// @todo not sure that actors with multiple components are handled properly...
	/** Count number of components of given type. If Component selection is not empty, returns that count, otherwise counts in all selected Actors */
	template<typename ComponentType>
	int CountSelectedComponentsOfType(const FToolBuilderState& InputState);

	/** First first available component of given type. Searches Components selection list first, then all Actors */
	template<typename ComponentType>
	ComponentType* FindFirstComponentOfType(const FToolBuilderState& InputState);

	INTERACTIVETOOLSFRAMEWORK_API
	void EnumerateComponents(const FToolBuilderState& InputState, TFunctionRef<void(UActorComponent*)> ComponentFunc);

	/** Count number of selected Actors that pass predicate. */
	INTERACTIVETOOLSFRAMEWORK_API
	int32 CountActors(const FToolBuilderState& InputState, const TFunction<bool(AActor*)>& Predicate);

	/** First first available Actor that passes predicate. Searches Actors selection list. */
	INTERACTIVETOOLSFRAMEWORK_API
	AActor* FindFirstActor(const FToolBuilderState& InputState, const TFunction<bool(AActor*)>& Predicate);

	/** First all Actors that pass predicate. Searches Actors selection list. */
	INTERACTIVETOOLSFRAMEWORK_API
	TArray<AActor*> FindAllActors(const FToolBuilderState& InputState, const TFunction<bool(AActor*)>& Predicate);

	/** Count number of selected Actors of given type. */
	template<typename ActorType>
	int CountSelectedActorsOfType(const FToolBuilderState& InputState);

	/** Find first first available Actor of given type, or return nullptr if not found */
	template<typename ActorType>
	ActorType* FindFirstActorOfType(const FToolBuilderState& InputState);

	/** Determine if component support UVs. */
	INTERACTIVETOOLSFRAMEWORK_API
	bool ComponentTypeCouldHaveUVs(const UActorComponent& Component);

	/** Determine if component is a volume or not. */
	INTERACTIVETOOLSFRAMEWORK_API
	bool IsVolume(const UActorComponent& Component);
}

/*
 * Template Implementations
 */
template<typename ComponentType>
int ToolBuilderUtil::CountSelectedComponentsOfType(const FToolBuilderState& InputState)
{
	return CountComponents(InputState, [](UActorComponent* Component) { return Cast<ComponentType>(Component) != nullptr; });
}

template<typename ComponentType>
ComponentType* ToolBuilderUtil::FindFirstComponentOfType(const FToolBuilderState& InputState)
{
	return static_cast<ComponentType*>(FindFirstComponent(InputState, [](UActorComponent* Component) { return Cast<ComponentType>(Component) != nullptr; }));
}

template<typename ActorType>
int ToolBuilderUtil::CountSelectedActorsOfType(const FToolBuilderState& InputState)
{
	return CountActors(InputState, [](AActor* Actor) { return Cast<ActorType>(Actor) != nullptr; });
}

template<typename ActorType>
ActorType* ToolBuilderUtil::FindFirstActorOfType(const FToolBuilderState& InputState)
{
	AActor* Found = FindFirstActor(InputState, [](AActor* Actor) { return Cast<ActorType>(Actor) != nullptr; });
	return (Found != nullptr) ? Cast<ActorType>(Found) : nullptr;
}