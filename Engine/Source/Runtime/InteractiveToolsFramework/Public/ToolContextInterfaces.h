// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentSourceInterfaces.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementSelectionSet.h" // For deprecated FToolBuilderState::TypedElementSelectionSet member
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/UniquePtr.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "GenericPlatform/ICursor.h"

#include "ToolContextInterfaces.generated.h"

// predeclarations so we don't have to include these in all tools
class AActor;
class FPrimitiveDrawInterface;
class FSceneView;
class FText;
class FToolCommandChange;
class FViewport;
class SWidget;
class UActorComponent;
class UInteractiveGizmoManager;
class UInteractiveToolManager;
class UMaterialInterface;
class UObject;
class UToolTargetManager;
class UTypedElementSelectionSet;

class UWorld;
struct FMeshDescription;
struct FTypedElementHandle;

/**
 * FToolBuilderState is a bucket of state information that a ToolBuilder might need
 * to construct a Tool. This information comes from a level above the Tools framework,
 * and depends on the context we are in (Editor vs Runtime, for example).
 */
struct FToolBuilderState
{
	/** The current UWorld */
	UWorld* World = nullptr;
	/** The current ToolManager */
	UInteractiveToolManager* ToolManager = nullptr;
	/** The current TargetManager */
	UToolTargetManager* TargetManager = nullptr;
	/** The current GizmoManager */
	UInteractiveGizmoManager* GizmoManager = nullptr;

	/** Current selected Actors. May be empty or nullptr. */
	TArray<AActor*> SelectedActors;
	/** Current selected Components. May be empty or nullptr. */
	TArray<UActorComponent*> SelectedComponents;

	UE_DEPRECATED(5.1, "This has moved to a context object. See IAssetEditorContextInterface")
	TWeakObjectPtr<UTypedElementSelectionSet> TypedElementSelectionSet;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FToolBuilderState() = default;
	FToolBuilderState(FToolBuilderState&&) = default;
	FToolBuilderState(const FToolBuilderState&) = default;
	FToolBuilderState& operator=(FToolBuilderState&&) = default;
	FToolBuilderState& operator=(const FToolBuilderState&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


/**
 * FViewCameraState is a bucket of state information that a Tool might
 * need to implement interactions that depend on the current scene view.
 */
struct FViewCameraState
{
	/** Current camera/head position */
	FVector Position = FVector(EForceInit::ForceInit);
	/** Current camera/head orientation */
	FQuat Orientation = FQuat(EForceInit::ForceInit);
	/** Current Horizontal Field of View Angle in Degrees. Only relevant if bIsOrthographic is false. */
	float HorizontalFOVDegrees = 90;
	/** Current width of viewport in world space coordinates. Only valid if bIsOrthographic is true. */
	float OrthoWorldCoordinateWidth = 100;
	/** Current Aspect Ratio */
	float AspectRatio = 1;
	/** Is current view an orthographic view */
	bool bIsOrthographic = false;
	/** Is current view a VR view */
	bool bIsVR = false;
	/** DPI scale */
	double DPIScale = 1.0;

	/** @return "right"/horizontal direction in camera plane */
	FVector Right() const { return Orientation.GetAxisY(); }
	/** @return "up"/vertical direction in camera plane */
	FVector Up() const { return Orientation.GetAxisZ(); }
	/** @return forward camera direction */
	FVector Forward() const { return Orientation.GetAxisX(); }

	/** @return scaling factor that should be applied to PDI thickness/size */
	float GetPDIScalingFactor() const { return HorizontalFOVDegrees / 90.0f; }

	/** @return FOV normalization factor that should be applied when comparing angles */
	float GetFOVAngleNormalizationFactor() const { return HorizontalFOVDegrees / 90.0f; }

};



/** Types of standard materials that Tools may request from Context */
UENUM()
enum class EStandardToolContextMaterials
{
	/** White material that displays vertex colors set on mesh */
	VertexColorMaterial = 1
};


/** Types of coordinate systems that a Tool/Gizmo might use */
UENUM()
enum class EToolContextCoordinateSystem : uint8
{
	/** World space coordinate system */
	World = 0,
	/** Local coordinate system */
	Local = 1,
	/** Screen space coordinate system */
	Screen = 2
};


/** 
 * High-level configuration options for a standard 3D translate/rotate/scale Gizmo, like is commonly used in 3D DCC tools, game editors, etc
 * This is meant to be used to convey UI-level settings to Tools/Gizmos, eg like the W/E/R toggles for Rranslate/Rotate/Scale in Maya or UE Editor.
 * More granular control over precise gizmo elements is available through other mechanisms, eg the ETransformGizmoSubElements flags in UCombinedTransformGizmo
 */
UENUM()
enum class EToolContextTransformGizmoMode : uint8
{
	/** Hide all Gizmo sub-elements */
	NoGizmo = 0,
	/** Only Show Translation sub-elements */
	Translation = 1,
	/** Only Show Rotation sub-elements */
	Rotation = 2,
	/** Only Show Scale sub-elements */
	Scale = 3,
	/** Show all available Gizmo sub-elements */
	Combined = 8
};



/**
 * Snapping configuration settings/state
 */
struct FToolContextSnappingConfiguration
{
	/** Whether to perform any kind of snapping */
	bool bIsSnappingActive = true;

	/** Specify whether position snapping should be applied */
	bool bEnablePositionGridSnapping = false;
	/** Specify position grid spacing or cell dimensions */
	FVector PositionGridDimensions = FVector::Zero();

	/** 
	 * @return true if position grid snapping is active (snapping enabled AND position grid snapping enabled)
	 */
	bool IsPositionGridSnappingActive() const { return bIsSnappingActive && bEnablePositionGridSnapping; }

	/** Specify whether rotation snapping should be applied */
	bool bEnableRotationGridSnapping = false;
	/** Specify rotation snapping step size */
	FRotator RotationGridAngles = FRotator::ZeroRotator;

	/** 
	 * @return true if rotation grid/angular snapping is active (snapping enabled AND rotation grid snapping enabled)
	 */
	bool IsRotationGridSnappingActive() const { return bIsSnappingActive && bEnableRotationGridSnapping; }

	/** Specify whether scale snapping should be applied */
	bool bEnableScaleGridSnapping = false;
	/** Specify scale factor */
	float ScaleGridSize = 1.0f;

	/** 
	 * @return true if scale grid snapping is active (snapping enabled AND scale grid snapping enabled)
	 */
	bool IsScaleGridSnappingActive() const { return bIsSnappingActive && bEnableScaleGridSnapping; }

	/** Specify whether Position snapping in World Coordinate System should be Absolute or Relative */
	bool bEnableAbsoluteWorldSnapping = false;

	/** Stores parameters applicable to actor/object transform snapping, corresponds with ESceneSnapQueryTargetType::ObjectTransform */
	struct FObjectTransformSnapping
	{
		/** Specify whether this snapping type is enabled */
		bool bEnable = false;

		/** The distance/radius around a candidate transform to automatically snap to. 0.0 requires precise placement. */
		float SnapDistance = 0.0f;
	} ObjectTransform;
	
	/** 
	 * @return true if object transform snapping is active (snapping enabled AND object transform snapping enabled)
	 */
	bool IsObjectTransformSnappingActive() const { return bIsSnappingActive && ObjectTransform.bEnable; }
};


/**
* Users of the Tools Framework can use  UToolsContextCursorAPI to
* access to setting and retrieving cursor overrides for tool activities.
*/
UCLASS(MinimalAPI)
class UToolsContextCursorAPI : public UObject
{
	GENERATED_BODY()

private:
	EMouseCursor::Type CursorOverrideShape;
	TSharedPtr<SWidget> CursorOverrideWidget;
	bool bIsCursorOverridden;
	
	TOptional<bool> UseSoftwareCursorOverride;

public:

	UToolsContextCursorAPI()
	{
		CursorOverrideShape = EMouseCursor::Default;
		CursorOverrideWidget = nullptr;
		bIsCursorOverridden = false;
	}

	virtual ~UToolsContextCursorAPI() override = default;

	virtual EMouseCursor::Type GetCurrentCursorOverride() const 
	{
		return CursorOverrideShape;
	}

	virtual TSharedPtr<SWidget> GetCurrentCursorOverrideWidget() const 
	{
		return CursorOverrideWidget;
	}

	virtual bool IsCursorOverridden() const
	{
		return bIsCursorOverridden;
	}

	virtual void SetCursorOverride(EMouseCursor::Type CursorOverride)
	{
		bIsCursorOverridden = true;
		CursorOverrideShape = CursorOverride;
	}

	virtual void SetCursorOverrideWidget(const TSharedRef<SWidget>& InCursorOverrideWidget)
	{
		bIsCursorOverridden = true;
		CursorOverrideShape = EMouseCursor::Custom;
		CursorOverrideWidget = InCursorOverrideWidget;
	}
	
	void SetUseSoftwareCursor(const TOptional<bool>& InUseSoftwareCursorOverride)
	{
		UseSoftwareCursorOverride = InUseSoftwareCursorOverride;
	}
	
	TOptional<bool> GetUseSoftwareCursorOverride() const
	{
		return UseSoftwareCursorOverride;
	}

	/** Clears the custom widget, if any, resets the shape to Default and resets the overridden flag. */
	virtual void ClearCursorOverride() 
	{
		bIsCursorOverridden = false;
		CursorOverrideShape = EMouseCursor::Default;
		CursorOverrideWidget = nullptr;
		UseSoftwareCursorOverride.Reset();
	}
};

/**
 * Users of the Tools Framework need to implement IToolsContextQueriesAPI to provide
 * access to scene state information like the current UWorld, active USelections, etc.
 */
class IToolsContextQueriesAPI
{
public:
	virtual ~IToolsContextQueriesAPI() = default;

	/**
	 * @return the UWorld currently being targetted by the ToolsContext, default location for new Actors/etc
	 */
	virtual UWorld* GetCurrentEditingWorld() const = 0;

	/**
	 * Collect up current-selection information for the current scene state (ie what is selected in Editor, etc)
	 * @param StateOut this structure is populated with available state information
	 */
	virtual void GetCurrentSelectionState(FToolBuilderState& StateOut) const = 0;

	/**
	 * Request information about current view state
	 * @param StateOut this structure is populated with available state information
	 */
	virtual void GetCurrentViewState(FViewCameraState& StateOut) const = 0;

	/**
	 * Request current external coordinate-system setting. Defaults to World coordinates.
	 */
	virtual EToolContextCoordinateSystem GetCurrentCoordinateSystem() const { return EToolContextCoordinateSystem::World; }

	/**
	 * Request current external Gizmo Mode setting. Defaulting this to Combined gizmo as this was the default behavior before UE-5.2.
	 */
	virtual EToolContextTransformGizmoMode GetCurrentTransformGizmoMode() const { return EToolContextTransformGizmoMode::Combined; }

	/**
	 * Request current external snapping settings. Defaults to no snapping.
	 */
	virtual FToolContextSnappingConfiguration GetCurrentSnappingSettings() const { return FToolContextSnappingConfiguration(); }

	/**
	 * Many tools need standard types of materials that the user should provide (eg a vertex-color material, etc)
	 * @param MaterialType the type of material being requested
	 * @return Instance of material to use for this purpose
	 */
	virtual UMaterialInterface* GetStandardMaterial(EStandardToolContextMaterials MaterialType) const = 0;

	/**
	 * @returns the last valid viewport that was hovered, or nullptr.
	 */
	virtual FViewport* GetHoveredViewport() const = 0;

	/**
	 * @returns the last valid viewport that received some input, or nullptr.
	 */
	virtual FViewport* GetFocusedViewport() const = 0;
};




/** Level of severity of messages emitted by Tool framework */
UENUM()
enum class EToolMessageLevel
{
	/** Development message goes into development log */
	Internal = 0,
	/** User message should appear in user-facing log */
	UserMessage = 1,
	/** Notification message should be shown in a non-modal notification window */
	UserNotification = 2,
	/** Warning message should be shown in a non-modal notification window with panache */
	UserWarning = 3,
	/** Error message should be shown in a modal notification window */
	UserError = 4
};


/** Type of change we want to apply to a selection */
UENUM()
enum class ESelectedObjectsModificationType
{
	Replace = 0,
	Add = 1,
	Remove = 2,
	Clear = 3
};


/** Represents a change to a set of selected Actors and Components */
struct FSelectedObjectsChangeList
{
	/** How should this list be interpreted in the context of a larger selection set */
	ESelectedObjectsModificationType ModificationType;
	/** List of Actors */
	TArray<AActor*> Actors;
	/** List of Componets */
	TArray<UActorComponent*> Components;
};

UE_DEPRECATED(5.4, "Use FSelectedObjectsChangeList instead")
typedef FSelectedObjectsChangeList FSelectedOjectsChangeList;


/**
 * Users of the Tools Framework need to implement IToolsContextTransactionsAPI so that
 * the Tools have the ability to create Transactions and emit Changes. Note that this is
 * technically optional, but that undo/redo won't be supported without it.
 */
class IToolsContextTransactionsAPI
{
public:
	virtual ~IToolsContextTransactionsAPI() = default;

	/**
	 * Request that context display message information.
	 * @param Message text of message
	 * @param Level severity level of message
	 */
	virtual void DisplayMessage(const FText& Message, EToolMessageLevel Level) = 0;

	/** 
	 * Forward an invalidation request from Tools framework, to cause repaint/etc. 
	 * This is not always necessary but in some situations (eg in Non-Realtime mode in Editor)
	 * a redraw will not happen every frame. 
	 * See UInputRouter for options to enable auto-invalidation.
	 */
	virtual void PostInvalidation() = 0;
	
	/**
	 * Begin a Transaction, whatever this means in the current Context. For example in the
	 * Editor it means open a GEditor Transaction. You must call EndUndoTransaction() after calling this.
	 * @param Description text description of the transaction that could be shown to user
	 */
	virtual void BeginUndoTransaction(const FText& Description) = 0;

	/**
	 * Complete the Transaction. Assumption is that Begin/End are called in pairs.
	 */
	virtual void EndUndoTransaction() = 0;

	/**
	 * Cancel and revert a started Transaction.
	 */
	virtual void CancelUndoTransaction() = 0;

	/**
	 * Insert an FChange into the transaction history in the current Context. 
	 * It is safe but not necessary to call this between Begin/EndUndoTransaction;
	 * the FChange will be automatically inserted into a Transaction.
	 * @param TargetObject The UObject this Change is applied to
	 * @param Change The Change implementation
	 * @param Description text description of the transaction that could be shown to user
	 */
	virtual void AppendChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description) = 0;



	/**
	 * Request a modification to the current selected objects
	 * @param SelectionChange desired modification to current selection
	 * @return true if the selection change could be applied
	 */
	virtual bool RequestSelectionChange(const FSelectedObjectsChangeList& SelectionChange) = 0;

};

UENUM()
enum class EViewInteractionState {
	None = 0,
	Hovered = 1,
	Focused = 2,

	All = Hovered | Focused
};
ENUM_CLASS_FLAGS(EViewInteractionState);

/**
 * Users of the Tools Framework need to implement IToolsContextRenderAPI to allow
 * Tools, Indicators, and Gizmos to make low-level rendering calls for things like line drawing.
 * This API will be passed to eg UInteractiveTool::Render(), so access is only provided when it
 * makes sense to call the functions
 */
class IToolsContextRenderAPI
{
public:
	virtual ~IToolsContextRenderAPI() = default;

	/** @return Current PDI */
	virtual FPrimitiveDrawInterface* GetPrimitiveDrawInterface() = 0;

	/** @return Current FSceneView */
	virtual const FSceneView* GetSceneView() = 0;

	/** @return Current Camera State for this Render API */
	virtual FViewCameraState GetCameraState() = 0;

	/** @return Current interaction state of the view to render */
	virtual EViewInteractionState GetViewInteractionState() = 0;
};

