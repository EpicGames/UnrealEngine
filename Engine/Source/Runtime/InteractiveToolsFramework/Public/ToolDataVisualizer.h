// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolContextInterfaces.h"  // for FViewCameraState

class FPrimitiveDrawInterface;
class IToolsContextRenderAPI;
class FMaterialRenderProxy;

/**
 * FToolDataVisualizer is a utility class for Tool and Gizmo implementations
 * to use to draw 3D lines, points, etc.
 * 
 * Currently uses PDI drawing but may use different back-ends in the future
 */
class FToolDataVisualizer
{
public:
	/** Default color used for drawing lines */
	FLinearColor LineColor;   // = Red
	/** Default thickness used for drawing lines */
	float LineThickness = 1.0f;
	/** Default size used for drawing arrow heads */
	float ArrowHeadSize = 4.0f;

	/** Default color used for drawing points */
	FLinearColor PointColor;   // = Red
	/** Default size used for drawing points */
	float PointSize = 1.0f;

	/** Depth bias applied to lines */
	float DepthBias = 0.0f;
	/** Should lines be clipped by 3D geometry, or should they be drawn with quasi-transparency */
	bool bDepthTested = true;

public:
	INTERACTIVETOOLSFRAMEWORK_API FToolDataVisualizer();
	INTERACTIVETOOLSFRAMEWORK_API FToolDataVisualizer(const FToolDataVisualizer&);
	INTERACTIVETOOLSFRAMEWORK_API virtual ~FToolDataVisualizer();


	//
	// Frame initialization/cleanup
	// 


	/** This must be called every frame to allow Visualizer to extract necessary rendering data/objects */
	INTERACTIVETOOLSFRAMEWORK_API void BeginFrame(IToolsContextRenderAPI* RenderAPI, const FViewCameraState& CameraState);

	/** This must be called every frame to allow Visualizer to extract necessary rendering data/objects */
	INTERACTIVETOOLSFRAMEWORK_API void BeginFrame(IToolsContextRenderAPI* RenderAPI);

	/** this should be called at the end of every frame to allow for necessary cleanup */
	INTERACTIVETOOLSFRAMEWORK_API void EndFrame();


	//
	// Transform support
	// 

	/** Clear transform stack and push the given Transform */
	INTERACTIVETOOLSFRAMEWORK_API void SetTransform(const FTransform& Transform);
	/** Push a Transform onto the transform stack */
	INTERACTIVETOOLSFRAMEWORK_API void PushTransform(const FTransform& Transform);
	/** Pop a transform from the transform stack */
	INTERACTIVETOOLSFRAMEWORK_API void PopTransform();
	/** Clear the transform stack to identity */
	INTERACTIVETOOLSFRAMEWORK_API void PopAllTransforms();

	/** @return input Point transformed by transform stack */
	FVector TransformP(const FVector& Point)
	{
		return TotalTransform.TransformPosition(Point);
	}

	/** @return input Vector transformed by transform stack */
	FVector TransformV(const FVector& Vector)
	{
		return TotalTransform.TransformVector(Vector);
	}

	/** @return input Normal transformed by transform stack */
	FVector TransformN(const FVector& Normal)
	{
		// transform normal by a safe inverse scale + normalize, and a standard rotation (TODO: move implementation into FTransform)
		const FVector& S = TotalTransform.GetScale3D();
		double DetSign = FMath::Sign(S.X*S.Y*S.Z); // we only need to multiply by the sign of the determinant, rather than divide by it, since we normalize later anyway
		FVector SafeInvS(S.Y*S.Z*DetSign, S.X*S.Z*DetSign, S.X*S.Y*DetSign);
		return TotalTransform.TransformVectorNoScale((SafeInvS*Normal).GetSafeNormal());
	}


	//
	// Parameters
	//


	/** Update the default line color and thickness */
	void SetLineParameters(const FLinearColor& Color, float Thickness, float InArrowHeadSize = 4.0f)
	{
		LineColor = Color;
		LineThickness = Thickness;
		ArrowHeadSize = InArrowHeadSize;
	}

	/** Update the default point color and size */
	void SetPointParameters(const FLinearColor& Color, float Size)
	{
		PointColor = Color;
		PointSize = Size;
	}


	//
	// Drawing functions
	// 


	/** Draw a line with default parameters */
	template<typename PointType>
	void DrawLine(const PointType& A, const PointType& B)
	{
		InternalDrawTransformedLine(TransformP((FVector)A), TransformP((FVector)B), LineColor, LineThickness, bDepthTested);
	}

	/** Draw a line with the given Color, otherwise use default parameters */
	template<typename PointType>
	void DrawLine(const PointType& A, const PointType& B, const FLinearColor& Color)
	{
		InternalDrawTransformedLine(TransformP((FVector)A), TransformP((FVector)B), Color, LineThickness, bDepthTested);
	}

	/** Draw a line with the given Color and Thickness, otherwise use default parameters */
	template<typename PointType>
	void DrawLine(const PointType& A, const PointType& B, const FLinearColor& Color, float LineThicknessIn)
	{
		InternalDrawTransformedLine(TransformP((FVector)A), TransformP((FVector)B), Color, LineThicknessIn, bDepthTested);
	}

	/** Draw a line with the given parameters */
	template<typename PointType>
	void DrawLine(const PointType& A, const PointType& B, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawTransformedLine(TransformP((FVector)A), TransformP((FVector)B), Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw an arrow with default parameters */
	template<typename PointType>
	void DrawDirectionalArrow(const PointType& A, const PointType& B, const FVector& InNormal)
	{
		InternalDrawDirectionalArrow(TransformP((FVector)A), TransformP((FVector)B), TransformN(InNormal), LineColor, ArrowHeadSize, LineThickness, bDepthTested);
	}

	/** Draw a line with the given Color, otherwise use default parameters */
	template<typename PointType>
	void DrawDirectionalArrow(const PointType& A, const PointType& B, const FVector& InNormal, const FLinearColor& Color)
	{
		InternalDrawDirectionalArrow(TransformP((FVector)A), TransformP((FVector)B), TransformN(InNormal), Color, ArrowHeadSize, LineThickness, bDepthTested);
	}

	/** Draw a line with the given Color and Thickness, otherwise use default parameters */
	template<typename PointType>
	void DrawDirectionalArrow(const PointType& A, const PointType& B, const FVector& InNormal, const FLinearColor& Color, float InArrowHeadSize, float LineThicknessIn)
	{
		InternalDrawDirectionalArrow(TransformP((FVector)A), TransformP((FVector)B), TransformN(InNormal), Color, InArrowHeadSize, LineThicknessIn, bDepthTested);
	}

	/** Draw a line with the given parameters */
	template<typename PointType>
	void DrawDirectionalArrow(const PointType& A, const PointType& B, const FVector& InNormal, const FLinearColor& Color, float InArrowHeadSize, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawDirectionalArrow(TransformP((FVector)A), TransformP((FVector)B), TransformN(InNormal), Color, InArrowHeadSize, LineThicknessIn, bDepthTestedIn);
	}


	/** Draw a point with default parameters */
	template<typename PointType>
	void DrawPoint(const PointType& Position)
	{
		InternalDrawTransformedPoint(TransformP((FVector)Position), PointColor, PointSize, bDepthTested);
	}

	/** Draw a point with the given Color, otherwise use default parameters */
	template<typename PointType>
	void DrawPoint(const PointType& Position, const FLinearColor& Color)
	{
		InternalDrawTransformedPoint(TransformP((FVector)Position), Color, PointSize, bDepthTested);
	}

	/** Draw a point with the given parameters */
	template<typename PointType>
	void DrawPoint(const PointType& Position, const FLinearColor& Color, float PointSizeIn, bool bDepthTestedIn)
	{
		InternalDrawTransformedPoint(TransformP((FVector)Position), Color, PointSizeIn, bDepthTestedIn);
	}


	/** Draw a 3D circle at given position/normal with the given parameters. Tangent axes are defined internally */
	template<typename PointType>
	void DrawCircle(const PointType& Position, const PointType& Normal, float Radius, int Steps, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawCircle((FVector)Position, (FVector)Normal, Radius, Steps, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a 3D circle at given position/normal with the given parameters. Tangent axes are defined internally */
	template<typename PointType>
	void DrawViewFacingCircle(const PointType& Position, float Radius, int Steps, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawViewFacingCircle((FVector)Position, Radius, Steps, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a 3D arc at given position/normal with the given parameters. Tangent axes are defined internally */
	template<typename PointType>
	void DrawArc(const PointType& Position, const PointType& Normal, float Radius, int Steps, float MinAngle, float MaxAngle, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawArc((FVector)Position, (FVector)Normal, Radius, Steps, MinAngle, MaxAngle, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a 3D arc at given position/normal with the given parameters. Tangent axes are defined internally */
	template<typename PointType>
	void DrawViewFacingArc(const PointType& Position, float Radius, int Steps, float MinAngle, float MaxAngle, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawViewFacingArc((FVector)Position, Radius, Steps, MinAngle, MaxAngle, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a world-space X facing the viewer at the given position. */
	template<typename PointType>
	void DrawViewFacingX(const PointType& Position, float Width)
	{
		InternalDrawViewFacingX((FVector)Position, Width, LineColor, LineThickness, bDepthTested);
	}

	/** Draw a world-space X facing the viewer at the given position. */
	template<typename PointType>
	void DrawViewFacingX(const PointType& Position, float Width, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawViewFacingX((FVector)Position, Width, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a 3D cylinder, parameterized the same as the 3D circle but extruded by Height */
	template<typename PointType>
	void DrawWireCylinder(const PointType& Position, const PointType& Normal, float Radius, float Height, int Steps, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawWireCylinder((FVector)Position, (FVector)Normal, Radius, Height, Steps, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a 3D cylinder, parameterized the same as the 3D circle but extruded by Height */
	template<typename PointType>
	void DrawWireCylinder(const PointType& Position, const PointType& Normal, float Radius, float Height, int Steps)
	{
		InternalDrawWireCylinder((FVector)Position, (FVector)Normal, Radius, Height, Steps, LineColor, LineThickness, bDepthTested);
	}

	/** Draw a 3D box, parameterized the same as the 3D circle but extruded by Height */
	void DrawWireBox(const FBox& Box)
	{
		InternalDrawWireBox(Box, LineColor, LineThickness, bDepthTested);
	}

	void DrawWireBox(const FBox& Box, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawWireBox(Box, Color, LineThicknessIn, bDepthTestedIn);
	}

	template<typename PointType>
	void DrawSquare(const PointType& Center, const PointType& SideA, const PointType& SideB, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawSquare(Center, SideA, SideB, Color, LineThicknessIn, bDepthTestedIn);
	}

	template<typename PointType>
	void DrawSquare(const PointType& Center, const PointType& SideA, const PointType& SideB)
	{
		InternalDrawSquare(Center, SideA, SideB, LineColor, LineThickness, bDepthTested);
	}

	template<typename PointType>
	void DrawGrid(const PointType& Center, const PointType& Normal, const PointType& Size, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawGrid(Center, Normal, Size, Color, LineThicknessIn, bDepthTestedIn);
	}

	template<typename PointType>
	void DrawGrid(const PointType& Center, const PointType& Normal,  const PointType& Up, const PointType& Side, const PointType& Size, const FLinearColor& Color, float LineThicknessIn, bool bDepthTestedIn)
	{
		InternalDrawGrid(Center, Normal, Up, Side, Size, Color, LineThicknessIn, bDepthTestedIn);
	}

	/** Draw a 3D disc at given position/normal with the given parameters. Tangent axes are defined internally */
	template<typename PointType>
	void DrawDisc(const PointType& Position, const PointType& Normal, float Radius, int Steps, const FColor& Color, FMaterialRenderProxy* RenderProxy, bool bDepthTestedIn)
	{
		InternalDrawDisc(Position, Normal, Radius, Steps, Color, RenderProxy, bDepthTestedIn);
	}

protected:
	/** We use this for drawing, extracted in InitializeFrame */
	FPrimitiveDrawInterface* CurrentPDI = nullptr;
	
	FViewCameraState CameraState;
	bool bHaveCameraState;

	// screen-space line thicknesses and point sizes are multiplied by this value to try to normalize for variable thickness
	// that occurs at different FOVs. 
	float PDISizeScale = 1.0;


	TArray<FTransform> TransformStack;
	FTransform TotalTransform;

	// actually does the line drawing; assumes A and B are already transformed
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawTransformedLine(const FVector& A, const FVector& B, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the arrow drawing; assumes A and B are already transformed
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawDirectionalArrow(const FVector& A, const FVector& B, const FVector& InNormal, const FLinearColor& Color, const float InHeadSize, float LineThickness, bool bDepthTested);
	// actually does the point drawing; assumes Position is already transformed
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawTransformedPoint(const FVector& Position, const FLinearColor& Color, float PointSize, bool bDepthTested);


	// actually does the circle drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawCircle(const FVector& Position, const FVector& Normal, float Radius, int Steps, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the circle drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawViewFacingCircle(const FVector& Position, float Radius, int Steps, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the arc drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawArc(const FVector& InPosition, const FVector& InNormal, const float InRadius, int32 InSteps, const float InMinAngle, const float InMaxAngle, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the arc drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawViewFacingArc(const FVector& InPosition, float InRadius, int32 InSteps, const float InMinAngle, const float InMaxAngle, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the cylinder drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawWireCylinder(const FVector& Position, const FVector& Normal, float Radius, float Height, int Steps, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the box drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawWireBox(const FBox& Box, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the square drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawSquare(const FVector& Center, const FVector& SideA, const FVector& SideB, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the grid drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawGrid(const FVector& Center, const FVector& Normal, const FVector& Size, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the grid drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawGrid(const FVector& Center, const FVector& Normal, const FVector& Up, const FVector& Side, const FVector& Size, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the X drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawViewFacingX(const FVector& Position, float Width, const FLinearColor& Color, float LineThickness, bool bDepthTested);
	// actually does the disc drawing
	INTERACTIVETOOLSFRAMEWORK_API virtual void InternalDrawDisc(const FVector& Position, const FVector& Normal, float Radius, int Steps, const FColor& Color, FMaterialRenderProxy* RenderProxy, bool bDepthTested);
};
