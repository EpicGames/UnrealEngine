// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehaviorSet.h"
#include "InteractiveToolBuilder.h"
#include "HAL/Platform.h"

class UInteractiveToolsContext;

/** Describes a tool type to be registered into a UInteractiveToolManager. */
UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
struct FTool
{
	/** ITF Tool Identifier (ToolManager tool type id) */
	FString Name;

	/** Builder class registered with the ToolManager. */
	TSubclassOf<UInteractiveToolBuilder> BuilderClass;
};
/**
 * Declarative description of a ToolGroup.
 *
 * A ToolGroup describes:
 *  - which tool types should be registered (FTool entries),
 *  - which shared, context-wide features should be installed (Install/Uninstall callbacks),
 *  - which input behaviors should be active (built into a per-context UInputBehaviorSet),
 *  - optional context policy hooks (eg editor-only widget visibility).
 *
 * ToolGroups are intentionally *immutable descriptors* and must not own per-context runtime state.
 * All per-context state (registered tool ids, acquired features, behavior sets, etc.) is owned by
 * UToolGroupManager in FEnabledToolGroupState.
 */
UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
class FToolGroup
{
public:
	/** A shared, reference-counted feature installed into a UInteractiveToolsContext. */
	struct FFeature
	{
		/** Stable key identifying the feature within a tools context. */
		FName Key;

		/** Called on 0->1 ref-count transition. */
		TFunction<void(UInteractiveToolsContext&)> Install;

		/** Called on 1->0 ref-count transition. */
		TFunction<void(UInteractiveToolsContext&)> Uninstall;
	};

	/** Hook used to build the per-context input behaviors for this group. */
	using FBuildBehaviorsHook = TFunction<void(UInteractiveToolsContext& ToolsContext, UInputBehaviorSet& BehaviorSet)>;

	virtual ~FToolGroup() = default;

	/** Enumerate tools in this group. Enumeration continues while ToolFunc returns true. */
	virtual void EnumerateTools(TFunction<bool(const FTool&)> ToolFunc) const
	{
		if (!ToolFunc)
		{
			return;
		}
		for (const FTool& Tool : Tools)
		{
			if (!ToolFunc(Tool))
			{
				break;
			}
		}
	}

	/** Enumerate shared feature requirements for this group. */
	virtual void EnumerateFeatures(TFunction<bool(const FFeature&)> FeatureFunc) const
	{
		if (!FeatureFunc)
		{
			return;
		}
		for (const FFeature& Feature : Features)
		{
			if (!FeatureFunc(Feature))
			{
				break;
			}
		}
	}

	/**
	 * Populate per-context behaviors into the provided BehaviorSet.
	 *
	 * Behavior instances are generally not safe to share across tools contexts. The ToolGroupManager
	 * will allocate a new BehaviorSet per enabled group per context, call this hook, and then register
	 * a behavior source with the context's InputRouter.
	 */
	virtual void BuildInputBehaviors(UInteractiveToolsContext& ToolsContext, UInputBehaviorSet& BehaviorSet) const
	{
		if (BuildBehaviorsFunc)
		{
			BuildBehaviorsFunc(ToolsContext, BehaviorSet);
		}
	}


public:
	/** Declarative tool list. */
	TArray<FTool> Tools;

	/** Declarative feature requirements. */
	TArray<FFeature> Features;

	/** Optional behavior builder (called per enabled group per context). */
	FBuildBehaviorsHook BuildBehaviorsFunc;
};