// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolManager.h"
#include "ToolGroup.h"
#include "UObject/Object.h"
#include "ToolGroupManager.generated.h"

class UInteractiveToolsContext;
class IInputBehaviorSource;

#define UE_API INTERACTIVETOOLSFRAMEWORK_API

/** Stable identifier for a shared feature installed into a UInteractiveToolsContext. */
using FToolGroupFeatureKey = FName;

/**
 * Stored state for a single acquired feature key.
 *
 * Features are context-wide singleton-style services (target factories, snapping managers, etc.)
 * installed into a UInteractiveToolsContext and shared across multiple enabled tool groups.
 */
UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
struct FToolGroupFeatureEntry
{
	/**
	 * Current number of enabled ToolGroups that depend on this feature.
	 *
	 * RefCount is managed by FToolGroupFeatureRegistry and should not be manipulated directly.
	 */
	int32 RefCount = 0;

	/** Called only when RefCount transitions 0 -> 1. */
	TFunction<void(UInteractiveToolsContext&)> Install;

	/** Called only when RefCount transitions 1 -> 0. */
	TFunction<void(UInteractiveToolsContext&)> Uninstall;
};

/**
 * Reference-counted registry for context-wide features.
 *
 * This lives per UToolGroupManager (ie per UInteractiveToolsContext). Multiple groups can request
 * the same feature key; the first request installs it into the context and the final release
 * uninstalls it.
 */
UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
class FToolGroupFeatureRegistry
{
public:
	/**
	 * Acquire the feature. If this is the first acquire for Key (0 -> 1), Install will be executed and the
	 * provided Install/Uninstall callbacks will be stored as the feature definition for this context.
	 *
	 * If the feature is already acquired (RefCount > 0), the stored Install/Uninstall callbacks are retained.
	 * Callers may pass callbacks again (eg from immutable ToolGroup descriptors); these callbacks are ignored
	 * and only the ref-count is incremented.
	 *
	 * Note: Feature keys are expected to uniquely identify feature behavior within a context. If multiple
	 * callers provide different implementations for the same key, behavior is defined by the first acquire.
	 */
	void Acquire(
		FToolGroupFeatureKey Key,
		UInteractiveToolsContext& Context,
		TFunction<void(UInteractiveToolsContext&)> Install,
		TFunction<void(UInteractiveToolsContext&)> Uninstall);

	/** Release a previously acquired feature. Final release triggers Uninstall. */
	void Release(FToolGroupFeatureKey Key, UInteractiveToolsContext& Context);

	/** True if the feature key is currently acquired by at least one group. */
	bool IsAcquired(FToolGroupFeatureKey Key) const { return Features.Contains(Key); }

private:
	TMap<FToolGroupFeatureKey, FToolGroupFeatureEntry> Features;
};

/**
 * Per-context runtime state for an enabled ToolGroup.
 *
 * ToolGroups themselves are immutable descriptors. All per-context state is owned here by the
 * per-context UToolGroupManager, including:
 *  - which tool type ids were registered for this group,
 *  - which shared features were acquired for this group,
 *  - the per-context UInputBehaviorSet constructed for this group,
 *  - the behavior source adapter registered with the context's InputRouter.
 */
UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
USTRUCT()
struct FEnabledToolGroupState
{
	GENERATED_BODY()
	
	/** Per-context behavior set for this enabled group. */
	UPROPERTY(transient)
	TObjectPtr<UInputBehaviorSet> BehaviorSet = nullptr;
	
	/** Tool type identifiers registered with the ToolManager on behalf of this group. */
	TSet<FString> RegisteredToolTypeIds;

	/** Features acquired by this enabled group (explicit tracking for cleanup). */
	TSet<FToolGroupFeatureKey> AcquiredFeatures;

	/** Adapter registered with the InputRouter for this group. */
	TSharedPtr<IInputBehaviorSource> BehaviorSource;

	/** True if BehaviorSource has been registered with the owning context's InputRouter. */
	bool bRegisteredAsBehaviorSource = false;
};

/**
 * Manages ToolGroups for a single UInteractiveToolsContext.
 *
 * UToolGroupManager is stored per-tools-context and is responsible for 
 * materializing per-context runtime state from immutable tool group descriptors.
 *
 * Responsibilities:
 *  - register immutable ToolGroup descriptors by name,
 *  - enable/disable groups within this tools context,
 *  - register/unregister tool types with UInteractiveToolManager,
 *  - acquire/release shared features via FToolGroupFeatureRegistry,
 *  - build/register per-context input behaviors with the context's InputRouter,
 *  - answer queries like "which enabled group owns the currently active tool".
 */
UE_EXPERIMENTAL(5.8, "Experimental, API may change in the future")
UCLASS(Transient)
class UToolGroupManager : public UObject
{
	GENERATED_BODY()

public:
	/** Initialize internal state. Must be called before registering/enabling groups. */
	UE_API void Initialize();
	
	/** Shutdown and disable all enabled tool groups, unregistering their tool types. */
	UE_API void Shutdown();
	
	/** Register an immutable ToolGroup descriptor that can later be enabled in this context. */
	UE_API bool RegisterToolGroup(FName GroupName, TSharedPtr<const FToolGroup> ToolGroup);

	/** Returns true if a tool group descriptor with this name is registered. */
	UE_API bool HasToolGroup(FName GroupName) const;

	/** Enable (materialize + register) the named tool group in this context. */
	UE_API bool EnableToolGroup(FName GroupName);

	/** Disable (unregister + cleanup) the named tool group in this context. */
	UE_API bool DisableToolGroup(FName GroupName);

	/** Disable all enabled tool groups in this context. */
	UE_API void DisableAllToolGroups();

	// Lifecycle
	/** Returns true if the named group is currently enabled in this context. */
	UE_API bool IsToolGroupEnabled(FName GroupName) const;

	/** Returns the names of currently enabled groups. */
	UE_API TArray<FName> GetEnabledToolGroupNames() const;


	// Tool Ownership
	/** If a tool is currently active, returns the name of the enabled group that registered it. */
	UE_API TOptional<FName> FindEnabledGroupOwningActiveTool(EToolSide Side) const;

	/** Returns true if any tool groups are enabled in this context. */
	UE_API bool HasAnyEnabledToolGroups();

private:
	void AcquireGroupFeatures(const FToolGroup& ToolGroup, FEnabledToolGroupState& State, UInteractiveToolsContext& Context);
	void ReleaseAllFeaturesForGroup(FEnabledToolGroupState& State, UInteractiveToolsContext& Context);

	void RegisterGroupTools(const FToolGroup& ToolGroup, FEnabledToolGroupState& State, UInteractiveToolManager& ToolManager);
	void UnregisterGroupTools(FEnabledToolGroupState& State, UInteractiveToolManager& ToolManager);

	void RegisterGroupBehaviors(const FToolGroup& ToolGroup, FEnabledToolGroupState& State, UInteractiveToolsContext& Context);
	void UnregisterGroupBehaviors(FEnabledToolGroupState& State, UInteractiveToolsContext& Context);

	UInteractiveToolsContext* GetOwningContext() const;

private:
	/** Registered immutable tool group descriptors by name. */
	TMap<FName, TSharedPtr<const FToolGroup>> RegisteredGroups;

	/** Per-context runtime state for enabled groups. */
	UPROPERTY(Transient)
	TMap<FName, FEnabledToolGroupState> EnabledGroups;

	/** Ref-counted context-wide feature registry. */
	FToolGroupFeatureRegistry FeatureRegistry;
};

#undef UE_API