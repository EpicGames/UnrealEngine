// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolTargets/ToolTarget.h"
#include "ToolContextInterfaces.h"

#include "ToolTargetManager.generated.h"

// TODO: Do we need more control over factory order to prioritize which gets used if we can
// make multiple qualifying targets? It should theoretically not matter, but in practice, one
// target or another could be more efficient for certain tasks. This is probably only worth
// thinking about once we encounter it.

// NOTE FOR IMPLEMENTING CACHING: Someday we will probably write a tool target manager that
// caches tool targets and gives prebuilt ones when possible to avoid any extra conversions.
// When implementing this, note that currently, some targets store converted versions of their
// source objects without a way to determine whether their stored conversion is out of date.
// The cached target will then give an incorrect result if the source object has been
// modified after caching.
// One way to deal with this is to have subclasses of UToolTarget that give the degree of
// cacheability- fully cacheable (i.e., detects changes and rebuilds if necessary), cacheable
// only if changes are made through that tool target (i.e. target manager has to clear it if
// object may have changed; may choose not to have this option), or not cacheable. 

/**
 * The tool target manager converts input objects into tool targets- objects that
 * can expose various interfaces that tools need to operate on them.
 *
 * Someday, the tool target manager may implement caching of targets.
 * 
 * See the class comment for UToolTarget for more info.
 */
UCLASS(Transient, MinimalAPI)
class UToolTargetManager : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * @return true if ToolManager is currently active, ie between Initialize() and Shutdown()
	 */
	bool IsActive() const { return bIsActive; }

	/**
	 * Add a new ToolTargetFactory
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void AddTargetFactory(UToolTargetFactory* Factory);

	/**
	 * Remove existing ToolTargetFactories by predicate
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void RemoveTargetFactoriesByPredicate(TFunctionRef<bool(UToolTargetFactory*)> Predicate);

	/**
	 * Find the first ToolTargetFactory that passes the Predicate function
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UToolTargetFactory* FindFirstFactoryByPredicate(TFunctionRef<bool(UToolTargetFactory*)> Predicate);

	/**
	 * Find the first ToolTargetFactory of a given type
	 */
	template<typename CastToType>
	CastToType* FindFirstFactoryByType()
	{
		UToolTargetFactory* Found = FindFirstFactoryByPredicate([](UToolTargetFactory* Factory) { return Cast<CastToType>(Factory) != nullptr; });
		return (Found != nullptr) ? Cast<CastToType>(Found) : nullptr;
	}



	/** Examines stored target factories to see if one can build the requested type of target. */
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetRequirements) const;

	/** 
	 * Uses one of the stored factories to build a tool target out of the given input object
	 * that satisfies the given requirements. If multiple factories are capable of building a 
	 * qualifying target, the first encountered one will be used. If none are capable, a nullptr 
	 * is returned.
	 * 
	 * @return Tool target that staisfies given requirements, or nullptr if none could be created.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetRequirements);

	/** Much like BuildTarget, but casts the target to the template argument before returning. */
	template<typename CastToType>
	CastToType* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetType)
	{
		UToolTarget* Result = BuildTarget(SourceObject, TargetType);
		return (Result != nullptr) ? Cast<CastToType>(Result) : nullptr;
	}

	/**
	 * Looks through the currently selected components and actors and counts the number of
	 * inputs that could be used to create qualifying tool targets.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 CountSelectedAndTargetable(const FToolBuilderState& SceneState,
		const FToolTargetTypeRequirements& TargetRequirements) const;

	/**
	 * Looks through the currently selected components and actors and counts the number of
	 * inputs that could be used to create qualifying tool targets.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual void EnumerateSelectedAndTargetableComponents(const FToolBuilderState& SceneState,
		const FToolTargetTypeRequirements& TargetRequirements,
		TFunctionRef<void(UActorComponent*)> ComponentFunc) const;

	/**
	 * Looks through the currently selected components and actors and counts the number of
	 * inputs that could be used to create qualifying tool targets with an additional test predicate.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 CountSelectedAndTargetableWithPredicate(const FToolBuilderState& SceneState,
		const FToolTargetTypeRequirements& TargetRequirements,
		TFunctionRef<bool(UActorComponent&)> ComponentPred) const;


	/**
	 * Looks through the currently selected components and actors and builds a target out of
	 * the first encountered element that satisfies the requirements. 
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UToolTarget* BuildFirstSelectedTargetable(const FToolBuilderState& SceneState,
		const FToolTargetTypeRequirements& TargetRequirements);

	/**
	 * Looks through the current selected components and actors and builds all targets that
	 * satisfy the requirements.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual TArray<TObjectPtr<UToolTarget>> BuildAllSelectedTargetable(const FToolBuilderState& SceneState,
		const FToolTargetTypeRequirements& TargetRequirements);

	/** Initialize the ToolTargetManager. UInteractiveToolsContext calls this, you should not. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Initialize();

	/** Shutdown the ToolTargetManager. Called by UInteractiveToolsContext. */
	INTERACTIVETOOLSFRAMEWORK_API virtual void Shutdown();

	/** Additional filter function that will be applied in CanBuildTarget and BuildTarget. */
	virtual void SetInputFilterFunction(TFunction<bool(const UObject*)> ShouldAllowBuildingFromInputObject) 
	{ 
		InputFilterFunction = ShouldAllowBuildingFromInputObject; 
	}

protected:

	// This should be removed once tools are transitioned to using tool targets, and the
	// functions in ComponentSourceInterfaces.h do not exist. For now, this is here to call
	// Initialize().
	friend INTERACTIVETOOLSFRAMEWORK_API void AddFactoryToDeprecatedToolTargetManager(UToolTargetFactory* Factory);

	UToolTargetManager(){};

	/** This flag is set to true on Initialize() and false on Shutdown(). */
	bool bIsActive = false;

	UPROPERTY()
	TArray<TObjectPtr<UToolTargetFactory>> Factories;

	// More state will go here if the manager deals with target caching.
	
private:
	TFunction<bool(const UObject*)> InputFilterFunction;
};
