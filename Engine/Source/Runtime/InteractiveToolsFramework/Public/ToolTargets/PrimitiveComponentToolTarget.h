// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargets/ToolTarget.h"

#include "PrimitiveComponentToolTarget.generated.h"

/** 
 * A tool target to share some reusable code for tool targets that are
 * backed by primitive components. 
 */
UCLASS(Transient, MinimalAPI)
class UPrimitiveComponentToolTarget : public UToolTarget, public IPrimitiveComponentBackedTarget
{
	GENERATED_BODY()
public:

	// UToolTarget
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsValid() const override;

	// IPrimitiveComponentBackedTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual UPrimitiveComponent* GetOwnerComponent() const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual USceneComponent* GetOwnerSceneComponent() const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual AActor* GetOwnerActor() const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetOwnerVisibility(bool bVisible) const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetWorldTransform() const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual bool HitTestComponent(const FRay& WorldRay, FHitResult& OutHit) const override;

	// UObject
	INTERACTIVETOOLSFRAMEWORK_API void BeginDestroy() override;

protected:
	friend class UPrimitiveComponentToolTargetFactory;
	
	INTERACTIVETOOLSFRAMEWORK_API void InitializeComponent(UPrimitiveComponent* ComponentIn);

	TWeakObjectPtr<UPrimitiveComponent> Component;

private:
#if WITH_EDITOR
	INTERACTIVETOOLSFRAMEWORK_API void OnObjectsReplaced(const FCoreUObjectDelegates::FReplacementObjectMap& Map);
#endif
};

/**
 * Factory for UPrimitiveComponentToolTarget to be used by the target manager.
 */
UCLASS(Transient, MinimalAPI)
class UPrimitiveComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()
public:
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};

