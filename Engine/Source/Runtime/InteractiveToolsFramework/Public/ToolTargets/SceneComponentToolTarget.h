// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargets/ToolTarget.h"

#include "SceneComponentToolTarget.generated.h"

/** 
 * A tool target to share some reusable code for tool targets that are
 * backed by scene components
 */
UCLASS(Transient, MinimalAPI)
class USceneComponentToolTarget : public UToolTarget, public ISceneComponentBackedTarget
{
	GENERATED_BODY()
public:

	// UToolTarget
	INTERACTIVETOOLSFRAMEWORK_API virtual bool IsValid() const override;

	// ISceneComponentBackedTarget implementation
	INTERACTIVETOOLSFRAMEWORK_API virtual USceneComponent* GetOwnerSceneComponent() const;
	INTERACTIVETOOLSFRAMEWORK_API virtual AActor* GetOwnerActor() const;
	INTERACTIVETOOLSFRAMEWORK_API virtual void SetOwnerVisibility(bool bVisible) const;
	INTERACTIVETOOLSFRAMEWORK_API virtual FTransform GetWorldTransform() const;

	// UObject
	INTERACTIVETOOLSFRAMEWORK_API void BeginDestroy() override;

protected:
	friend class USceneComponentToolTargetFactory;
	
	INTERACTIVETOOLSFRAMEWORK_API void InitializeComponent(USceneComponent* ComponentIn);

	TWeakObjectPtr<USceneComponent> Component;

private:
#if WITH_EDITOR
	INTERACTIVETOOLSFRAMEWORK_API void OnObjectsReplaced(const FCoreUObjectDelegates::FReplacementObjectMap& Map);
#endif
};

/**
 * Factory for USceneComponentToolTarget to be used by the target manager.
 */
UCLASS(Transient, MinimalAPI)
class USceneComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()
public:
	INTERACTIVETOOLSFRAMEWORK_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;
	INTERACTIVETOOLSFRAMEWORK_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};

