// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ToolTarget.generated.h"

class FToolTargetTypeRequirements;
class UClass;

/**
 * A tool target is a stand-in object that a tool can operate on. It exposes the necessary
 * interfaces to the tool.
 *
 * There are two intended purposes of the tool target system:
 * 1. Allow tools to operate on arbitrary objects as long as they can be made to provide the 
 *   tool with the necessary inputs. For instance, a mesh editing tool should be able to operate
 *   on skeletal, static, volume, and other mesh as long as the target manager has a registered
 *   factory that can use that type of mesh to create a suitable target.
 * 2. (not yet used) Help cache tool inputs. I.e., if a tool requires an expensive
 *   conversion before it can work on an item, the converted result can be stored in the
 *   tool target which can be cached by the target manager and provided the next time the
 *   same type of target is requested for that item.
 * 
 * Given an object, tool builders usually ask the target manager to turn it into a target that
 * has the interfaces the tools needs. The tools cast the target to those interfaces to use
 * them.
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UToolTarget : public UObject
{
	GENERATED_BODY()
public:

	/** @return true if target is still valid. May become invalid for various reasons (eg Component was deleted out from under us) */
	virtual bool IsValid() const PURE_VIRTUAL(UToolTarget::IsValid, return false;);
};


/**
 * A structure used to specify the requirements of a tool for its target. E.g., a tool
 * may need a target that has interfaces x,y,z.
 */
class FToolTargetTypeRequirements
{
public:

	TArray<const UClass*, TInlineAllocator<5>> Interfaces;

	FToolTargetTypeRequirements()
	{
	}

	explicit FToolTargetTypeRequirements(const UClass* Interface0)
	{
		Interfaces.Add(Interface0);
	}

	explicit FToolTargetTypeRequirements(const TArray<const UClass*>& InterfacesIn)
	{
		Interfaces = InterfacesIn;
	}

	FToolTargetTypeRequirements& Add(UClass* Interface)
	{
		Interfaces.Add(Interface);
		return *this;
	}

	INTERACTIVETOOLSFRAMEWORK_API bool AreSatisfiedBy(UClass* Class) const;

	INTERACTIVETOOLSFRAMEWORK_API bool AreSatisfiedBy(UToolTarget* ToolTarget) const;
};


/**
 * Base class for factories of tool targets, which let a tool manager build targets
 * out of inputs without knowing anything about the inputs itself, as long as it
 * has a factory registered that is able to process the input.
 */
UCLASS(Transient, Abstract, MinimalAPI)
class UToolTargetFactory : public UObject
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const PURE_VIRTUAL(UToolTargetFactory::CanBuildTarget, return false;);
	virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) PURE_VIRTUAL(UToolTargetFactory::BuildTarget, return nullptr;);


	// The following methods are used when building from multiple input objects. If not overriden, they will
	//  call BuildTarget/CanBuildTarget on each object in sequence, but they can be overriden to create
	//  combined tool targets out of multiple inputs.
	
	/**
	 * Called to see if a factory could make one or more targets out of the given array of input objects, 
	 *  if multiple are provided.
	 * 
	 * @param WouldBeUsedOut Set to be 1:1 with InputObjects, with each entry set to true if that input object
	 *   would be used in the creation of a tool target
	 * @return Number of targets that would be returned by the equivalent BuildTargets call.
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual int32 CanBuildTargets(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WouldBeUsedOut);
	
	/**
	 * Called to build one or more targets out of the given array of input objects, if multiple are provided.
	 * 
	 * @param WouldBeUsedOut Set to be 1:1 with InputObjects, with each entry set to true if that input object
	 *   was used in the creation of a tool target
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual TArray<UToolTarget*> BuildTargets(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WasUsedOut);

	/**
	 * Called by BuildFirstSelectedTargetable, intended to create just one target out of the given input.
	 * 
	 * @param WouldBeUsedOut Set to be 1:1 with InputObjects, with each entry set to true if that input object
	 *   was used in the creation of a tool target
	 */
	INTERACTIVETOOLSFRAMEWORK_API virtual UToolTarget* BuildFirstTarget(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WasUsedOut);
};
